{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Chaos Cloth Asset",
	"Description": "Pattern based cloth asset using the Chaos Cloth simulation.",
	"Category": "Physics",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "ChaosClothAsset",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "ChaosClothAssetEngine",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "ChaosClothAssetTools",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [ "Win64", "Mac", "Linux" ]
		}
	],
	"Plugins": [
		{
			"Name": "ChaosCloth",
			"Enabled": true
		},
		{
			"Name": "GeometryCache",
			"Enabled": true
		},
		{
			"Name": "Dataflow",
			"Enabled": true
		}
	]
}
