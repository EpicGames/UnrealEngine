// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ChaosClothAsset : ModuleRules
{
	public ChaosClothAsset(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"GeometryCore",
				"MeshConversion",
				"ClothingSystemRuntimeCommon",
				"RenderCore"
			}
		);
		SetupModulePhysicsSupport(Target);
	}
}
