// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"

namespace UE::Chaos::ClothAsset
{
	class FClothAssetModule : public IModuleInterface
	{
	public:
		virtual void StartupModule() override
		{
		}

		virtual void ShutdownModule() override
		{
		}
	};
}

IMPLEMENT_MODULE(UE::Chaos::ClothAsset::FClothAssetModule, ChaosClothAsset);
