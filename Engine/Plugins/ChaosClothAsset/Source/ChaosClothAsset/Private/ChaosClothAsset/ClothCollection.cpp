// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ClothCollection.h"
#include "ChaosClothAsset/ClothCollectionGroup.h"
#include "ChaosClothAsset/ClothCollectionAttribute.h"
#include "GeometryCollection/ManagedArrayCollection.h"

namespace UE::Chaos::ClothAsset::Private
{
	// Note: static variables that depend on other static variables from other translation units must be accessed through getter functions,
	// otherwise static variables can contain uninitialized values.

	// Lods Group
	static const TArray<FName>& GetLodsGroupAttributes()
	{
		static const TArray<FName> LodsGroupAttributes =
		{
			ClothCollectionAttribute::PhysicsAssetSoftObjectPathName,
			ClothCollectionAttribute::SkeletalMeshSoftObjectPathName,
			ClothCollectionAttribute::ReferenceBoneName,
			ClothCollectionAttribute::CopyAllUserAttributesToSimModel,
		};
		return LodsGroupAttributes;
	}

	// Solvers Group
	static const TArray<FName>& GetSolverGroupAttributes()
	{
		static const TArray<FName> SolverGroupAttributes =
		{
			ClothCollectionAttribute::SolverGravity,
			ClothCollectionAttribute::SolverAirDamping,
			ClothCollectionAttribute::SolverSubSteps,
			ClothCollectionAttribute::SolverTimeStep,
		};
		return SolverGroupAttributes;
	}

	// Fabrics Group
	static const TArray<FName>& GetSimFabricGroupAttributes()
	{
		static const TArray<FName> SimFabricGroupAttributes =
		{
			ClothCollectionAttribute::FabricBendingStiffness,
			ClothCollectionAttribute::FabricBucklingStiffness,
			ClothCollectionAttribute::FabricStretchStiffness,
			ClothCollectionAttribute::FabricBucklingRatio,
			ClothCollectionAttribute::FabricDensity,
			ClothCollectionAttribute::FabricFriction,
			ClothCollectionAttribute::FabricDamping,
			ClothCollectionAttribute::FabricPressure,
			ClothCollectionAttribute::FabricLayer,
			ClothCollectionAttribute::FabricCollisionThickness,
		};
		return SimFabricGroupAttributes;
	}

	// Seam Group
	static const TArray<FName>& GetSeamsGroupAttributes()
	{
		static const TArray<FName> SeamsGroupAttributes =
		{
			ClothCollectionAttribute::SeamStitchStart,
			ClothCollectionAttribute::SeamStitchEnd
		};
		return SeamsGroupAttributes;
	}

	// Seam Stitches Group
	static const TArray<FName>& GetSeamStitchesGroupAttributes()
	{
		static const TArray<FName> SeamStitchesGroupAttributes =
		{
			ClothCollectionAttribute::SeamStitch2DEndIndices,
			ClothCollectionAttribute::SeamStitch3DIndex
		};
		return SeamStitchesGroupAttributes;
	}

	// Sim Patterns Group
	static const TArray<FName>& GetSimPatternsGroupAttributes()
	{
		static const TArray<FName> SimPatternsGroupAttributes =
		{
			ClothCollectionAttribute::SimVertices2DStart,
			ClothCollectionAttribute::SimVertices2DEnd,
			ClothCollectionAttribute::SimFacesStart,
			ClothCollectionAttribute::SimFacesEnd,
			ClothCollectionAttribute::SimPatternFabric,
		};
		return SimPatternsGroupAttributes;
	}

	// Render Patterns Group
	static const TArray<FName>& GetRenderPatternsGroupAttributes()
	{
		static const TArray<FName> RenderPatternsGroupAttributes =
		{
			ClothCollectionAttribute::RenderVerticesStart,
			ClothCollectionAttribute::RenderVerticesEnd,
			ClothCollectionAttribute::RenderFacesStart,
			ClothCollectionAttribute::RenderFacesEnd,
			ClothCollectionAttribute::RenderDeformerNumInfluences,
			ClothCollectionAttribute::RenderMaterialSoftObjectPathName
		};
		return RenderPatternsGroupAttributes;
	}

	// Sim Faces Group
	static const TArray<FName>& GetSimFacesGroupAttributes()
	{
		static const TArray<FName> SimFacesGroupAttributes =
		{
			ClothCollectionAttribute::SimIndices2D,
			ClothCollectionAttribute::SimIndices3D
		};
		return SimFacesGroupAttributes;
	}

	// Sim Vertices 2D Group
	static const TArray<FName>& GetSimVertices2DGroupAttributes()
	{
		static const TArray<FName> SimVertices2DGroupAttributes =
		{
			ClothCollectionAttribute::SimPosition2D,
			ClothCollectionAttribute::SimVertex3DLookup,
			ClothCollectionAttribute::SimImportVertexID,
		};
		return SimVertices2DGroupAttributes;
	}

	// Sim Vertices 3D Group
	// NOTE: if you add anything here, you need to implement how to merge it 
	// in CollectionClothSeamFacade. Otherwise, the data in the lowest vertex index 
	// will survive and the other data will be lost whenever seams are added.
	static const TArray<FName>& GetSimVertices3DGroupAttributes()
	{
		static const TArray<FName> SimVertices3DGroupAttributes =
		{
			ClothCollectionAttribute::SimPosition3D,
			ClothCollectionAttribute::PreResizedSimPosition3D,
			ClothCollectionAttribute::SimNormal,
			ClothCollectionAttribute::SimBoneIndices,
			ClothCollectionAttribute::SimBoneWeights,
			ClothCollectionAttribute::SimVertex2DLookup,
			ClothCollectionAttribute::SeamStitchLookup,
			ClothCollectionAttribute::SimCustomResizingBlend,
		};
		return SimVertices3DGroupAttributes;
	}

	static const TArray<FName>& GetSimAccessoryMeshVertices3DGroupAttributePrefixes()
	{
		static const TArray<FName> SimAccessoryMeshVertices3DGroupAttributePrefixes =
		{
			ClothCollectionAttribute::SimAccessoryMeshPosition3DPrefix,
			ClothCollectionAttribute::SimAccessoryMeshNormalPrefix,
			ClothCollectionAttribute::SimAccessoryMeshBoneIndicesPrefix,
			ClothCollectionAttribute::SimAccessoryMeshBoneWeightsPrefix,
		};
		return SimAccessoryMeshVertices3DGroupAttributePrefixes;
	}

	//~ Sim Morph Targets Group
	static const TArray<FName>& GetSimMorphTargetsGroupAttributes()
	{
		static const TArray<FName> SimMorphTargetsGroupAttributes =
		{
			ClothCollectionAttribute::SimMorphTargetName,
			ClothCollectionAttribute::SimMorphTargetVerticesStart,
			ClothCollectionAttribute::SimMorphTargetVerticesEnd,
		};
		return SimMorphTargetsGroupAttributes;
	}

	//~ Sim Morph Targets Vertices Group
	static const TArray<FName>& GetSimMorphTargetVerticesGroupAttributes()
	{
		static const TArray<FName> SimMorphTargetVerticesGroupAttributes =
		{
			ClothCollectionAttribute::SimMorphTargetPositionDelta,
			ClothCollectionAttribute::SimMorphTargetTangentZDelta,
			ClothCollectionAttribute::SimMorphTargetSimVertex3DIndex,
		};
		return SimMorphTargetVerticesGroupAttributes;
	}

	// Render Faces Group
	static const TArray<FName>& GetRenderFacesGroupAttributes()
	{
		static const TArray<FName> RenderFacesGroupAttributes =
		{
			ClothCollectionAttribute::RenderIndices,
		};
		return RenderFacesGroupAttributes;
	}

	// Render Vertices Group
	static const TArray<FName>& GetRenderVerticesGroupAttributes()
	{
		static const TArray<FName> RenderVerticesGroupAttributes =
		{
			ClothCollectionAttribute::RenderPosition,
			ClothCollectionAttribute::RenderNormal,
			ClothCollectionAttribute::RenderTangentU,
			ClothCollectionAttribute::RenderTangentV,
			ClothCollectionAttribute::RenderUVs,
			ClothCollectionAttribute::RenderColor,
			ClothCollectionAttribute::RenderBoneIndices,
			ClothCollectionAttribute::RenderBoneWeights,
			ClothCollectionAttribute::RenderDeformerPositionBaryCoordsAndDist,
			ClothCollectionAttribute::RenderDeformerNormalBaryCoordsAndDist,
			ClothCollectionAttribute::RenderDeformerTangentBaryCoordsAndDist,
			ClothCollectionAttribute::RenderDeformerSimIndices3D,
			ClothCollectionAttribute::RenderDeformerWeight,
			ClothCollectionAttribute::RenderDeformerSkinningBlend,
			ClothCollectionAttribute::RenderCustomResizingBlend,
		};
		return RenderVerticesGroupAttributes;
	}

	//~ Group Resizing Group
	static const TArray<FName>& GetGroupResizingGroupAttributes()
	{
		static const TArray<FName> GroupResizingGroupAttributes =
		{
			ClothCollectionAttribute::CustomResizingRegionSet,
			ClothCollectionAttribute::CustomResizingRegionType
		};
		return GroupResizingGroupAttributes;
	}

	//~ Sim Accessory Meshes Group
	static const TArray<FName>& GetSimAccessoryMeshesGroupAttributes()
	{
		static const TArray<FName> SimAccessoryMeshesGroupAttributes =
		{
			ClothCollectionAttribute::SimAccessoryMeshName,
			ClothCollectionAttribute::SimAccessoryMeshPosition3DAttribute,
			ClothCollectionAttribute::SimAccessoryMeshNormalAttribute,
			ClothCollectionAttribute::SimAccessoryMeshBoneIndicesAttribute,
			ClothCollectionAttribute::SimAccessoryMeshBoneWeightsAttribute,
		};
		return SimAccessoryMeshesGroupAttributes;
	}

	// All fixed attributes for this collection
	static const TMap<FName, TArray<FName>>& GetFixedAttributeNamesMap()
	{
		static const TMap<FName, TArray<FName>> FixedAttributeNamesMap =
		{
			{ ClothCollectionGroup::Lods, GetLodsGroupAttributes() },
			{ ClothCollectionGroup::Seams, GetSeamsGroupAttributes() },
			{ ClothCollectionGroup::SeamStitches, GetSeamStitchesGroupAttributes() },
			{ ClothCollectionGroup::SimPatterns, GetSimPatternsGroupAttributes() },
			{ ClothCollectionGroup::RenderPatterns, GetRenderPatternsGroupAttributes() },
			{ ClothCollectionGroup::SimFaces, GetSimFacesGroupAttributes() },
			{ ClothCollectionGroup::SimVertices2D, GetSimVertices2DGroupAttributes() },
			{ ClothCollectionGroup::SimVertices3D, GetSimVertices3DGroupAttributes() },
			{ ClothCollectionGroup::SimMorphTargets, GetSimMorphTargetsGroupAttributes() },
			{ ClothCollectionGroup::SimMorphTargetVertices, GetSimMorphTargetVerticesGroupAttributes() },
			{ ClothCollectionGroup::RenderFaces, GetRenderFacesGroupAttributes() },
			{ ClothCollectionGroup::RenderVertices, GetRenderVerticesGroupAttributes() },
			{ ClothCollectionGroup::Fabrics, GetSimFabricGroupAttributes() },
			{ ClothCollectionGroup::Solvers, GetSolverGroupAttributes() },
			{ ClothCollectionGroup::CustomResizingRegions, GetGroupResizingGroupAttributes() },
			{ ClothCollectionGroup::SimAccessoryMeshes, GetSimAccessoryMeshesGroupAttributes() },
		};
		return FixedAttributeNamesMap;
	}

	// All fixed attribute prefixes for this collection (see SimAccessoryMesh SimVertex3D)
	static const TMap<FName, TArray<FName>>& GetFixedAttributePrefixNamesMap()
	{
		static const TMap<FName, TArray<FName>> FixedAttributePrefixNamesMap = {
			{ ClothCollectionGroup::SimVertices3D, GetSimAccessoryMeshVertices3DGroupAttributePrefixes() },
		};
		return FixedAttributePrefixNamesMap;
	}

	// All paintable attributes for this collection
	static const TMap<FName, TArray<FName>>& GetUserAccessibleAttributeNamesMap()
	{
		static const TMap<FName, TArray<FName>> UserAccessibleAttributeNamesMap =
		{
			{ ClothCollectionGroup::SimVertices3D, { ClothCollectionAttribute::SimCustomResizingBlend } },  // Can be painted if required
			{ ClothCollectionGroup::RenderVertices, { ClothCollectionAttribute::RenderDeformerSkinningBlend, ClothCollectionAttribute::RenderCustomResizingBlend } },  // Can be painted if required
		};
		return UserAccessibleAttributeNamesMap;
	}

	static bool NamesHaveMatchingPrefix(const FName& Name, const FName& Prefix)
	{
#if UE_FNAME_OUTLINE_NUMBER
		// Comparison index includes number. Create a new FName without a number and compare it to prefix.
		FName NumberlessName(Name);
		NumberlessName.SetNumber(NAME_NO_NUMBER_INTERNAL);
		return NumberlessName == Prefix;
#else
		// Comparison index does not include number
		return Name.GetComparisonIndex() == Prefix.GetComparisonIndex();
#endif
	}

	template<typename T>
	FName AddUniquePrefixedAttribute(const FName& Group, const FName& AttributePrefix, const TManagedArray<FName>* ExistingAttributes, TSharedRef<FManagedArrayCollection> Collection)
	{
		int32 MaxExistingNumber = 0;
		if (ExistingAttributes)
		{
			for (const FName& ExistingAttr : *ExistingAttributes)
			{
				if (ExistingAttr == NAME_None)
				{
					continue;
				}
				if (ensure(NamesHaveMatchingPrefix(ExistingAttr, AttributePrefix)))
				{
					MaxExistingNumber = FMath::Max(MaxExistingNumber, ExistingAttr.GetNumber());
				}
			}
		}

		const FName NewAttributeName(AttributePrefix, MaxExistingNumber + 1);
		check(!Collection->HasAttribute(NewAttributeName, Group));

		Collection->AddAttribute<T>(NewAttributeName, Group);
		return NewAttributeName;
	}
}  // End namespace UE::Chaos::ClothAsset::Private

namespace UE::Chaos::ClothAsset
{
	template<typename T UE_REQUIRES_DEFINITION(std::is_same_v<T, FManagedArrayCollection> || std::is_same_v<T, const FManagedArrayCollection>)>
	FConstClothCollection::FConstClothCollection(const TSharedRef<T>& InManagedArrayCollection, TProtectedInit)
		: ManagedArrayCollection(InManagedArrayCollection)
	{
		// LODs Group
		PhysicsAssetSoftObjectPathName = ManagedArrayCollection->FindAttribute<FSoftObjectPath>(ClothCollectionAttribute::PhysicsAssetSoftObjectPathName, ClothCollectionGroup::Lods);
		SkeletalMeshSoftObjectPathName = ManagedArrayCollection->FindAttribute<FSoftObjectPath>(ClothCollectionAttribute::SkeletalMeshSoftObjectPathName, ClothCollectionGroup::Lods);
		ReferenceBoneName = ManagedArrayCollection->FindAttribute<FName>(ClothCollectionAttribute::ReferenceBoneName, ClothCollectionGroup::Lods);
		CopyAllUserAttributesToSimModel = ManagedArrayCollection->FindAttribute<bool>(ClothCollectionAttribute::CopyAllUserAttributesToSimModel, ClothCollectionGroup::Lods);

		// Solvers Group
		SolverGravity = ManagedArrayCollection->FindAttribute<FVector3f>(ClothCollectionAttribute::SolverGravity, ClothCollectionGroup::Solvers);
		SolverAirDamping = ManagedArrayCollection->FindAttribute<float>(ClothCollectionAttribute::SolverAirDamping, ClothCollectionGroup::Solvers);
		SolverSubSteps = ManagedArrayCollection->FindAttribute<int32>(ClothCollectionAttribute::SolverSubSteps, ClothCollectionGroup::Solvers);
		SolverTimeStep = ManagedArrayCollection->FindAttribute<float>(ClothCollectionAttribute::SolverTimeStep, ClothCollectionGroup::Solvers);

		// Seam Group
		SeamStitchStart = ManagedArrayCollection->FindAttribute<int32>(ClothCollectionAttribute::SeamStitchStart, ClothCollectionGroup::Seams);
		SeamStitchEnd = ManagedArrayCollection->FindAttribute<int32>(ClothCollectionAttribute::SeamStitchEnd, ClothCollectionGroup::Seams);

		// Seam Stitches Group
		SeamStitch2DEndIndices = ManagedArrayCollection->FindAttribute<FIntVector2>(ClothCollectionAttribute::SeamStitch2DEndIndices, ClothCollectionGroup::SeamStitches);
		SeamStitch3DIndex = ManagedArrayCollection->FindAttribute<int32>(ClothCollectionAttribute::SeamStitch3DIndex, ClothCollectionGroup::SeamStitches);

		// Sim Patterns Group
		SimVertices2DStart = ManagedArrayCollection->FindAttribute<int32>(ClothCollectionAttribute::SimVertices2DStart, ClothCollectionGroup::SimPatterns);
		SimVertices2DEnd = ManagedArrayCollection->FindAttribute<int32>(ClothCollectionAttribute::SimVertices2DEnd, ClothCollectionGroup::SimPatterns);
		SimFacesStart = ManagedArrayCollection->FindAttribute<int32>(ClothCollectionAttribute::SimFacesStart, ClothCollectionGroup::SimPatterns);
		SimFacesEnd = ManagedArrayCollection->FindAttribute<int32>(ClothCollectionAttribute::SimFacesEnd, ClothCollectionGroup::SimPatterns);
		SimPatternFabric = ManagedArrayCollection->FindAttribute<int32>(ClothCollectionAttribute::SimPatternFabric, ClothCollectionGroup::SimPatterns);

		// Render Patterns Group
		RenderVerticesStart = ManagedArrayCollection->FindAttribute<int32>(ClothCollectionAttribute::RenderVerticesStart, ClothCollectionGroup::RenderPatterns);
		RenderVerticesEnd = ManagedArrayCollection->FindAttribute<int32>(ClothCollectionAttribute::RenderVerticesEnd, ClothCollectionGroup::RenderPatterns);
		RenderFacesStart = ManagedArrayCollection->FindAttribute<int32>(ClothCollectionAttribute::RenderFacesStart, ClothCollectionGroup::RenderPatterns);
		RenderFacesEnd = ManagedArrayCollection->FindAttribute<int32>(ClothCollectionAttribute::RenderFacesEnd, ClothCollectionGroup::RenderPatterns);
		RenderDeformerNumInfluences = ManagedArrayCollection->FindAttribute<int32>(ClothCollectionAttribute::RenderDeformerNumInfluences, ClothCollectionGroup::RenderPatterns);
		RenderMaterialSoftObjectPathName = ManagedArrayCollection->FindAttribute<FSoftObjectPath>(ClothCollectionAttribute::RenderMaterialSoftObjectPathName, ClothCollectionGroup::RenderPatterns);

		//~ Fabric Group
		FabricBendingStiffness = ManagedArrayCollection->FindAttribute<FVector3f>(ClothCollectionAttribute::FabricBendingStiffness, ClothCollectionGroup::Fabrics);
		FabricBucklingStiffness = ManagedArrayCollection->FindAttribute<FVector3f>(ClothCollectionAttribute::FabricBucklingStiffness, ClothCollectionGroup::Fabrics);
		FabricStretchStiffness = ManagedArrayCollection->FindAttribute<FVector3f>(ClothCollectionAttribute::FabricStretchStiffness, ClothCollectionGroup::Fabrics);
		FabricBucklingRatio = ManagedArrayCollection->FindAttribute<float>(ClothCollectionAttribute::FabricBucklingRatio, ClothCollectionGroup::Fabrics);
		FabricDensity = ManagedArrayCollection->FindAttribute<float>(ClothCollectionAttribute::FabricDensity, ClothCollectionGroup::Fabrics);
		FabricFriction = ManagedArrayCollection->FindAttribute<float>(ClothCollectionAttribute::FabricFriction, ClothCollectionGroup::Fabrics);
		FabricDamping = ManagedArrayCollection->FindAttribute<float>(ClothCollectionAttribute::FabricDamping, ClothCollectionGroup::Fabrics);
		FabricPressure = ManagedArrayCollection->FindAttribute<float>(ClothCollectionAttribute::FabricPressure, ClothCollectionGroup::Fabrics);
		FabricLayer = ManagedArrayCollection->FindAttribute<int32>(ClothCollectionAttribute::FabricLayer, ClothCollectionGroup::Fabrics);
		FabricCollisionThickness = ManagedArrayCollection->FindAttribute<float>(ClothCollectionAttribute::FabricCollisionThickness, ClothCollectionGroup::Fabrics);

		// Sim Faces Group
		SimIndices2D = ManagedArrayCollection->FindAttribute<FIntVector3>(ClothCollectionAttribute::SimIndices2D, ClothCollectionGroup::SimFaces);
		SimIndices3D = ManagedArrayCollection->FindAttribute<FIntVector3>(ClothCollectionAttribute::SimIndices3D, ClothCollectionGroup::SimFaces);

		// Sim Vertices 2D Group
		SimPosition2D = ManagedArrayCollection->FindAttribute<FVector2f>(ClothCollectionAttribute::SimPosition2D, ClothCollectionGroup::SimVertices2D);
		SimVertex3DLookup = ManagedArrayCollection->FindAttribute<int32>(ClothCollectionAttribute::SimVertex3DLookup, ClothCollectionGroup::SimVertices2D);
		SimImportVertexID = ManagedArrayCollection->FindAttribute<int32>(ClothCollectionAttribute::SimImportVertexID, ClothCollectionGroup::SimVertices2D);

		// Sim Vertices 3D Group
		SimPosition3D = ManagedArrayCollection->FindAttribute<FVector3f>(ClothCollectionAttribute::SimPosition3D, ClothCollectionGroup::SimVertices3D);
		PreResizedSimPosition3D = ManagedArrayCollection->FindAttribute<FVector3f>(ClothCollectionAttribute::PreResizedSimPosition3D, ClothCollectionGroup::SimVertices3D);
		SimNormal = ManagedArrayCollection->FindAttribute<FVector3f>(ClothCollectionAttribute::SimNormal, ClothCollectionGroup::SimVertices3D);
		SimBoneIndices = ManagedArrayCollection->FindAttribute<TArray<int32>>(ClothCollectionAttribute::SimBoneIndices, ClothCollectionGroup::SimVertices3D);
		SimBoneWeights = ManagedArrayCollection->FindAttribute<TArray<float>>(ClothCollectionAttribute::SimBoneWeights, ClothCollectionGroup::SimVertices3D);
		TetherKinematicIndex = ManagedArrayCollection->FindAttribute<TArray<int32>>(ClothCollectionAttribute::TetherKinematicIndex, ClothCollectionGroup::SimVertices3D);
		TetherReferenceLength = ManagedArrayCollection->FindAttribute<TArray<float>>(ClothCollectionAttribute::TetherReferenceLength, ClothCollectionGroup::SimVertices3D);
		SimVertex2DLookup = ManagedArrayCollection->FindAttribute<TArray<int32>>(ClothCollectionAttribute::SimVertex2DLookup, ClothCollectionGroup::SimVertices3D);
		SeamStitchLookup = ManagedArrayCollection->FindAttribute<TArray<int32>>(ClothCollectionAttribute::SeamStitchLookup, ClothCollectionGroup::SimVertices3D);
		SimCustomResizingBlend = ManagedArrayCollection->FindAttribute<float>(ClothCollectionAttribute::SimCustomResizingBlend, ClothCollectionGroup::SimVertices3D);

		// Sim Morph Targets Group
		SimMorphTargetName = ManagedArrayCollection->FindAttribute<FString>(ClothCollectionAttribute::SimMorphTargetName, ClothCollectionGroup::SimMorphTargets);
		SimMorphTargetVerticesStart = ManagedArrayCollection->FindAttribute<int32>(ClothCollectionAttribute::SimMorphTargetVerticesStart, ClothCollectionGroup::SimMorphTargets);
		SimMorphTargetVerticesEnd = ManagedArrayCollection->FindAttribute<int32>(ClothCollectionAttribute::SimMorphTargetVerticesEnd, ClothCollectionGroup::SimMorphTargets);

		// Sim Morph Targets Vertices Group
		SimMorphTargetPositionDelta = ManagedArrayCollection->FindAttribute<FVector3f>(ClothCollectionAttribute::SimMorphTargetPositionDelta, ClothCollectionGroup::SimMorphTargetVertices);
		SimMorphTargetTangentZDelta = ManagedArrayCollection->FindAttribute<FVector3f>(ClothCollectionAttribute::SimMorphTargetTangentZDelta, ClothCollectionGroup::SimMorphTargetVertices);
		SimMorphTargetSimVertex3DIndex = ManagedArrayCollection->FindAttribute<int32>(ClothCollectionAttribute::SimMorphTargetSimVertex3DIndex, ClothCollectionGroup::SimMorphTargetVertices);

		// Render Faces Group
		RenderIndices = ManagedArrayCollection->FindAttribute<FIntVector3>(ClothCollectionAttribute::RenderIndices, ClothCollectionGroup::RenderFaces);

		// Render Vertices Group
		RenderPosition = ManagedArrayCollection->FindAttribute<FVector3f>(ClothCollectionAttribute::RenderPosition, ClothCollectionGroup::RenderVertices);
		RenderNormal = ManagedArrayCollection->FindAttribute<FVector3f>(ClothCollectionAttribute::RenderNormal, ClothCollectionGroup::RenderVertices);
		RenderTangentU = ManagedArrayCollection->FindAttribute<FVector3f>(ClothCollectionAttribute::RenderTangentU, ClothCollectionGroup::RenderVertices);
		RenderTangentV = ManagedArrayCollection->FindAttribute<FVector3f>(ClothCollectionAttribute::RenderTangentV, ClothCollectionGroup::RenderVertices);
		RenderUVs = ManagedArrayCollection->FindAttribute<TArray<FVector2f>>(ClothCollectionAttribute::RenderUVs, ClothCollectionGroup::RenderVertices);
		RenderColor = ManagedArrayCollection->FindAttribute<FLinearColor>(ClothCollectionAttribute::RenderColor, ClothCollectionGroup::RenderVertices);
		RenderBoneIndices = ManagedArrayCollection->FindAttribute<TArray<int32>>(ClothCollectionAttribute::RenderBoneIndices, ClothCollectionGroup::RenderVertices);
		RenderBoneWeights = ManagedArrayCollection->FindAttribute<TArray<float>>(ClothCollectionAttribute::RenderBoneWeights, ClothCollectionGroup::RenderVertices);
		RenderDeformerPositionBaryCoordsAndDist = ManagedArrayCollection->FindAttribute<TArray<FVector4f>>(ClothCollectionAttribute::RenderDeformerPositionBaryCoordsAndDist, ClothCollectionGroup::RenderVertices);
		RenderDeformerNormalBaryCoordsAndDist = ManagedArrayCollection->FindAttribute<TArray<FVector4f>>(ClothCollectionAttribute::RenderDeformerNormalBaryCoordsAndDist, ClothCollectionGroup::RenderVertices);
		RenderDeformerTangentBaryCoordsAndDist = ManagedArrayCollection->FindAttribute<TArray<FVector4f>>(ClothCollectionAttribute::RenderDeformerTangentBaryCoordsAndDist, ClothCollectionGroup::RenderVertices);
		RenderDeformerSimIndices3D = ManagedArrayCollection->FindAttribute<TArray<FIntVector3>>(ClothCollectionAttribute::RenderDeformerSimIndices3D, ClothCollectionGroup::RenderVertices);
		RenderDeformerWeight = ManagedArrayCollection->FindAttribute<TArray<float>>(ClothCollectionAttribute::RenderDeformerWeight, ClothCollectionGroup::RenderVertices);
		RenderDeformerSkinningBlend = ManagedArrayCollection->FindAttribute<float>(ClothCollectionAttribute::RenderDeformerSkinningBlend, ClothCollectionGroup::RenderVertices);
		RenderCustomResizingBlend = ManagedArrayCollection->FindAttribute<float>(ClothCollectionAttribute::RenderCustomResizingBlend, ClothCollectionGroup::RenderVertices);

		// Group Resizing Group
		CustomResizingRegionSet = ManagedArrayCollection->FindAttribute<FString>(ClothCollectionAttribute::CustomResizingRegionSet, ClothCollectionGroup::CustomResizingRegions);
		CustomResizingRegionType = ManagedArrayCollection->FindAttribute<int32>(ClothCollectionAttribute::CustomResizingRegionType, ClothCollectionGroup::CustomResizingRegions);

		// Sim Accessory Meshes Group
		SimAccessoryMeshName = ManagedArrayCollection->FindAttribute<FName>(ClothCollectionAttribute::SimAccessoryMeshName, ClothCollectionGroup::SimAccessoryMeshes);
		SimAccessoryMeshPosition3DAttribute = ManagedArrayCollection->FindAttribute<FName>(ClothCollectionAttribute::SimAccessoryMeshPosition3DAttribute, ClothCollectionGroup::SimAccessoryMeshes);
		SimAccessoryMeshNormalAttribute = ManagedArrayCollection->FindAttribute<FName>(ClothCollectionAttribute::SimAccessoryMeshNormalAttribute, ClothCollectionGroup::SimAccessoryMeshes);
		SimAccessoryMeshBoneIndicesAttribute = ManagedArrayCollection->FindAttribute<FName>(ClothCollectionAttribute::SimAccessoryMeshBoneIndicesAttribute, ClothCollectionGroup::SimAccessoryMeshes);
		SimAccessoryMeshBoneWeightsAttribute = ManagedArrayCollection->FindAttribute<FName>(ClothCollectionAttribute::SimAccessoryMeshBoneWeightsAttribute, ClothCollectionGroup::SimAccessoryMeshes);
	}

	FConstClothCollection::FConstClothCollection(const TSharedRef<const FManagedArrayCollection>& InManagedArrayCollection)
		: FConstClothCollection(InManagedArrayCollection, TProtectedInit{})
	{
	}

	bool FConstClothCollection::IsValid(EClothCollectionExtendedSchemas OptionalSchemas) const
	{
		if (EnumHasAnyFlags(OptionalSchemas, EClothCollectionExtendedSchemas::CookedOnly))
		{
			// Sim Morph Targets Group
			return
				SimMorphTargetName &&
				SimMorphTargetVerticesStart &&
				SimMorphTargetVerticesEnd &&

				// Sim Morph Target Vertices Group
				SimMorphTargetPositionDelta &&
				SimMorphTargetTangentZDelta &&
				SimMorphTargetSimVertex3DIndex &&

				// Sim Vertices 3D Group
				PreResizedSimPosition3D &&

				// Sim Accessory Meshes Group
				SimAccessoryMeshName &&
				SimAccessoryMeshPosition3DAttribute &&
				SimAccessoryMeshNormalAttribute &&
				SimAccessoryMeshBoneIndicesAttribute &&
				SimAccessoryMeshBoneWeightsAttribute;
		}

		return
			// LODs Group
			PhysicsAssetSoftObjectPathName &&
			SkeletalMeshSoftObjectPathName &&
			ReferenceBoneName &&
			CopyAllUserAttributesToSimModel &&

			// Solvers Group
			(!EnumHasAnyFlags(OptionalSchemas, EClothCollectionExtendedSchemas::Solvers) ||
				(SolverGravity &&
					SolverAirDamping &&
					SolverSubSteps &&
					SolverTimeStep)) &&

			// Seam Group
			SeamStitchStart &&
			SeamStitchEnd &&

			// Seam Stitches Group
			SeamStitch2DEndIndices &&
			SeamStitch3DIndex &&

			// Sim Patterns Group
			SimVertices2DStart &&
			SimVertices2DEnd &&
			SimFacesStart &&
			SimFacesEnd &&
			SimPatternFabric &&

			// Render Patterns Group
			RenderVerticesStart &&
			RenderVerticesEnd &&
			RenderFacesStart &&
			RenderFacesEnd &&
			RenderMaterialSoftObjectPathName &&
			(!EnumHasAnyFlags(OptionalSchemas, EClothCollectionExtendedSchemas::RenderDeformer) ||
				RenderDeformerNumInfluences) &&

			// Sim Faces Group
			SimIndices2D &&
			SimIndices3D &&

			// Fabrics Group
			(!EnumHasAnyFlags(OptionalSchemas, EClothCollectionExtendedSchemas::Fabrics) ||
				(FabricBendingStiffness &&
					FabricBucklingStiffness &&
					FabricStretchStiffness &&
					FabricBucklingRatio &&
					FabricDensity &&
					FabricFriction &&
					FabricDamping &&
					FabricPressure &&
					FabricLayer &&
					FabricCollisionThickness)) &&

			// Sim Vertices 2D Group
			SimPosition2D &&
			SimVertex3DLookup &&
			(!EnumHasAnyFlags(OptionalSchemas, EClothCollectionExtendedSchemas::Import) ||
				(SimImportVertexID)) &&

			// Sim Vertices 3D Group
			SimPosition3D &&
			SimNormal &&
			SimBoneIndices &&
			SimBoneWeights &&
			TetherKinematicIndex &&
			TetherReferenceLength &&
			SimVertex2DLookup &&
			SeamStitchLookup &&
			(!EnumHasAnyFlags(OptionalSchemas, EClothCollectionExtendedSchemas::Resizing) ||
				(SimCustomResizingBlend && PreResizedSimPosition3D)) &&

			// Sim Morph Targets Group
			(!EnumHasAnyFlags(OptionalSchemas, EClothCollectionExtendedSchemas::SimMorphTargets) ||
				(SimMorphTargetName &&
					SimMorphTargetVerticesStart &&
					SimMorphTargetVerticesEnd)) &&

			// Sim Morph Target Vertices Group
			(!EnumHasAnyFlags(OptionalSchemas, EClothCollectionExtendedSchemas::SimMorphTargets) ||
				(SimMorphTargetPositionDelta &&
					SimMorphTargetTangentZDelta &&
					SimMorphTargetSimVertex3DIndex)) &&

			// Render Faces Group
			RenderIndices &&

			// Render Vertices Group
			RenderPosition &&
			RenderNormal &&
			RenderTangentU &&
			RenderTangentV &&
			RenderUVs &&
			RenderColor &&
			RenderBoneIndices &&
			RenderBoneWeights &&
			(!EnumHasAnyFlags(OptionalSchemas, EClothCollectionExtendedSchemas::RenderDeformer) ||
				(RenderDeformerPositionBaryCoordsAndDist &&
				RenderDeformerNormalBaryCoordsAndDist &&
				RenderDeformerTangentBaryCoordsAndDist &&
				RenderDeformerSimIndices3D &&
				RenderDeformerWeight &&
				RenderDeformerSkinningBlend)) &&
			(!EnumHasAnyFlags(OptionalSchemas, EClothCollectionExtendedSchemas::Resizing) ||
				RenderCustomResizingBlend) &&
				
			// Group Resizing Group
			(!EnumHasAnyFlags(OptionalSchemas, EClothCollectionExtendedSchemas::Resizing) ||
				(CustomResizingRegionSet &&
					CustomResizingRegionType)) &&

			// Sim Accessory Meshes Group
			(!EnumHasAnyFlags(OptionalSchemas, EClothCollectionExtendedSchemas::SimAccessoryMeshes) ||
				(SimAccessoryMeshName &&
					SimAccessoryMeshPosition3DAttribute &&
					SimAccessoryMeshNormalAttribute &&
					SimAccessoryMeshBoneIndicesAttribute &&
					SimAccessoryMeshBoneWeightsAttribute));
	}

	FClothCollection::FClothCollection(const TSharedRef<FManagedArrayCollection>& InManagedArrayCollection)
		: FConstClothCollection(InManagedArrayCollection, TProtectedInit{})
	{
	}

	void FClothCollection::DefineSchema(EClothCollectionExtendedSchemas OptionalSchemas)
	{
		using namespace UE::Chaos::ClothAsset::Private;

		// Dependencies
		constexpr bool bSaved = true;
		constexpr bool bAllowCircularDependency = true;
		FManagedArrayCollection::FConstructionParameters SeamStitchesDependency(ClothCollectionGroup::SeamStitches, bSaved, bAllowCircularDependency);
		FManagedArrayCollection::FConstructionParameters RenderFacesDependency(ClothCollectionGroup::RenderFaces, bSaved, bAllowCircularDependency);
		FManagedArrayCollection::FConstructionParameters RenderVerticesDependency(ClothCollectionGroup::RenderVertices, bSaved, bAllowCircularDependency);
		FManagedArrayCollection::FConstructionParameters SimFacesDependency(ClothCollectionGroup::SimFaces, bSaved, bAllowCircularDependency);
		FManagedArrayCollection::FConstructionParameters SimFabricsDependency(ClothCollectionGroup::Fabrics, bSaved, bAllowCircularDependency);
		FManagedArrayCollection::FConstructionParameters SimVertices2DDependency(ClothCollectionGroup::SimVertices2D, bSaved, bAllowCircularDependency);
		FManagedArrayCollection::FConstructionParameters SimVertices3DDependency(ClothCollectionGroup::SimVertices3D, bSaved, bAllowCircularDependency);  // Any attribute with this dependency must handle welding and splitting in FCollectionClothSeamFacade
		FManagedArrayCollection::FConstructionParameters SimMorphTargetVerticesDependency(ClothCollectionGroup::SimMorphTargetVertices, bSaved, bAllowCircularDependency);

		if (EnumHasAnyFlags(OptionalSchemas, EClothCollectionExtendedSchemas::CookedOnly))
		{
			// Special case: only define cooked schema.

			// Sim Morph Targets Group
			SimMorphTargetName = &GetManagedArrayCollection()->AddAttribute<FString>(ClothCollectionAttribute::SimMorphTargetName, ClothCollectionGroup::SimMorphTargets);
			SimMorphTargetVerticesStart = &GetManagedArrayCollection()->AddAttribute<int32>(ClothCollectionAttribute::SimMorphTargetVerticesStart, ClothCollectionGroup::SimMorphTargets);
			SimMorphTargetVerticesEnd = &GetManagedArrayCollection()->AddAttribute<int32>(ClothCollectionAttribute::SimMorphTargetVerticesEnd, ClothCollectionGroup::SimMorphTargets);

			// Sim Morph Target Vertices Group
			SimMorphTargetPositionDelta = &GetManagedArrayCollection()->AddAttribute<FVector3f>(ClothCollectionAttribute::SimMorphTargetPositionDelta, ClothCollectionGroup::SimMorphTargetVertices);
			SimMorphTargetTangentZDelta = &GetManagedArrayCollection()->AddAttribute<FVector3f>(ClothCollectionAttribute::SimMorphTargetTangentZDelta, ClothCollectionGroup::SimMorphTargetVertices);
			SimMorphTargetSimVertex3DIndex = &GetManagedArrayCollection()->AddAttribute<int32>(ClothCollectionAttribute::SimMorphTargetSimVertex3DIndex, ClothCollectionGroup::SimMorphTargetVertices);
			
			// Sim Vertices 3D Group
			PreResizedSimPosition3D = &GetManagedArrayCollection()->AddAttribute<FVector3f>(ClothCollectionAttribute::PreResizedSimPosition3D, ClothCollectionGroup::SimVertices3D);

			// Sim Accessory Meshes Group
			SimAccessoryMeshName = &GetManagedArrayCollection()->AddAttribute<FName>(ClothCollectionAttribute::SimAccessoryMeshName, ClothCollectionGroup::SimAccessoryMeshes);
			SimAccessoryMeshPosition3DAttribute = &GetManagedArrayCollection()->AddAttribute<FName>(ClothCollectionAttribute::SimAccessoryMeshPosition3DAttribute, ClothCollectionGroup::SimAccessoryMeshes);
			SimAccessoryMeshNormalAttribute = &GetManagedArrayCollection()->AddAttribute<FName>(ClothCollectionAttribute::SimAccessoryMeshNormalAttribute, ClothCollectionGroup::SimAccessoryMeshes);
			SimAccessoryMeshBoneIndicesAttribute = &GetManagedArrayCollection()->AddAttribute<FName>(ClothCollectionAttribute::SimAccessoryMeshBoneIndicesAttribute, ClothCollectionGroup::SimAccessoryMeshes);
			SimAccessoryMeshBoneWeightsAttribute = &GetManagedArrayCollection()->AddAttribute<FName>(ClothCollectionAttribute::SimAccessoryMeshBoneWeightsAttribute, ClothCollectionGroup::SimAccessoryMeshes);

			return;
		}

		// LODs Group
		PhysicsAssetSoftObjectPathName = &GetManagedArrayCollection()->AddAttribute<FSoftObjectPath>(ClothCollectionAttribute::PhysicsAssetSoftObjectPathName, ClothCollectionGroup::Lods);
		SkeletalMeshSoftObjectPathName = &GetManagedArrayCollection()->AddAttribute<FSoftObjectPath>(ClothCollectionAttribute::SkeletalMeshSoftObjectPathName, ClothCollectionGroup::Lods);
		ReferenceBoneName = &GetManagedArrayCollection()->AddAttribute<FName>(ClothCollectionAttribute::ReferenceBoneName, ClothCollectionGroup::Lods);
		CopyAllUserAttributesToSimModel = &GetManagedArrayCollection()->AddAttribute<bool>(ClothCollectionAttribute::CopyAllUserAttributesToSimModel, ClothCollectionGroup::Lods);

		// Solvers Group
		if (EnumHasAnyFlags(OptionalSchemas, EClothCollectionExtendedSchemas::Solvers))
		{
			SolverGravity = &GetManagedArrayCollection()->AddAttribute<FVector3f>(ClothCollectionAttribute::SolverGravity, ClothCollectionGroup::Solvers);
			SolverAirDamping = &GetManagedArrayCollection()->AddAttribute<float>(ClothCollectionAttribute::SolverAirDamping, ClothCollectionGroup::Solvers);
			SolverSubSteps = &GetManagedArrayCollection()->AddAttribute<int32>(ClothCollectionAttribute::SolverSubSteps, ClothCollectionGroup::Solvers);
			SolverTimeStep = &GetManagedArrayCollection()->AddAttribute<float>(ClothCollectionAttribute::SolverTimeStep, ClothCollectionGroup::Solvers);
		}

		// Seams Group
		SeamStitchStart = &GetManagedArrayCollection()->AddAttribute<int32>(ClothCollectionAttribute::SeamStitchStart, ClothCollectionGroup::Seams, SeamStitchesDependency);
		SeamStitchEnd = &GetManagedArrayCollection()->AddAttribute<int32>(ClothCollectionAttribute::SeamStitchEnd, ClothCollectionGroup::Seams, SeamStitchesDependency);

		// Seam Stitches Group
		SeamStitch2DEndIndices = &GetManagedArrayCollection()->AddAttribute<FIntVector2>(ClothCollectionAttribute::SeamStitch2DEndIndices, ClothCollectionGroup::SeamStitches, SimVertices2DDependency);
		SeamStitch3DIndex = &GetManagedArrayCollection()->AddAttribute<int32>(ClothCollectionAttribute::SeamStitch3DIndex, ClothCollectionGroup::SeamStitches, SimVertices3DDependency);

		// Sim Patterns Group
		SimVertices2DStart = &GetManagedArrayCollection()->AddAttribute<int32>(ClothCollectionAttribute::SimVertices2DStart, ClothCollectionGroup::SimPatterns, SimVertices2DDependency);
		SimVertices2DEnd = &GetManagedArrayCollection()->AddAttribute<int32>(ClothCollectionAttribute::SimVertices2DEnd, ClothCollectionGroup::SimPatterns, SimVertices2DDependency);
		SimFacesStart = &GetManagedArrayCollection()->AddAttribute<int32>(ClothCollectionAttribute::SimFacesStart, ClothCollectionGroup::SimPatterns, SimFacesDependency);
		SimFacesEnd = &GetManagedArrayCollection()->AddAttribute<int32>(ClothCollectionAttribute::SimFacesEnd, ClothCollectionGroup::SimPatterns, SimFacesDependency);
		SimPatternFabric = &GetManagedArrayCollection()->AddAttribute<int32>(ClothCollectionAttribute::SimPatternFabric, ClothCollectionGroup::SimPatterns, SimFabricsDependency);

		// Render Patterns Group
		RenderVerticesStart = &GetManagedArrayCollection()->AddAttribute<int32>(ClothCollectionAttribute::RenderVerticesStart, ClothCollectionGroup::RenderPatterns, RenderVerticesDependency);
		RenderVerticesEnd = &GetManagedArrayCollection()->AddAttribute<int32>(ClothCollectionAttribute::RenderVerticesEnd, ClothCollectionGroup::RenderPatterns, RenderVerticesDependency);
		RenderFacesStart = &GetManagedArrayCollection()->AddAttribute<int32>(ClothCollectionAttribute::RenderFacesStart, ClothCollectionGroup::RenderPatterns, RenderFacesDependency);
		RenderFacesEnd = &GetManagedArrayCollection()->AddAttribute<int32>(ClothCollectionAttribute::RenderFacesEnd, ClothCollectionGroup::RenderPatterns, RenderFacesDependency);
		RenderMaterialSoftObjectPathName = &GetManagedArrayCollection()->AddAttribute<FSoftObjectPath>(ClothCollectionAttribute::RenderMaterialSoftObjectPathName, ClothCollectionGroup::RenderPatterns);

		//~ Fabric Group
		if (EnumHasAnyFlags(OptionalSchemas, EClothCollectionExtendedSchemas::Fabrics))
		{
			FabricBendingStiffness = &GetManagedArrayCollection()->AddAttribute<FVector3f>(ClothCollectionAttribute::FabricBendingStiffness, ClothCollectionGroup::Fabrics);
			FabricBucklingStiffness = &GetManagedArrayCollection()->AddAttribute<FVector3f>(ClothCollectionAttribute::FabricBucklingStiffness, ClothCollectionGroup::Fabrics);
			FabricStretchStiffness = &GetManagedArrayCollection()->AddAttribute<FVector3f>(ClothCollectionAttribute::FabricStretchStiffness, ClothCollectionGroup::Fabrics);
			FabricBucklingRatio = &GetManagedArrayCollection()->AddAttribute<float>(ClothCollectionAttribute::FabricBucklingRatio, ClothCollectionGroup::Fabrics);
			FabricDensity = &GetManagedArrayCollection()->AddAttribute<float>(ClothCollectionAttribute::FabricDensity, ClothCollectionGroup::Fabrics);
			FabricFriction = &GetManagedArrayCollection()->AddAttribute<float>(ClothCollectionAttribute::FabricFriction, ClothCollectionGroup::Fabrics);
			FabricDamping = &GetManagedArrayCollection()->AddAttribute<float>(ClothCollectionAttribute::FabricDamping, ClothCollectionGroup::Fabrics);
			FabricPressure = &GetManagedArrayCollection()->AddAttribute<float>(ClothCollectionAttribute::FabricPressure, ClothCollectionGroup::Fabrics);
			FabricLayer = &GetManagedArrayCollection()->AddAttribute<int32>(ClothCollectionAttribute::FabricLayer, ClothCollectionGroup::Fabrics);
			FabricCollisionThickness = &GetManagedArrayCollection()->AddAttribute<float>(ClothCollectionAttribute::FabricCollisionThickness, ClothCollectionGroup::Fabrics);
		}

		// Sim Faces Group
		SimIndices2D = &GetManagedArrayCollection()->AddAttribute<FIntVector3>(ClothCollectionAttribute::SimIndices2D, ClothCollectionGroup::SimFaces, SimVertices2DDependency);
		SimIndices3D = &GetManagedArrayCollection()->AddAttribute<FIntVector3>(ClothCollectionAttribute::SimIndices3D, ClothCollectionGroup::SimFaces, SimVertices3DDependency);

		// Sim Vertices 2D Group
		SimPosition2D = &GetManagedArrayCollection()->AddAttribute<FVector2f>(ClothCollectionAttribute::SimPosition2D, ClothCollectionGroup::SimVertices2D);
		SimVertex3DLookup = &GetManagedArrayCollection()->AddAttribute<int32>(ClothCollectionAttribute::SimVertex3DLookup, ClothCollectionGroup::SimVertices2D, SimVertices3DDependency);
		if (EnumHasAnyFlags(OptionalSchemas, EClothCollectionExtendedSchemas::Import))
		{
			SimImportVertexID = &GetManagedArrayCollection()->AddAttribute<int32>(ClothCollectionAttribute::SimImportVertexID, ClothCollectionGroup::SimVertices2D);
		}

		// Sim Vertices 3D Group
		SimPosition3D = &GetManagedArrayCollection()->AddAttribute<FVector3f>(ClothCollectionAttribute::SimPosition3D, ClothCollectionGroup::SimVertices3D);
		SimNormal = &GetManagedArrayCollection()->AddAttribute<FVector3f>(ClothCollectionAttribute::SimNormal, ClothCollectionGroup::SimVertices3D);
		SimBoneIndices = &GetManagedArrayCollection()->AddAttribute<TArray<int32>>(ClothCollectionAttribute::SimBoneIndices, ClothCollectionGroup::SimVertices3D);
		SimBoneWeights = &GetManagedArrayCollection()->AddAttribute<TArray<float>>(ClothCollectionAttribute::SimBoneWeights, ClothCollectionGroup::SimVertices3D);
		TetherKinematicIndex = &GetManagedArrayCollection()->AddAttribute<TArray<int32>>(ClothCollectionAttribute::TetherKinematicIndex, ClothCollectionGroup::SimVertices3D, SimVertices3DDependency);
		TetherReferenceLength = &GetManagedArrayCollection()->AddAttribute<TArray<float>>(ClothCollectionAttribute::TetherReferenceLength, ClothCollectionGroup::SimVertices3D);
		SimVertex2DLookup = &GetManagedArrayCollection()->AddAttribute<TArray<int32>>(ClothCollectionAttribute::SimVertex2DLookup, ClothCollectionGroup::SimVertices3D, SimVertices2DDependency);
		SeamStitchLookup = &GetManagedArrayCollection()->AddAttribute<TArray<int32>>(ClothCollectionAttribute::SeamStitchLookup, ClothCollectionGroup::SimVertices3D, SeamStitchesDependency);

		if (EnumHasAnyFlags(OptionalSchemas, EClothCollectionExtendedSchemas::SimMorphTargets))
		{
			// Sim Morph Targets Group
			SimMorphTargetName = &GetManagedArrayCollection()->AddAttribute<FString>(ClothCollectionAttribute::SimMorphTargetName, ClothCollectionGroup::SimMorphTargets);
			SimMorphTargetVerticesStart = &GetManagedArrayCollection()->AddAttribute<int32>(ClothCollectionAttribute::SimMorphTargetVerticesStart, ClothCollectionGroup::SimMorphTargets, SimMorphTargetVerticesDependency);
			SimMorphTargetVerticesEnd = &GetManagedArrayCollection()->AddAttribute<int32>(ClothCollectionAttribute::SimMorphTargetVerticesEnd, ClothCollectionGroup::SimMorphTargets, SimMorphTargetVerticesDependency);

			// Sim Morph Target Vertices Group
			SimMorphTargetPositionDelta = &GetManagedArrayCollection()->AddAttribute<FVector3f>(ClothCollectionAttribute::SimMorphTargetPositionDelta, ClothCollectionGroup::SimMorphTargetVertices);
			SimMorphTargetTangentZDelta = &GetManagedArrayCollection()->AddAttribute<FVector3f>(ClothCollectionAttribute::SimMorphTargetTangentZDelta, ClothCollectionGroup::SimMorphTargetVertices);
			SimMorphTargetSimVertex3DIndex = &GetManagedArrayCollection()->AddAttribute<int32>(ClothCollectionAttribute::SimMorphTargetSimVertex3DIndex, ClothCollectionGroup::SimMorphTargetVertices, SimVertices3DDependency);
		}

		// Render Faces Group
		RenderIndices = &GetManagedArrayCollection()->AddAttribute<FIntVector3>(ClothCollectionAttribute::RenderIndices, ClothCollectionGroup::RenderFaces, RenderVerticesDependency);

		// Render Vertices Group
		RenderPosition = &GetManagedArrayCollection()->AddAttribute<FVector3f>(ClothCollectionAttribute::RenderPosition, ClothCollectionGroup::RenderVertices);
		RenderNormal = &GetManagedArrayCollection()->AddAttribute<FVector3f>(ClothCollectionAttribute::RenderNormal, ClothCollectionGroup::RenderVertices);
		RenderTangentU = &GetManagedArrayCollection()->AddAttribute<FVector3f>(ClothCollectionAttribute::RenderTangentU, ClothCollectionGroup::RenderVertices);
		RenderTangentV = &GetManagedArrayCollection()->AddAttribute<FVector3f>(ClothCollectionAttribute::RenderTangentV, ClothCollectionGroup::RenderVertices);
		RenderUVs = &GetManagedArrayCollection()->AddAttribute<TArray<FVector2f>>(ClothCollectionAttribute::RenderUVs, ClothCollectionGroup::RenderVertices);
		RenderColor = &GetManagedArrayCollection()->AddAttribute<FLinearColor>(ClothCollectionAttribute::RenderColor, ClothCollectionGroup::RenderVertices);
		RenderBoneIndices = &GetManagedArrayCollection()->AddAttribute<TArray<int32>>(ClothCollectionAttribute::RenderBoneIndices, ClothCollectionGroup::RenderVertices);
		RenderBoneWeights = &GetManagedArrayCollection()->AddAttribute<TArray<float>>(ClothCollectionAttribute::RenderBoneWeights, ClothCollectionGroup::RenderVertices);

		if (EnumHasAnyFlags(OptionalSchemas, EClothCollectionExtendedSchemas::RenderDeformer))
		{
			// Render Patterns Group
			RenderDeformerNumInfluences = &GetManagedArrayCollection()->AddAttribute<int32>(ClothCollectionAttribute::RenderDeformerNumInfluences, ClothCollectionGroup::RenderPatterns);

			// Render Vertices Group
			RenderDeformerPositionBaryCoordsAndDist = &GetManagedArrayCollection()->AddAttribute<TArray<FVector4f>>(ClothCollectionAttribute::RenderDeformerPositionBaryCoordsAndDist, ClothCollectionGroup::RenderVertices);
			RenderDeformerNormalBaryCoordsAndDist = &GetManagedArrayCollection()->AddAttribute<TArray<FVector4f>>(ClothCollectionAttribute::RenderDeformerNormalBaryCoordsAndDist, ClothCollectionGroup::RenderVertices);
			RenderDeformerTangentBaryCoordsAndDist = &GetManagedArrayCollection()->AddAttribute<TArray<FVector4f>>(ClothCollectionAttribute::RenderDeformerTangentBaryCoordsAndDist, ClothCollectionGroup::RenderVertices);
			RenderDeformerSimIndices3D = &GetManagedArrayCollection()->AddAttribute<TArray<FIntVector3>>(ClothCollectionAttribute::RenderDeformerSimIndices3D, ClothCollectionGroup::RenderVertices, SimVertices3DDependency);
			RenderDeformerWeight = &GetManagedArrayCollection()->AddAttribute<TArray<float>>(ClothCollectionAttribute::RenderDeformerWeight, ClothCollectionGroup::RenderVertices);
			RenderDeformerSkinningBlend = &GetManagedArrayCollection()->AddAttribute<float>(ClothCollectionAttribute::RenderDeformerSkinningBlend, ClothCollectionGroup::RenderVertices);
		}
		
		if (EnumHasAnyFlags(OptionalSchemas, EClothCollectionExtendedSchemas::Resizing))
		{
			// Sim Vertices 3D Group
			SimCustomResizingBlend = &GetManagedArrayCollection()->AddAttribute<float>(ClothCollectionAttribute::SimCustomResizingBlend, ClothCollectionGroup::SimVertices3D);
			PreResizedSimPosition3D = &GetManagedArrayCollection()->AddAttribute<FVector3f>(ClothCollectionAttribute::PreResizedSimPosition3D, ClothCollectionGroup::SimVertices3D);

			// Render Vertices Group
			RenderCustomResizingBlend = &GetManagedArrayCollection()->AddAttribute<float>(ClothCollectionAttribute::RenderCustomResizingBlend, ClothCollectionGroup::RenderVertices);

			// Group Resizing Group
			CustomResizingRegionSet = &GetManagedArrayCollection()->AddAttribute<FString>(ClothCollectionAttribute::CustomResizingRegionSet, ClothCollectionGroup::CustomResizingRegions);
			CustomResizingRegionType = &GetManagedArrayCollection()->AddAttribute<int32>(ClothCollectionAttribute::CustomResizingRegionType, ClothCollectionGroup::CustomResizingRegions);
		}

		if (EnumHasAnyFlags(OptionalSchemas, EClothCollectionExtendedSchemas::SimAccessoryMeshes))
		{
			// Sim Accessory Meshes Group
			SimAccessoryMeshName = &GetManagedArrayCollection()->AddAttribute<FName>(ClothCollectionAttribute::SimAccessoryMeshName, ClothCollectionGroup::SimAccessoryMeshes);
			SimAccessoryMeshPosition3DAttribute = &GetManagedArrayCollection()->AddAttribute<FName>(ClothCollectionAttribute::SimAccessoryMeshPosition3DAttribute, ClothCollectionGroup::SimAccessoryMeshes);
			SimAccessoryMeshNormalAttribute = &GetManagedArrayCollection()->AddAttribute<FName>(ClothCollectionAttribute::SimAccessoryMeshNormalAttribute, ClothCollectionGroup::SimAccessoryMeshes);
			SimAccessoryMeshBoneIndicesAttribute = &GetManagedArrayCollection()->AddAttribute<FName>(ClothCollectionAttribute::SimAccessoryMeshBoneIndicesAttribute, ClothCollectionGroup::SimAccessoryMeshes);
			SimAccessoryMeshBoneWeightsAttribute = &GetManagedArrayCollection()->AddAttribute<FName>(ClothCollectionAttribute::SimAccessoryMeshBoneWeightsAttribute, ClothCollectionGroup::SimAccessoryMeshes);
		}
	}

	void FClothCollection::PostSerialize(const FArchive& Ar)
	{
		if (Ar.IsLoading())
		{
			auto ConvertStringToSoftObjectPath = [](const TManagedArray<FString>& StringArray, TManagedArray<FSoftObjectPath>& SoftObjectPathArray)
				{
					check(StringArray.Num() == SoftObjectPathArray.Num());
					for (int32 Index = 0; Index < StringArray.Num(); ++Index)
					{
						SoftObjectPathArray[Index] = FSoftObjectPath(StringArray[Index]);
						SoftObjectPathArray[Index].PreSavePath(); // This will fixup the path with redirects.
					}
				};

			// PhysicsAssetPathName -> PhysicsAssetSoftObjectPathName
			if (const TManagedArray<FString>* const InPhysicsAssetPathArray = ManagedArrayCollection->FindAttributeTyped<FString>(ClothCollectionAttribute::PhysicsAssetPathName, ClothCollectionGroup::Lods))
			{
				if (!PhysicsAssetSoftObjectPathName)
				{
					PhysicsAssetSoftObjectPathName = &GetManagedArrayCollection()->AddAttribute<FSoftObjectPath>(ClothCollectionAttribute::PhysicsAssetSoftObjectPathName, ClothCollectionGroup::Lods);
				}
				ConvertStringToSoftObjectPath(*InPhysicsAssetPathArray, *GetPhysicsAssetSoftObjectPathName());
				GetManagedArrayCollection()->RemoveAttribute(ClothCollectionAttribute::PhysicsAssetPathName, ClothCollectionGroup::Lods);
			}

			// SkeletalMeshPathName -> SkeletalMeshSoftObjectPathName
			if (const TManagedArray<FString>* const InSkeletalMeshPathName = ManagedArrayCollection->FindAttributeTyped<FString>(ClothCollectionAttribute::SkeletalMeshPathName, ClothCollectionGroup::Lods))
			{
				if (!SkeletalMeshSoftObjectPathName)
				{
					SkeletalMeshSoftObjectPathName = &GetManagedArrayCollection()->AddAttribute<FSoftObjectPath>(ClothCollectionAttribute::SkeletalMeshSoftObjectPathName, ClothCollectionGroup::Lods);
				}
				ConvertStringToSoftObjectPath(*InSkeletalMeshPathName, *GetSkeletalMeshSoftObjectPathName());
				GetManagedArrayCollection()->RemoveAttribute(ClothCollectionAttribute::SkeletalMeshPathName, ClothCollectionGroup::Lods);
			}

			// RenderMaterialPathName -> RenderMaterialSoftObjectPathName
			if (const TManagedArray<FString>* const InRenderMaterialPathName = ManagedArrayCollection->FindAttributeTyped<FString>(ClothCollectionAttribute::RenderMaterialPathName, ClothCollectionGroup::RenderPatterns))
			{
				if (!RenderMaterialSoftObjectPathName)
				{
					RenderMaterialSoftObjectPathName = &GetManagedArrayCollection()->AddAttribute<FSoftObjectPath>(ClothCollectionAttribute::RenderMaterialSoftObjectPathName, ClothCollectionGroup::RenderPatterns);
				}
				ConvertStringToSoftObjectPath(*InRenderMaterialPathName, *GetRenderMaterialSoftObjectPathName());
				GetManagedArrayCollection()->RemoveAttribute(ClothCollectionAttribute::RenderMaterialPathName, ClothCollectionGroup::RenderPatterns);
			}

			// Add ReferenceBoneName attribute if other LODs group attributes are defined.
			if (PhysicsAssetSoftObjectPathName && SkeletalMeshSoftObjectPathName && !ReferenceBoneName)
			{
				ReferenceBoneName = &GetManagedArrayCollection()->AddAttribute<FName>(ClothCollectionAttribute::ReferenceBoneName, ClothCollectionGroup::Lods);
			}

			// Add CopyAllUserAttributesToSimModel attribute if other LODs group attributes are defined.
			if (PhysicsAssetSoftObjectPathName && SkeletalMeshSoftObjectPathName && ReferenceBoneName && !CopyAllUserAttributesToSimModel)
			{
				CopyAllUserAttributesToSimModel = &GetManagedArrayCollection()->AddAttribute<bool>(ClothCollectionAttribute::CopyAllUserAttributesToSimModel, ClothCollectionGroup::Lods);
			}
		}
	}

	int32 FConstClothCollection::GetNumElements(const FName& GroupName) const
	{
		return ManagedArrayCollection->NumElements(GroupName);
	}

	int32 FConstClothCollection::GetNumElements(const TManagedArray<int32>* StartArray, const TManagedArray<int32>* EndArray, int32 ArrayIndex) const
	{
		if (StartArray && StartArray->GetConstArray().IsValidIndex(ArrayIndex) &&
			EndArray && EndArray->GetConstArray().IsValidIndex(ArrayIndex))
		{
			const int32 Start = (*StartArray)[ArrayIndex];
			const int32 End = (*EndArray)[ArrayIndex];
			if (Start != INDEX_NONE && End != INDEX_NONE)
			{
				return End - Start + 1;
			}
			checkf(Start == End, TEXT("Only one boundary of the range is set to INDEX_NONE, when both should."));
		}
		return 0;
	}

	void FClothCollection::SetNumElements(int32 InNumElements, const FName& GroupName)
	{
		check(InNumElements >= 0);
		
		const int32 NumElements = GetManagedArrayCollection()->NumElements(GroupName);

		if (const int32 Delta = InNumElements - NumElements)
		{
			if (Delta > 0)
			{
				GetManagedArrayCollection()->AddElements(Delta, GroupName);
			}
			else
			{
				GetManagedArrayCollection()->RemoveElements(GroupName, -Delta, InNumElements);
			}
		}
	}

	int32 FClothCollection::SetNumElements(int32 InNumElements, const FName& GroupName, TManagedArray<int32>* StartArray, TManagedArray<int32>* EndArray, int32 ArrayIndex)
	{
		check(InNumElements >= 0);

		check(StartArray && StartArray->GetConstArray().IsValidIndex(ArrayIndex));
		check(EndArray && EndArray->GetConstArray().IsValidIndex(ArrayIndex));

		int32& Start = (*StartArray)[ArrayIndex];
		int32& End = (*EndArray)[ArrayIndex];
		check(Start != INDEX_NONE || End == INDEX_NONE);  // Best to avoid situations where only one boundary of the range is set to INDEX_NONE

		const int32 NumElements = (Start == INDEX_NONE) ? 0 : End - Start + 1;

		if (const int32 Delta = InNumElements - NumElements)
		{
			if (Delta > 0)
			{
				// Find a previous valid index range to insert after when the range is empty
				auto ComputeEnd = [&EndArray, ArrayIndex]()->int32
				{
					for (int32 Index = ArrayIndex; Index >= 0; --Index)
					{
						if ((*EndArray)[Index] != INDEX_NONE)
						{
							return (*EndArray)[Index];
						}
					}
					return INDEX_NONE;
				};

				// Grow the array
				const int32 Position = ComputeEnd() + 1;
				GetManagedArrayCollection()->InsertElements(Delta, Position, GroupName);

				// Update Start/End
				if (!NumElements)
				{
					Start = Position;
				}
				End = Start + InNumElements - 1;
			}
			else
			{
				// Shrink the array
				const int32 Position = Start + InNumElements;
				GetManagedArrayCollection()->RemoveElements(GroupName, -Delta, Position);

				// Update Start/End
				if (InNumElements)
				{
					End = Position - 1;
				}
				else
				{
					End = Start = INDEX_NONE;  // It is important to set the start & end to INDEX_NONE so that they never get automatically re-indexed by the managed array collection
				}
			}
		}
		return Start;
	}

	void FClothCollection::RemoveElements(const FName& Group, const TArray<int32>& SortedDeletionList)
	{
		GetManagedArrayCollection()->RemoveElements(Group, SortedDeletionList);
	}

	void FClothCollection::RemoveElements(const FName& GroupName, const TArray<int32>& SortedDeletionList, TManagedArray<int32>* StartArray, TManagedArray<int32>* EndArray, int32 ArrayIndex)
	{
		if (SortedDeletionList.IsEmpty())
		{
			return;
		}

		check(StartArray && StartArray->GetConstArray().IsValidIndex(ArrayIndex));
		check(EndArray && EndArray->GetConstArray().IsValidIndex(ArrayIndex));

		int32& Start = (*StartArray)[ArrayIndex];
		int32& End = (*EndArray)[ArrayIndex];
		check(Start != INDEX_NONE && End != INDEX_NONE);

		const int32 OrigStart = Start;
		const int32 OrigNumElements = End - Start + 1;

		check(SortedDeletionList[0] >= Start);
		check(SortedDeletionList.Last() <= End);
		check(OrigNumElements >= SortedDeletionList.Num());

		GetManagedArrayCollection()->RemoveElements(GroupName, SortedDeletionList);

		if (SortedDeletionList.Num() == OrigNumElements)
		{
			Start = End = INDEX_NONE;
		}
		else
		{
			const int32 NewNumElements = OrigNumElements - SortedDeletionList.Num();
			const int32 NewEnd = OrigStart + NewNumElements - 1;
			check(Start == OrigStart || Start == INDEX_NONE);
			check(End == NewEnd || End == INDEX_NONE);
			Start = OrigStart;
			End = NewEnd;
		}
	}

	/*static*/ int32 FConstClothCollection::GetElementsOffset(const TManagedArray<int32>* StartArray, int32 BaseElementIndex, int32 ElementIndex)
	{
		while (BaseElementIndex < ElementIndex && (*StartArray)[BaseElementIndex] == INDEX_NONE)
		{
			++BaseElementIndex;
		}
		return (*StartArray)[ElementIndex] - (*StartArray)[BaseElementIndex];
	}

	/*static*/ int32 FConstClothCollection::GetArrayIndexForContainedElement(
		const TManagedArray<int32>* StartArray,
		const TManagedArray<int32>* EndArray,
		int32 ElementIndex)
	{
		for (int32 ArrayIndex = 0; ArrayIndex < StartArray->Num(); ++ArrayIndex)
		{
			if (ElementIndex >= (*StartArray)[ArrayIndex] && ElementIndex <= (*EndArray)[ArrayIndex])
			{
				return ArrayIndex;
			}
		}
		return INDEX_NONE;
	}

	/*static*/ int32 FConstClothCollection::GetNumSubElements(
		const TManagedArray<int32>* StartArray,
		const TManagedArray<int32>* EndArray,
		const TManagedArray<int32>* StartSubArray,
		const TManagedArray<int32>* EndSubArray,
		int32 ArrayIndex)
	{
		const TTuple<int32, int32> StartEnd = GetSubElementsStartEnd(StartArray, EndArray, StartSubArray, EndSubArray, ArrayIndex);
		const int32 Start = StartEnd.Get<0>();
		const int32 End = StartEnd.Get<1>();
		if (Start != INDEX_NONE && End != INDEX_NONE)
		{
			return End - Start + 1;
		}
		checkf(Start == End, TEXT("Only one boundary of the range is set to INDEX_NONE, when both should."));
		return 0;
	}

	template<bool bStart, bool bEnd>
	/*static*/ TTuple<int32, int32> FConstClothCollection::GetSubElementsStartEnd(
		const TManagedArray<int32>* StartArray,
		const TManagedArray<int32>* EndArray,
		const TManagedArray<int32>* StartSubArray,
		const TManagedArray<int32>* EndSubArray,
		int32 ArrayIndex)
	{
		int32 Start = INDEX_NONE;  // Find Start and End indices for the entire LOD minding empty patterns on the way
		int32 End = INDEX_NONE;

		if (StartArray && StartArray->GetConstArray().IsValidIndex(ArrayIndex) &&
			EndArray && EndArray->GetConstArray().IsValidIndex(ArrayIndex))
		{
			const int32 SubStart = (*StartArray)[ArrayIndex];
			const int32 SubEnd = (*EndArray)[ArrayIndex];

			if (SubStart != INDEX_NONE && SubEnd != INDEX_NONE)
			{
				for (int32 SubIndex = SubStart; SubIndex <= SubEnd; ++SubIndex)
				{
					if (bStart && (*StartSubArray)[SubIndex] != INDEX_NONE)
					{
						Start = (Start == INDEX_NONE) ? (*StartSubArray)[SubIndex] : FMath::Min(Start, (*StartSubArray)[SubIndex]);
					}
					if (bEnd && (*EndSubArray)[SubIndex] != INDEX_NONE)
					{
						End = (End == INDEX_NONE) ? (*EndSubArray)[SubIndex] : FMath::Max(End, (*EndSubArray)[SubIndex]);
					}
				}
			}
			else
			{
				checkf(SubStart == SubEnd, TEXT("Only one boundary of the range is set to INDEX_NONE, when both should."));
			}
		}
		return TTuple<int32, int32>(Start, End);
	}
	template CHAOSCLOTHASSET_API TTuple<int32, int32> FConstClothCollection::GetSubElementsStartEnd<true, false>(const TManagedArray<int32>* StartArray, const TManagedArray<int32>* EndArray, const TManagedArray<int32>* StartSubArray, const TManagedArray<int32>* EndSubArray, int32 ArrayIndex);
	template CHAOSCLOTHASSET_API TTuple<int32, int32> FConstClothCollection::GetSubElementsStartEnd<false, true>(const TManagedArray<int32>* StartArray, const TManagedArray<int32>* EndArray, const TManagedArray<int32>* StartSubArray, const TManagedArray<int32>* EndSubArray, int32 ArrayIndex);
	template CHAOSCLOTHASSET_API TTuple<int32, int32> FConstClothCollection::GetSubElementsStartEnd<true, true>(const TManagedArray<int32>* StartArray, const TManagedArray<int32>* EndArray, const TManagedArray<int32>* StartSubArray, const TManagedArray<int32>* EndSubArray, int32 ArrayIndex);

	template<typename T UE_REQUIRES_DEFINITION(TIsUserAttributeType<T>::Value)>
	TArray<FName> FConstClothCollection::GetUserDefinedAttributeNames(const FName& GroupName) const
	{
		using namespace UE::Chaos::ClothAsset::Private;

		TArray<FName> UserDefinedAttributeNames;

		if (const TArray<FName>* const FixedAttributeNames = GetFixedAttributeNamesMap().Find(GroupName))  // Also checks that the group name is a recognized group name
		{
			const TArray<FName>* const FixedAttributePrefixNames = GetFixedAttributePrefixNamesMap().Find(GroupName);

			const TArray<FName> AttributeNames = ManagedArrayCollection->AttributeNames(GroupName);

			for (const FName& AttributeName : AttributeNames)
			{
				if (!FixedAttributeNames->Contains(AttributeName) && ManagedArrayCollection->FindAttributeTyped<T>(AttributeName, GroupName)
					&& (!FixedAttributePrefixNames || !FixedAttributePrefixNames->ContainsByPredicate([&AttributeName](const FName& Prefix)
						{
							return Private::NamesHaveMatchingPrefix(AttributeName, Prefix);
						})))
				{
						UserDefinedAttributeNames.Add(AttributeName);
				}
			}
		}

		return UserDefinedAttributeNames;
	}
	template CHAOSCLOTHASSET_API TArray<FName> FConstClothCollection::GetUserDefinedAttributeNames<bool>(const FName& GroupName) const;
	template CHAOSCLOTHASSET_API TArray<FName> FConstClothCollection::GetUserDefinedAttributeNames<int32>(const FName& GroupName) const;
	template CHAOSCLOTHASSET_API TArray<FName> FConstClothCollection::GetUserDefinedAttributeNames<float>(const FName& GroupName) const;
	template CHAOSCLOTHASSET_API TArray<FName> FConstClothCollection::GetUserDefinedAttributeNames<FVector3f>(const FName& GroupName) const;
	template CHAOSCLOTHASSET_API TArray<FName> FConstClothCollection::GetUserDefinedAttributeNames<TArray<int32>>(const FName& GroupName) const;

	template<typename T UE_REQUIRES_DEFINITION(TIsUserAttributeType<T>::Value)>
	TManagedArray<T>* FClothCollection::FindOrAddUserDefinedAttribute(const FName& Name, const FName& GroupName, const FName& GroupDependency)
	{
		using namespace UE::Chaos::ClothAsset::Private;
		if (const TArray<FName>* const FixedAttributePrefixes = GetFixedAttributePrefixNamesMap().Find(GroupName))
		{
			// Don't allow adding a name reserved as a fixed attribute prefix.
			if (FixedAttributePrefixes->ContainsByPredicate([&Name](const FName& Prefix)
				{
					return NamesHaveMatchingPrefix(Name, Prefix);
				}))
			{
				return nullptr;
			}
		}
		if (const TArray<FName>* const FixedAttributeNames = GetFixedAttributeNamesMap().Find(GroupName))
		{
			// Only allow adding to known groups. Do not allow adding a name reserved as a fixed attribute.
			if (!FixedAttributeNames->Contains(Name))
			{
				return GetManagedArrayCollection()->FindOrAddAttributeTyped<T>(Name, GroupName, FManagedArrayCollection::FConstructionParameters(GroupDependency));
			}
		}
		if (const TArray<FName>* const UserAccessibleAttributeNames = GetUserAccessibleAttributeNamesMap().Find(GroupName))
		{
			// Paintable attribute can only be found, not created (they are part of the schema)
			if (UserAccessibleAttributeNames->Contains(Name))
			{
				return GetManagedArrayCollection()->FindAttributeTyped<T>(Name, GroupName);
			}
		}
		return nullptr;
	}
	template CHAOSCLOTHASSET_API TManagedArray<bool>* FClothCollection::FindOrAddUserDefinedAttribute<bool>(const FName& Name, const FName& GroupName, const FName& GroupDependency);
	template CHAOSCLOTHASSET_API TManagedArray<int32>* FClothCollection::FindOrAddUserDefinedAttribute<int32>(const FName& Name, const FName& GroupName, const FName& GroupDependency);
	template CHAOSCLOTHASSET_API TManagedArray<float>* FClothCollection::FindOrAddUserDefinedAttribute<float>(const FName& Name, const FName& GroupName, const FName& GroupDependency);
	template CHAOSCLOTHASSET_API TManagedArray<FVector3f>* FClothCollection::FindOrAddUserDefinedAttribute<FVector3f>(const FName& Name, const FName& GroupName, const FName& GroupDependency);
	template CHAOSCLOTHASSET_API TManagedArray<TArray<int32>>* FClothCollection::FindOrAddUserDefinedAttribute<TArray<int32>>(const FName& Name, const FName& GroupName, const FName& GroupDependency);

	void FClothCollection::RemoveUserDefinedAttribute(const FName& Name, const FName& GroupName)
	{
		using namespace UE::Chaos::ClothAsset::Private;
		if (const TArray<FName>* const FixedAttributeNames = GetFixedAttributeNamesMap().Find(GroupName))
		{
			if (const TArray<FName>* const FixedAttributePrefixes = GetFixedAttributePrefixNamesMap().Find(GroupName))
			{
				// Don't allow removing a reserved attribute
				if (FixedAttributePrefixes->ContainsByPredicate([&Name](const FName& Prefix)
					{
						return NamesHaveMatchingPrefix(Name, Prefix);
					}))
				{
					return;
				}
			}
			// User defined attributes are only allowed in known group names. Do not allow removing fixed attributes through this method.
			if (!FixedAttributeNames->Contains(Name))
			{
				GetManagedArrayCollection()->RemoveAttribute(Name, GroupName);
			}
		}
	}

	template<typename T UE_REQUIRES_DEFINITION(TIsUserAttributeType<T>::Value)>
	bool FConstClothCollection::HasUserDefinedAttribute(const FName& Name, const FName& GroupName) const
	{
		using namespace UE::Chaos::ClothAsset::Private;
		if (const TArray<FName>* const FixedAttributePrefixes = GetFixedAttributePrefixNamesMap().Find(GroupName))
		{
			// Don't consider reserved attribute prefixes as user defined.
			if (FixedAttributePrefixes->ContainsByPredicate([&Name](const FName& Prefix)
				{
					return NamesHaveMatchingPrefix(Name, Prefix);
				}))
			{
				return false;
			}
		}
		if (const TArray<FName>* const FixedAttributeNames = GetFixedAttributeNamesMap().Find(GroupName))
		{
			// User defined attributes are only allowed in known group names.
			if (!FixedAttributeNames->Contains(Name))
			{
				return ManagedArrayCollection->FindAttributeTyped<T>(Name, GroupName) != nullptr;
			}
		}
		if (const TArray<FName>* const UserAccessibleAttributeNames = GetUserAccessibleAttributeNamesMap().Find(GroupName))
		{
			// Paintable attribute can only be found, not created (they are part of the schema)
			if (UserAccessibleAttributeNames->Contains(Name))
			{
				return ManagedArrayCollection->FindAttributeTyped<T>(Name, GroupName) != nullptr;
			}
		}
		return false;
	}
	template CHAOSCLOTHASSET_API bool FConstClothCollection::HasUserDefinedAttribute<bool>(const FName& Name, const FName& GroupName) const;
	template CHAOSCLOTHASSET_API bool FConstClothCollection::HasUserDefinedAttribute<int32>(const FName& Name, const FName& GroupName) const;
	template CHAOSCLOTHASSET_API bool FConstClothCollection::HasUserDefinedAttribute<float>(const FName& Name, const FName& GroupName) const;
	template CHAOSCLOTHASSET_API bool FConstClothCollection::HasUserDefinedAttribute<FVector3f>(const FName& Name, const FName& GroupName) const;
	template CHAOSCLOTHASSET_API bool FConstClothCollection::HasUserDefinedAttribute<TArray<int32>>(const FName& Name, const FName& GroupName) const;

	template<typename T UE_REQUIRES_DEFINITION(TIsUserAttributeType<T>::Value)>
	const TManagedArray<T>* FConstClothCollection::GetUserDefinedAttribute(const FName& Name, const FName& GroupName) const
	{
		using namespace UE::Chaos::ClothAsset::Private;
		if (const TArray<FName>* const FixedAttributePrefixes = GetFixedAttributePrefixNamesMap().Find(GroupName))
		{
			// Don't consider reserved attribute prefixes as user defined.
			if (FixedAttributePrefixes->ContainsByPredicate([&Name](const FName& Prefix)
				{
					return NamesHaveMatchingPrefix(Name, Prefix);
				}))
			{
				return nullptr;
			}
		}
		if (const TArray<FName>* const FixedAttributeNames = GetFixedAttributeNamesMap().Find(GroupName))
		{
			// User defined attributes are only allowed in known group names.
			if (!FixedAttributeNames->Contains(Name))
			{
				return ManagedArrayCollection->FindAttributeTyped<T>(Name, GroupName);
			}
		}
		if (const TArray<FName>* const UserAccessibleAttributeNames = GetUserAccessibleAttributeNamesMap().Find(GroupName))
		{
			// Paintable attribute can only be found, not created (they are part of the schema)
			if (UserAccessibleAttributeNames->Contains(Name))
			{
				return ManagedArrayCollection->FindAttributeTyped<T>(Name, GroupName);
			}
		}
		return nullptr;
	}
	template CHAOSCLOTHASSET_API const TManagedArray<bool>* FConstClothCollection::GetUserDefinedAttribute<bool>(const FName& Name, const FName& GroupName) const;
	template CHAOSCLOTHASSET_API const TManagedArray<int32>* FConstClothCollection::GetUserDefinedAttribute<int32>(const FName& Name, const FName& GroupName) const;
	template CHAOSCLOTHASSET_API const TManagedArray<float>* FConstClothCollection::GetUserDefinedAttribute<float>(const FName& Name, const FName& GroupName) const;
	template CHAOSCLOTHASSET_API const TManagedArray<FVector3f>* FConstClothCollection::GetUserDefinedAttribute<FVector3f>(const FName& Name, const FName& GroupName) const;
	template CHAOSCLOTHASSET_API const TManagedArray<TArray<int32>>* FConstClothCollection::GetUserDefinedAttribute<TArray<int32>>(const FName& Name, const FName& GroupName) const;

	template<typename T UE_REQUIRES_DEFINITION(TIsUserAttributeType<T>::Value)>
	TManagedArray<T>* FClothCollection::GetUserDefinedAttribute(const FName& Name, const FName& GroupName)
	{
		using namespace UE::Chaos::ClothAsset::Private;
		if (const TArray<FName>* const FixedAttributePrefixes = GetFixedAttributePrefixNamesMap().Find(GroupName))
		{
			// Don't consider reserved attribute prefixes as user defined.
			if (FixedAttributePrefixes->ContainsByPredicate([&Name](const FName& Prefix)
				{
					return NamesHaveMatchingPrefix(Name, Prefix);
				}))
			{
				return nullptr;
			}
		}
		if (const TArray<FName>* const FixedAttributeNames = GetFixedAttributeNamesMap().Find(GroupName))
		{
			// User defined attributes are only allowed in known group names.
			if (!FixedAttributeNames->Contains(Name))
			{
				return GetManagedArrayCollection()->FindAttributeTyped<T>(Name, GroupName);
			}
		}
		if (const TArray<FName>* const UserAccessibleAttributeNames = GetUserAccessibleAttributeNamesMap().Find(GroupName))
		{
			// Paintable attribute can only be found, not created (they are part of the schema)
			if (UserAccessibleAttributeNames->Contains(Name))
			{
				return GetManagedArrayCollection()->FindAttributeTyped<T>(Name, GroupName);
			}
		}
		return nullptr;
	}
	template CHAOSCLOTHASSET_API TManagedArray<bool>* FClothCollection::GetUserDefinedAttribute<bool>(const FName& Name, const FName& GroupName);
	template CHAOSCLOTHASSET_API TManagedArray<int32>* FClothCollection::GetUserDefinedAttribute<int32>(const FName& Name, const FName& GroupName);
	template CHAOSCLOTHASSET_API TManagedArray<float>* FClothCollection::GetUserDefinedAttribute<float>(const FName& Name, const FName& GroupName);
	template CHAOSCLOTHASSET_API TManagedArray<FVector3f>* FClothCollection::GetUserDefinedAttribute<FVector3f>(const FName& Name, const FName& GroupName);
	template CHAOSCLOTHASSET_API TManagedArray<TArray<int32>>* FClothCollection::GetUserDefinedAttribute<TArray<int32>>(const FName& Name, const FName& GroupName);

	/*static*/ TArray<FName> FConstClothCollection::GetValidClothCollectionGroupName()
	{
		using namespace UE::Chaos::ClothAsset::Private;
		TArray<FName> ValidClothCollectionGroupName;
		GetFixedAttributeNamesMap().GetKeys(ValidClothCollectionGroupName);
		return ValidClothCollectionGroupName;
	}

	/*static*/ bool FConstClothCollection::IsValidClothCollectionGroupName(const FName& GroupName)
	{
		using namespace UE::Chaos::ClothAsset::Private;
		return GetFixedAttributeNamesMap().Contains(GroupName);
	}

	bool FConstClothCollection::IsValidUserDefinedAttributeName(const FName& Name, const FName& GroupName) const
	{
		using namespace UE::Chaos::ClothAsset::Private;
		if (const TArray<FName>* const FixedAttributePrefixes = GetFixedAttributePrefixNamesMap().Find(GroupName))
		{
			// Don't consider reserved attribute prefixes as user defined.
			if (FixedAttributePrefixes->ContainsByPredicate([&Name](const FName& Prefix)
				{
					return NamesHaveMatchingPrefix(Name, Prefix);
				}))
			{
				return false;
			}
		}
		if (const TArray<FName>* const FixedAttributeNames = GetFixedAttributeNamesMap().Find(GroupName))
		{
			// User defined attributes are only allowed in known group names and cannot be reserved by FixedAttributeNames.
			if (!FixedAttributeNames->Contains(Name))
			{
				return true;
			}
		}
		if (const TArray<FName>* const UserAccessibleAttributeNames = GetUserAccessibleAttributeNamesMap().Find(GroupName))
		{
			// Paintable attribute can only be found, not created (they are part of the schema)
			if (UserAccessibleAttributeNames->Contains(Name) && ManagedArrayCollection->HasAttribute(Name, GroupName))
			{
				return true;
			}
		}
		return false;
	}

	const TManagedArray<FVector3f>* FConstClothCollection::GetSimAccessoryMeshPosition3D(const FName& AttributeName) const
	{
		if (AttributeName == NAME_None)
		{
			return nullptr;
		}
		check(Private::NamesHaveMatchingPrefix(AttributeName, ClothCollectionAttribute::SimAccessoryMeshPosition3DPrefix));
		return ManagedArrayCollection->FindAttribute<FVector3f>(AttributeName, ClothCollectionGroup::SimVertices3D);
	}

	const TManagedArray<FVector3f>* FConstClothCollection::GetSimAccessoryMeshNormal(const FName& AttributeName) const
	{
		if (AttributeName == NAME_None)
		{
			return nullptr;
		}
		check(Private::NamesHaveMatchingPrefix(AttributeName, ClothCollectionAttribute::SimAccessoryMeshNormalPrefix));
		return ManagedArrayCollection->FindAttribute<FVector3f>(AttributeName, ClothCollectionGroup::SimVertices3D);
	}

	const TManagedArray<TArray<int32>>* FConstClothCollection::GetSimAccessoryMeshBoneIndices(const FName& AttributeName) const
	{
		if (AttributeName == NAME_None)
		{
			return nullptr;
		}
		check(Private::NamesHaveMatchingPrefix(AttributeName, ClothCollectionAttribute::SimAccessoryMeshBoneIndicesPrefix));
		return ManagedArrayCollection->FindAttribute<TArray<int32>>(AttributeName, ClothCollectionGroup::SimVertices3D);
	}

	const TManagedArray<TArray<float>>* FConstClothCollection::GetSimAccessoryMeshBoneWeights(const FName& AttributeName) const
	{
		if (AttributeName == NAME_None)
		{
			return nullptr;
		}
		check(Private::NamesHaveMatchingPrefix(AttributeName, ClothCollectionAttribute::SimAccessoryMeshBoneWeightsPrefix));
		return ManagedArrayCollection->FindAttribute<TArray<float>>(AttributeName, ClothCollectionGroup::SimVertices3D);
	}

	FName FClothCollection::AddSimAccessoryMeshPosition3D()
	{
		return Private::AddUniquePrefixedAttribute<FVector3f>(ClothCollectionGroup::SimVertices3D, ClothCollectionAttribute::SimAccessoryMeshPosition3DPrefix, GetSimAccessoryMeshPosition3DAttribute(), GetManagedArrayCollection());
	}

	FName FClothCollection::AddSimAccessoryMeshNormal()
	{
		return Private::AddUniquePrefixedAttribute<FVector3f>(ClothCollectionGroup::SimVertices3D, ClothCollectionAttribute::SimAccessoryMeshNormalPrefix, GetSimAccessoryMeshNormalAttribute(), GetManagedArrayCollection());
	}

	FName FClothCollection::AddSimAccessoryMeshBoneIndices()
	{
		return Private::AddUniquePrefixedAttribute<TArray<int32>>(ClothCollectionGroup::SimVertices3D, ClothCollectionAttribute::SimAccessoryMeshBoneIndicesPrefix, GetSimAccessoryMeshBoneIndicesAttribute(), GetManagedArrayCollection());
	}

	FName FClothCollection::AddSimAccessoryMeshBoneWeights()
	{
		return Private::AddUniquePrefixedAttribute<TArray<float>>(ClothCollectionGroup::SimVertices3D, ClothCollectionAttribute::SimAccessoryMeshBoneWeightsPrefix, GetSimAccessoryMeshBoneWeightsAttribute(), GetManagedArrayCollection());
	}

	void FClothCollection::RemoveSimAccessoryMeshPosition3D(const FName& AttributeName)
	{
		check(Private::NamesHaveMatchingPrefix(AttributeName, ClothCollectionAttribute::SimAccessoryMeshPosition3DPrefix));
		GetManagedArrayCollection()->RemoveAttribute(AttributeName, ClothCollectionGroup::SimVertices3D);
	}

	void FClothCollection::RemoveSimAccessoryMeshNormal(const FName& AttributeName)
	{
		check(Private::NamesHaveMatchingPrefix(AttributeName, ClothCollectionAttribute::SimAccessoryMeshNormalPrefix));
		GetManagedArrayCollection()->RemoveAttribute(AttributeName, ClothCollectionGroup::SimVertices3D);
	}

	void FClothCollection::RemoveSimAccessoryMeshBoneIndices(const FName& AttributeName)
	{
		check(Private::NamesHaveMatchingPrefix(AttributeName, ClothCollectionAttribute::SimAccessoryMeshBoneIndicesPrefix));
		GetManagedArrayCollection()->RemoveAttribute(AttributeName, ClothCollectionGroup::SimVertices3D);
	}

	void FClothCollection::RemoveSimAccessoryMeshBoneWeights(const FName& AttributeName)
	{
		check(Private::NamesHaveMatchingPrefix(AttributeName, ClothCollectionAttribute::SimAccessoryMeshBoneWeightsPrefix));
		GetManagedArrayCollection()->RemoveAttribute(AttributeName, ClothCollectionGroup::SimVertices3D);
	}


} // End namespace UE::Chaos::ClothAsset
