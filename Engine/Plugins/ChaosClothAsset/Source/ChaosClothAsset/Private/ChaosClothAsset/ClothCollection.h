// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryCollection/ManagedArray.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "ChaosClothAsset/ClothCollectionOptionalSchemas.h"
#endif
#include "ChaosClothAsset/ClothCollectionExtendedSchemas.h"
#include "ChaosClothAsset/IsUserAttributeType.h"

struct FManagedArrayCollection;

namespace UE::Chaos::ClothAsset
{
	/**
	 * Cloth collection facade data.
	 */
	class FConstClothCollection
	{
	public:
		static constexpr int8 MaxNumBoneInfluences = 12; // This should be <= MAX_TOTAL_INFLUENCES defined in GPUSkinPublicDefs.h 
		static constexpr int8 MaxNumTetherAttachments = 4; // This should be <= FClothTetherDataPrivate::MaxNumAttachments

		explicit FConstClothCollection(const TSharedRef<const FManagedArrayCollection>& InManagedArrayCollection);
		~FConstClothCollection() = default;

		FConstClothCollection(const FConstClothCollection&) = delete;
		FConstClothCollection& operator=(const FConstClothCollection&) = delete;
		FConstClothCollection(FConstClothCollection&&) = delete;
		FConstClothCollection& operator=(FConstClothCollection&&) = delete;

		/** Return whether the underlying collection is a valid cloth collection. */
		bool IsValid(EClothCollectionExtendedSchemas OptionalSchemas = EClothCollectionExtendedSchemas::None) const;

		/** Get the number of elements of a group. */
		int32 GetNumElements(const FName& GroupName) const;

		/** Get the number of elements of one of the sub groups that have start/end indices. */
		int32 GetNumElements(const TManagedArray<int32>* StartArray, const TManagedArray<int32>* EndArray, int32 Index) const;

		template<typename T>
		static inline TConstArrayView<T> GetElements(const TManagedArray<T>* ElementArray, const TManagedArray<int32>* StartArray, const TManagedArray<int32>* EndArray, int32 ArrayIndex);

		template<typename T>
		static inline TArrayView<T> GetElements(TManagedArray<T>* ElementArray, const TManagedArray<int32>* StartArray, const TManagedArray<int32>* EndArray, int32 ArrayIndex);

		template<typename T>
		static inline TConstArrayView<T> GetElements(const TManagedArray<T>* ElementArray);

		template<typename T>
		static inline TArrayView<T> GetElements(TManagedArray<T>* ElementArray);

		/**
		 * Return the difference between the start index of an element to the start index of the first sub-element in the group (base).
		 * Usefull for getting back and forth between LOD/pattern indexation modes.
		 */
		static int32 GetElementsOffset(const TManagedArray<int32>* StartArray, int32 BaseElementIndex, int32 ElementIndex);

		/**
		* Return the ArrayIndex (into StartArray/EndArray) which corresponds with the subarray that contains ElementIndex.
		* Useful for finding which pattern contains a vertex or face index. Returns INDEX_NONE if not found.
		* NOTE: typically we have a small number of patterns (case where this is used), so just doing a linear search since
		* a bisection search would get confused by the empty patterns.
		*/
		static int32 GetArrayIndexForContainedElement(
			const TManagedArray<int32>* StartArray,
			const TManagedArray<int32>* EndArray,
			int32 ElementIndex);

		static int32 GetNumSubElements(
			const TManagedArray<int32>* StartArray,
			const TManagedArray<int32>* EndArray,
			const TManagedArray<int32>* StartSubArray,
			const TManagedArray<int32>* EndSubArray,
			int32 LodIndex);

		template<typename T>
		static inline TConstArrayView<T> GetSubElements(
			const TManagedArray<T>* SubElementArray,
			const TManagedArray<int32>* StartArray,
			const TManagedArray<int32>* EndArray,
			const TManagedArray<int32>* StartSubArray,
			const TManagedArray<int32>* EndSubArray,
			int32 ArrayIndex);

		template<typename T>
		static inline TArrayView<T> GetSubElements(
			TManagedArray<T>* SubElementArray,
			const TManagedArray<int32>* StartArray,
			const TManagedArray<int32>* EndArray,
			const TManagedArray<int32>* StartSubArray,
			const TManagedArray<int32>* EndSubArray,
			int32 ArrayIndex);

		template<bool bStart = true, bool bEnd = true>
		static TTuple<int32, int32> GetSubElementsStartEnd(
			const TManagedArray<int32>* StartArray,
			const TManagedArray<int32>* EndArray,
			const TManagedArray<int32>* StartSubArray,
			const TManagedArray<int32>* EndSubArray,
			int32 ArrayIndex);


		/** Useful method for copying data between ArrayViews. Putting here for now. */
		template<typename T>
		static inline void CopyArrayViewData(const TArrayView<T>& To, const TConstArrayView<T>& From);

		template<typename T UE_REQUIRES(!std::is_array_v<T>)>
		static inline void CopyArrayViewDataAndApplyOffset(const TArrayView<T>& To, const TConstArrayView<T>& From, const T Offset);

		template<typename T>
		static inline void CopyArrayViewDataAndApplyOffset(const TArrayView<TArray<T>>& To, const TConstArrayView<TArray<T>>& From, const T Offset);

		template<typename T>
		static inline uint32 GetElementsTypeHash(const TManagedArray<T>* ElementArray);

		//~ Weight maps
		template<typename T UE_REQUIRES(TIsUserAttributeType<T>::Value)>
		TArray<FName> GetUserDefinedAttributeNames(const FName& GroupName) const;

		template<typename T UE_REQUIRES(TIsUserAttributeType<T>::Value)>
		bool HasUserDefinedAttribute(const FName& Name, const FName& GroupName) const;

		template<typename T UE_REQUIRES(TIsUserAttributeType<T>::Value)>
		const TManagedArray<T>* GetUserDefinedAttribute(const FName& Name, const FName& GroupName) const;

		static TArray<FName> GetValidClothCollectionGroupName();
		static bool IsValidClothCollectionGroupName(const FName& GroupName);
		bool IsValidUserDefinedAttributeName(const FName& Name, const FName& GroupName) const;

		//~ LODs Group (There should be only one LOD per ClothCollection)
		const TManagedArray<FSoftObjectPath>* GetPhysicsAssetSoftObjectPathName() const { return PhysicsAssetSoftObjectPathName; }
		const TManagedArray<FSoftObjectPath>* GetSkeletalMeshSoftObjectPathName() const { return SkeletalMeshSoftObjectPathName; }
		UE_DEPRECATED(5.7, "Use GetPhysicsAssetSoftObjectPathName instead")
		const TManagedArray<FString>* GetPhysicsAssetPathName() const { return nullptr; }
		UE_DEPRECATED(5.7, "Use GetSkeletalMeshSoftObjectPathName instead")
		const TManagedArray<FString>* GetSkeletalMeshPathName() const { return nullptr; }
		const TManagedArray<FName>* GetReferenceBoneName() const { return ReferenceBoneName; }
		const TManagedArray<bool>* GetCopyAllUserAttributesToSimModel() const { return CopyAllUserAttributesToSimModel; }

		// ~ Solvers Group
		const TManagedArray<FVector3f>* GetSolverGravity() const { return SolverGravity; }
		const TManagedArray<float>* GetSolverAirDamping() const { return SolverAirDamping; }
		const TManagedArray<int32>* GetSolverSubSteps() const { return SolverSubSteps; }
		const TManagedArray<float>* GetSolverTimeStep() const { return SolverTimeStep; }

		//~ Seam Group
		const TManagedArray<int32>* GetSeamStitchStart() const { return SeamStitchStart; }
		const TManagedArray<int32>* GetSeamStitchEnd() const { return SeamStitchEnd; }

		//~ Seam Stitches Group
		const TManagedArray<FIntVector2>* GetSeamStitch2DEndIndices() const { return SeamStitch2DEndIndices; }
		const TManagedArray<int32>* GetSeamStitch3DIndex() const { return SeamStitch3DIndex; }

		//~ Fabric Group
		const TManagedArray<FVector3f>* GetFabricBendingStiffness() const { return FabricBendingStiffness; }
		const TManagedArray<FVector3f>* GetFabricBucklingStiffness() const { return FabricBucklingStiffness; }
		const TManagedArray<FVector3f>* GetFabricStretchStiffness() const { return FabricStretchStiffness; }
		const TManagedArray<float>* GetFabricBucklingRatio() const { return FabricBucklingRatio; }
		const TManagedArray<float>* GetFabricDensity() const { return FabricDensity; }
		const TManagedArray<float>* GetFabricFriction() const { return FabricFriction; }
		const TManagedArray<float>* GetFabricDamping() const { return FabricDamping; }
		const TManagedArray<float>* GetFabricPressure() const { return FabricPressure; }
		const TManagedArray<int32>* GetFabricLayer() const { return FabricLayer; }
		const TManagedArray<float>* GetFabricCollisionThickness() const { return FabricCollisionThickness; }
		
		//~ Sim Patterns Group
		const TManagedArray<int32>* GetSimVertices2DStart() const { return SimVertices2DStart; }
		const TManagedArray<int32>* GetSimVertices2DEnd() const { return SimVertices2DEnd; }
		const TManagedArray<int32>* GetSimFacesStart() const { return SimFacesStart; }
		const TManagedArray<int32>* GetSimFacesEnd() const { return SimFacesEnd; }
		const TManagedArray<int32>* GetSimPatternFabric() const { return SimPatternFabric; }
		
		//~ Render Patterns Group
		const TManagedArray<int32>* GetRenderVerticesStart() const { return RenderVerticesStart; }
		const TManagedArray<int32>* GetRenderVerticesEnd() const { return RenderVerticesEnd; }
		const TManagedArray<int32>* GetRenderFacesStart() const { return RenderFacesStart; }
		const TManagedArray<int32>* GetRenderFacesEnd() const { return RenderFacesEnd; }
		const TManagedArray<int32>* GetRenderDeformerNumInfluences() const { return RenderDeformerNumInfluences; }
		const TManagedArray<FSoftObjectPath>* GetRenderMaterialSoftObjectPathName() const { return RenderMaterialSoftObjectPathName; }
		UE_DEPRECATED(5.7, "Use GetRenderMaterialSoftObjectPathName instead")
		const TManagedArray<FString>* GetRenderMaterialPathName() const { return nullptr; }

		//~ Sim Faces Group
		const TManagedArray<FIntVector3>* GetSimIndices2D() const { return SimIndices2D; }
		const TManagedArray<FIntVector3>* GetSimIndices3D() const { return SimIndices3D; }

		//~ Sim Vertices 2D Group
		const TManagedArray<FVector2f>* GetSimPosition2D() const { return SimPosition2D; }
		const TManagedArray<int32>* GetSimVertex3DLookup() const { return SimVertex3DLookup; }
		const TManagedArray<int32>* GetSimImportVertexID() const { return SimImportVertexID; }

		//~ Sim Vertices 3D Group
		const TManagedArray<FVector3f>* GetSimPosition3D() const { return SimPosition3D; }
		const TManagedArray<FVector3f>* GetPreResizedSimPosition3D() const { return PreResizedSimPosition3D; }
		const TManagedArray<FVector3f>* GetSimNormal() const { return SimNormal; }
		const TManagedArray<TArray<int32>>* GetSimBoneIndices() const { return SimBoneIndices; }
		const TManagedArray<TArray<float>>* GetSimBoneWeights() const { return SimBoneWeights; }
		const TManagedArray<TArray<int32>>* GetTetherKinematicIndex() const { return TetherKinematicIndex; }
		const TManagedArray<TArray<float>>* GetTetherReferenceLength() const { return TetherReferenceLength; }
		const TManagedArray<TArray<int32>>* GetSimVertex2DLookup() const { return SimVertex2DLookup; }
		const TManagedArray<TArray<int32>>* GetSeamStitchLookup() const { return SeamStitchLookup; }
		const TManagedArray<float>* GetSimCustomResizingBlend() const { return SimCustomResizingBlend; }

		//~ Sim Vertices 3D Group -- Sim Accessory Mesh
		const TManagedArray<FVector3f>* GetSimAccessoryMeshPosition3D(const FName& AttributeName) const;
		const TManagedArray<FVector3f>* GetSimAccessoryMeshNormal(const FName& AttributeName) const;
		const TManagedArray<TArray<int32>>* GetSimAccessoryMeshBoneIndices(const FName& AttributeName) const;
		const TManagedArray<TArray<float>>* GetSimAccessoryMeshBoneWeights(const FName& AttributeName) const;

		//~ Sim Morph Targets Group
		const TManagedArray<FString>* GetSimMorphTargetName() const { return SimMorphTargetName; }
		const TManagedArray<int32>* GetSimMorphTargetVerticesStart() const { return SimMorphTargetVerticesStart; }
		const TManagedArray<int32>* GetSimMorphTargetVerticesEnd() const { return SimMorphTargetVerticesEnd; }

		//~ Sim Morph Targets Vertices Group
		const TManagedArray<FVector3f>* GetSimMorphTargetPositionDelta() const { return SimMorphTargetPositionDelta; }
		const TManagedArray<FVector3f>* GetSimMorphTargetTangentZDelta() const { return SimMorphTargetTangentZDelta; }
		const TManagedArray<int32>* GetSimMorphTargetSimVertex3DIndex() const { return SimMorphTargetSimVertex3DIndex; }

		//~ Render Faces Group
		const TManagedArray<FIntVector3>* GetRenderIndices() const { return RenderIndices; }

		//~ Render Vertices Group
		const TManagedArray<FVector3f>* GetRenderPosition() const { return RenderPosition; }
		const TManagedArray<FVector3f>* GetRenderNormal() const { return RenderNormal; }
		const TManagedArray<FVector3f>* GetRenderTangentU() const { return RenderTangentU; }
		const TManagedArray<FVector3f>* GetRenderTangentV() const { return RenderTangentV; }
		const TManagedArray<TArray<FVector2f>>* GetRenderUVs() const { return RenderUVs; }
		const TManagedArray<FLinearColor>* GetRenderColor() const { return RenderColor; }
		const TManagedArray<TArray<int32>>* GetRenderBoneIndices() const { return RenderBoneIndices; }
		const TManagedArray<TArray<float>>* GetRenderBoneWeights() const { return RenderBoneWeights; }
		const TManagedArray<TArray<FVector4f>>* GetRenderDeformerPositionBaryCoordsAndDist() const { return RenderDeformerPositionBaryCoordsAndDist; }
		const TManagedArray<TArray<FVector4f>>* GetRenderDeformerNormalBaryCoordsAndDist() const { return RenderDeformerNormalBaryCoordsAndDist; }
		const TManagedArray<TArray<FVector4f>>* GetRenderDeformerTangentBaryCoordsAndDist() const { return RenderDeformerTangentBaryCoordsAndDist; }
		const TManagedArray<TArray<FIntVector3>>* GetRenderDeformerSimIndices3D() const { return RenderDeformerSimIndices3D; }
		const TManagedArray<TArray<float>>* GetRenderDeformerWeight() const { return RenderDeformerWeight; }
		const TManagedArray<float>* GetRenderDeformerSkinningBlend() const { return RenderDeformerSkinningBlend; }
		const TManagedArray<float>* GetRenderCustomResizingBlend() const { return RenderCustomResizingBlend; }

		//~ Custom Resizing Regions Group
		const TManagedArray<FString>* GetCustomResizingRegionSet() const { return CustomResizingRegionSet; }
		const TManagedArray<int32>* GetCustomResizingRegionType() const { return CustomResizingRegionType; }

		//~ Sim Accessory Meshes Group
		const TManagedArray<FName>* GetSimAccessoryMeshName() const { return SimAccessoryMeshName; }
		const TManagedArray<FName>* GetSimAccessoryMeshPosition3DAttribute() const { return SimAccessoryMeshPosition3DAttribute; }
		const TManagedArray<FName>* GetSimAccessoryMeshNormalAttribute() const { return SimAccessoryMeshNormalAttribute; }
		const TManagedArray<FName>* GetSimAccessoryMeshBoneIndicesAttribute() const { return SimAccessoryMeshBoneIndicesAttribute; }
		const TManagedArray<FName>* GetSimAccessoryMeshBoneWeightsAttribute() const { return SimAccessoryMeshBoneWeightsAttribute; }

	protected:
		struct TProtectedInit {};

		template<typename T UE_REQUIRES(std::is_same_v<T, FManagedArrayCollection> || std::is_same_v<T, const FManagedArrayCollection>)>
		FConstClothCollection(const TSharedRef<T>& InManagedArrayCollection, TProtectedInit);

		//~ Cloth collection
		TSharedRef<const FManagedArrayCollection> ManagedArrayCollection;

		//~ LODs Group
		const TManagedArray<FSoftObjectPath>* PhysicsAssetSoftObjectPathName;
		const TManagedArray<FSoftObjectPath>* SkeletalMeshSoftObjectPathName;
		UE_DEPRECATED(5.7, "Use PhysicsAssetSoftObjectPathName instead")
		const TManagedArray<FString>* PhysicsAssetPathName;
		UE_DEPRECATED(5.7, "Use SkeletalMeshSoftObjectPathName instead")
		const TManagedArray<FString>* SkeletalMeshPathName;
		const TManagedArray<FName>* ReferenceBoneName;
		const TManagedArray<bool>* CopyAllUserAttributesToSimModel;

		//~ Solvers group
		const TManagedArray<FVector3f>* SolverGravity;
		const TManagedArray<float>* SolverAirDamping;
		const TManagedArray<int32>* SolverSubSteps;
		const TManagedArray<float>* SolverTimeStep;

		//~ Fabrics Group
		const TManagedArray<FVector3f>* FabricBendingStiffness;
		const TManagedArray<FVector3f>* FabricBucklingStiffness;
		const TManagedArray<FVector3f>* FabricStretchStiffness;
		const TManagedArray<float>* FabricBucklingRatio;
		const TManagedArray<float>* FabricDensity;
		const TManagedArray<float>* FabricFriction;
		const TManagedArray<float>* FabricDamping;
		const TManagedArray<float>* FabricPressure;
		const TManagedArray<int32>* FabricLayer;
		const TManagedArray<float>* FabricCollisionThickness;

		//~ Seam Group
		const TManagedArray<int32>* SeamStitchStart;
		const TManagedArray<int32>* SeamStitchEnd;

		//~ Seam Stitches Group
		const TManagedArray<FIntVector2>* SeamStitch2DEndIndices;  // Stitched 2D vertex indices pair
		const TManagedArray<int32>* SeamStitch3DIndex;  // Corresponding stitched 3D vertex

		//~ Sim Patterns Group
		const TManagedArray<int32>* SimVertices2DStart;
		const TManagedArray<int32>* SimVertices2DEnd;
		const TManagedArray<int32>* SimFacesStart;
		const TManagedArray<int32>* SimFacesEnd;
		const TManagedArray<int32>* SimPatternFabric;

		//~ Render Patterns Group
		const TManagedArray<int32>* RenderVerticesStart;
		const TManagedArray<int32>* RenderVerticesEnd;
		const TManagedArray<int32>* RenderFacesStart;
		const TManagedArray<int32>* RenderFacesEnd;
		const TManagedArray<int32>* RenderDeformerNumInfluences;  // Number of deformer mapping influences per render vertex, either 0 (no mappings), 1 or 5 (= NUM_INFLUENCES_PER_VERTEX as defined in the ush files)
		const TManagedArray<FSoftObjectPath>* RenderMaterialSoftObjectPathName;
		UE_DEPRECATED(5.7, "Use RenderMaterialSoftObjectPathName instead")
		const TManagedArray<FString>* RenderMaterialPathName;

		//~ Sim Faces Group
		const TManagedArray<FIntVector3>* SimIndices2D;
		const TManagedArray<FIntVector3>* SimIndices3D;

		//~ Sim Vertices 2D Group
		const TManagedArray<FVector2f>* SimPosition2D;
		const TManagedArray<int32>* SimVertex3DLookup; // Lookup into corresponding 3D vertices
		const TManagedArray<int32>* SimImportVertexID; // Can be populated by importers to match data 

		//~ Sim Vertices 3D Group
		const TManagedArray<FVector3f>* SimPosition3D;
		const TManagedArray<FVector3f>* SimNormal;  // Used for capture, maxdistance, backstop authoring ...etc
		const TManagedArray<TArray<int32>>* SimBoneIndices;
		const TManagedArray<TArray<float>>* SimBoneWeights;
		const TManagedArray<TArray<int32>>* TetherKinematicIndex;
		const TManagedArray<TArray<float>>* TetherReferenceLength;
		const TManagedArray<TArray<int32>>* SimVertex2DLookup; // Lookup into corresponding 2D vertices
		const TManagedArray<TArray<int32>>* SeamStitchLookup; // Lookup into any seam stitches which weld this vertex
		const TManagedArray<float>* SimCustomResizingBlend; // Resizing weight map. How much to use group resizing result (1) vs rbf interpolation result (0) when doing sim mesh resizing
		const TManagedArray<FVector3f>* PreResizedSimPosition3D;

		//~ Sim Morph Targets Group
		const TManagedArray<FString>* SimMorphTargetName;
		const TManagedArray<int32>* SimMorphTargetVerticesStart;
		const TManagedArray<int32>* SimMorphTargetVerticesEnd;

		//~ Sim Morph Target Vertices Group
		const TManagedArray<FVector3f>* SimMorphTargetPositionDelta;
		const TManagedArray<FVector3f>* SimMorphTargetTangentZDelta;
		const TManagedArray<int32>* SimMorphTargetSimVertex3DIndex;

		//~ Render Faces Group
		const TManagedArray<FIntVector3>* RenderIndices;

		//~ Render Vertices Group
		const TManagedArray<FVector3f>* RenderPosition;
		const TManagedArray<FVector3f>* RenderNormal;
		const TManagedArray<FVector3f>* RenderTangentU;
		const TManagedArray<FVector3f>* RenderTangentV;
		const TManagedArray<TArray<FVector2f>>* RenderUVs;
		const TManagedArray<FLinearColor>* RenderColor;
		const TManagedArray<TArray<int32>>* RenderBoneIndices;
		const TManagedArray<TArray<float>>* RenderBoneWeights;
		const TManagedArray<TArray<FVector4f>>* RenderDeformerPositionBaryCoordsAndDist;  // Barycentric coords and distance along normal for the position of the final vert
		const TManagedArray<TArray<FVector4f>>* RenderDeformerNormalBaryCoordsAndDist;  // Barycentric coords and distance along normal for the location of the unit normal endpoint
		const TManagedArray<TArray<FVector4f>>* RenderDeformerTangentBaryCoordsAndDist;  // Barycentric coords and distance along normal for the location of the unit Tangent endpoint
		const TManagedArray<TArray<FIntVector3>>* RenderDeformerSimIndices3D;  // The source mesh triangle
		const TManagedArray<TArray<float>>* RenderDeformerWeight;  // For weighted averaging of multiple triangle influences
		const TManagedArray<float>* RenderDeformerSkinningBlend;  // Render weight map. How much the vertex actually contributes, value between 0 (fully deformed) and 1 (fully skinned)
		const TManagedArray<float>* RenderCustomResizingBlend; // Resizing weight map. How much to use group resizing result (1) vs rbf interpolation result (0) when doing render mesh resizing

		//~ Group Resizing Group
		const TManagedArray<FString>* CustomResizingRegionSet; // Name of the set associated with this binding type
		const TManagedArray<int32>* CustomResizingRegionType; // Actually an Enum for binding type

		//~ Sim Accessory Meshes Group
		const TManagedArray<FName>* SimAccessoryMeshName;
		const TManagedArray<FName>* SimAccessoryMeshPosition3DAttribute;
		const TManagedArray<FName>* SimAccessoryMeshNormalAttribute;
		const TManagedArray<FName>* SimAccessoryMeshBoneIndicesAttribute;
		const TManagedArray<FName>* SimAccessoryMeshBoneWeightsAttribute;
	};

	/**
	 * Cloth collection facade data.
	 */
	class FClothCollection final : public FConstClothCollection
	{
	public:
		explicit FClothCollection(const TSharedRef<FManagedArrayCollection>& InManagedArrayCollection);
		~FClothCollection() = default;

		FClothCollection(const FClothCollection&) = delete;
		FClothCollection& operator=(const FClothCollection&) = delete;
		FClothCollection(FClothCollection&&) = delete;
		FClothCollection& operator=(FClothCollection&&) = delete;

		/** Make the underlying collection a cloth collection. */
		void DefineSchema(EClothCollectionExtendedSchemas OptionalSchemas = EClothCollectionExtendedSchemas::None);

		/** Do any PostSerialize fixups */
		void PostSerialize(const FArchive& Ar);

		/** Set the number of elements of a group. */
		void SetNumElements(int32 InNumElements, const FName& GroupName);

		/** Set the number of elements to one of the sub groups while maintaining the correct order of the data, and return the first index of the range. */
		int32 SetNumElements(int32 InNumElements, const FName& GroupName, TManagedArray<int32>* StartArray, TManagedArray<int32>* EndArray, int32 Index);

		/** Remove Elements */
		void RemoveElements(const FName& Group, const TArray<int32>& SortedDeletionList);

		/** Remove Elements. SortedDeletionList should be global indices. */
		void RemoveElements(const FName& GroupName, const TArray<int32>& SortedDeletionList, TManagedArray<int32>* StartArray, TManagedArray<int32>* EndArray, int32 Index);

		template<typename T UE_REQUIRES(TIsUserAttributeType<T>::Value)>
		TManagedArray<T>* FindOrAddUserDefinedAttribute(const FName& Name, const FName& GroupName, const FName& GroupDependency = NAME_None);

		template<typename T UE_REQUIRES(TIsUserAttributeType<T>::Value)>
		void AddUserDefinedAttribute(const FName& Name, const FName& GroupName, const FName& GroupDependency = NAME_None)
		{
			FindOrAddUserDefinedAttribute<T>(Name, GroupName, GroupDependency);
		}

		void RemoveUserDefinedAttribute(const FName& Name, const FName& GroupName);

		template<typename T UE_REQUIRES(TIsUserAttributeType<T>::Value)>
		TManagedArray<T>* GetUserDefinedAttribute(const FName& Name, const FName& GroupName);

		//~ LODs Group (There should be only one LOD per ClothCollection)
		TManagedArray<FSoftObjectPath>* GetPhysicsAssetSoftObjectPathName() { return  const_cast<TManagedArray<FSoftObjectPath>*>(PhysicsAssetSoftObjectPathName); }
		TManagedArray<FSoftObjectPath>* GetSkeletalMeshSoftObjectPathName() { return  const_cast<TManagedArray<FSoftObjectPath>*>(SkeletalMeshSoftObjectPathName); }
		UE_DEPRECATED(5.7, "Use GetPhysicsAssetSoftObjectPathName instead")
		TManagedArray<FString>* GetPhysicsAssetPathName() { return nullptr; }
		UE_DEPRECATED(5.7, "Use GetSkeletalMeshSoftObjectPathName instead")
		TManagedArray<FString>* GetSkeletalMeshPathName() { return nullptr; }
		TManagedArray<FName>* GetReferenceBoneName() { return  const_cast<TManagedArray<FName>*>(ReferenceBoneName); }
		TManagedArray<bool>* GetCopyAllUserAttributesToSimModel() { return const_cast<TManagedArray<bool>*>(CopyAllUserAttributesToSimModel); }

		//~ Solver Group
		TManagedArray<FVector3f>* GetSolverGravity() { return const_cast<TManagedArray<FVector3f>*>(SolverGravity); }
		TManagedArray<float>* GetSolverAirDamping() { return const_cast<TManagedArray<float>*>(SolverAirDamping); }
		TManagedArray<int32>* GetSolverSubSteps() { return const_cast<TManagedArray<int32>*>(SolverSubSteps); }
		TManagedArray<float>* GetSolverTimeStep() { return const_cast<TManagedArray<float>*>(SolverTimeStep); }

		//~ Fabric Group
		TManagedArray<FVector3f>* GetFabricBendingStiffness() { return const_cast<TManagedArray<FVector3f>*>(FabricBendingStiffness); }
		TManagedArray<FVector3f>* GetFabricBucklingStiffness() { return const_cast<TManagedArray<FVector3f>*>(FabricBucklingStiffness); }
		TManagedArray<FVector3f>* GetFabricStretchStiffness() { return const_cast<TManagedArray<FVector3f>*>(FabricStretchStiffness); }
		TManagedArray<float>* GetFabricBucklingRatio() { return const_cast<TManagedArray<float>*>(FabricBucklingRatio); }
		TManagedArray<float>* GetFabricDensity() { return const_cast<TManagedArray<float>*>(FabricDensity); }
		TManagedArray<float>* GetFabricFriction() { return const_cast<TManagedArray<float>*>(FabricFriction); }
		TManagedArray<float>* GetFabricDamping() { return const_cast<TManagedArray<float>*>(FabricDamping); }
		TManagedArray<float>* GetFabricPressure() { return const_cast<TManagedArray<float>*>(FabricPressure); }
		TManagedArray<int32>* GetFabricLayer() { return const_cast<TManagedArray<int32>*>(FabricLayer); }
		TManagedArray<float>* GetFabricCollisionThickness() { return const_cast<TManagedArray<float>*>(FabricCollisionThickness); }

		//~ Seam Group
		TManagedArray<int32>* GetSeamStitchStart() { return const_cast<TManagedArray<int32>*>(SeamStitchStart); }
		TManagedArray<int32>* GetSeamStitchEnd() { return const_cast<TManagedArray<int32>*>(SeamStitchEnd); }

		//~ Seam Stitches Group
		TManagedArray<FIntVector2>* GetSeamStitch2DEndIndices() { return const_cast<TManagedArray<FIntVector2>*>(SeamStitch2DEndIndices); }
		TManagedArray<int32>* GetSeamStitch3DIndex() { return const_cast<TManagedArray<int32>*>(SeamStitch3DIndex); }

		//~ Sim Patterns Group
		TManagedArray<int32>* GetSimVertices2DStart() { return const_cast<TManagedArray<int32>*>(SimVertices2DStart); }
		TManagedArray<int32>* GetSimVertices2DEnd() { return const_cast<TManagedArray<int32>*>(SimVertices2DEnd); }
		TManagedArray<int32>* GetSimFacesStart() { return const_cast<TManagedArray<int32>*>(SimFacesStart); }
		TManagedArray<int32>* GetSimFacesEnd() { return const_cast<TManagedArray<int32>*>(SimFacesEnd); }
		TManagedArray<int32>* GetSimPatternFabric() { return const_cast<TManagedArray<int32>*>(SimPatternFabric); }

		//~ Render Patterns Group
		TManagedArray<int32>* GetRenderVerticesStart() { return const_cast<TManagedArray<int32>*>(RenderVerticesStart); }
		TManagedArray<int32>* GetRenderVerticesEnd() { return const_cast<TManagedArray<int32>*>(RenderVerticesEnd); }
		TManagedArray<int32>* GetRenderFacesStart() { return const_cast<TManagedArray<int32>*>(RenderFacesStart); }
		TManagedArray<int32>* GetRenderFacesEnd() { return const_cast<TManagedArray<int32>*>(RenderFacesEnd); }
		TManagedArray<int32>* GetRenderDeformerNumInfluences() { return const_cast<TManagedArray<int32>*>(RenderDeformerNumInfluences); }
		TManagedArray<FSoftObjectPath>* GetRenderMaterialSoftObjectPathName() { return const_cast<TManagedArray<FSoftObjectPath>*>(RenderMaterialSoftObjectPathName); }
		UE_DEPRECATED(5.7, "Use GetRenderMaterialSoftObjectPathName instead")
		TManagedArray<FString>* GetRenderMaterialPathName() { return nullptr; }

		//~ Sim Faces Group
		TManagedArray<FIntVector3>* GetSimIndices2D() { return const_cast<TManagedArray<FIntVector3>*>(SimIndices2D); }
		TManagedArray<FIntVector3>* GetSimIndices3D() { return const_cast<TManagedArray<FIntVector3>*>(SimIndices3D); }

		//~ Sim Vertices 2D Group
		TManagedArray<FVector2f>* GetSimPosition2D() { return const_cast<TManagedArray<FVector2f>*>(SimPosition2D); }
		TManagedArray<int32>* GetSimVertex3DLookup() { return const_cast<TManagedArray<int32>*>(SimVertex3DLookup); }
		TManagedArray<int32>* GetSimImportVertexID() { return const_cast<TManagedArray<int32>*>(SimImportVertexID); }

		//~ Sim Vertices 3D Group
		TManagedArray<FVector3f>* GetSimPosition3D() { return const_cast<TManagedArray<FVector3f>*>(SimPosition3D); }
		TManagedArray<FVector3f>* GetPreResizedSimPosition3D() { return const_cast<TManagedArray<FVector3f>*>(PreResizedSimPosition3D); }
		TManagedArray<FVector3f>* GetSimNormal() { return const_cast<TManagedArray<FVector3f>*>(SimNormal); }
		TManagedArray<TArray<int32>>* GetSimBoneIndices() { return const_cast<TManagedArray<TArray<int32>>*>(SimBoneIndices); }
		TManagedArray<TArray<float>>* GetSimBoneWeights() { return const_cast<TManagedArray<TArray<float>>*>(SimBoneWeights); }
		TManagedArray<TArray<int32>>* GetTetherKinematicIndex() { return const_cast<TManagedArray<TArray<int32>>*>(TetherKinematicIndex); }
		TManagedArray<TArray<float>>* GetTetherReferenceLength() { return const_cast<TManagedArray<TArray<float>>*>(TetherReferenceLength); }
		TManagedArray<TArray<int32>>* GetSimVertex2DLookup() { return const_cast<TManagedArray<TArray<int32>>*>(SimVertex2DLookup); }
		TManagedArray<TArray<int32>>* GetSeamStitchLookup() { return const_cast<TManagedArray<TArray<int32>>*>(SeamStitchLookup); }
		TManagedArray<float>* GetSimCustomResizingBlend() { return const_cast<TManagedArray<float>*>(SimCustomResizingBlend); }

		//~ Sim Vertices 3D Group -- Sim Accessory Mesh
		TManagedArray<FVector3f>* GetSimAccessoryMeshPosition3D(const FName& AttributeName) { return const_cast<TManagedArray<FVector3f>*>(((FConstClothCollection*)this)->GetSimAccessoryMeshPosition3D(AttributeName)); }
		TManagedArray<FVector3f>* GetSimAccessoryMeshNormal(const FName& AttributeName) const { return const_cast<TManagedArray<FVector3f>*>(((FConstClothCollection*)this)->GetSimAccessoryMeshNormal(AttributeName)); }
		TManagedArray<TArray<int32>>* GetSimAccessoryMeshBoneIndices(const FName& AttributeName) const { return const_cast<TManagedArray<TArray<int32>>*>(((FConstClothCollection*)this)->GetSimAccessoryMeshBoneIndices(AttributeName)); }
		TManagedArray<TArray<float>>* GetSimAccessoryMeshBoneWeights(const FName& AttributeName) const { return const_cast<TManagedArray<TArray<float>>*>(((FConstClothCollection*)this)->GetSimAccessoryMeshBoneWeights(AttributeName)); }

		// Add accessory mesh attribute and return resulting attribute's name. Note: these currently do a linear search through existing attributes to generate a unique name.
		// This does not record the new attribute in the associated SimAccessoryMesh group Attribute 
		FName AddSimAccessoryMeshPosition3D();
		FName AddSimAccessoryMeshNormal();
		FName AddSimAccessoryMeshBoneIndices();
		FName AddSimAccessoryMeshBoneWeights();
		// Remove accessory mesh attribute by name.
		// This does not reset the attribute in the associated SimAccessoryMesh group Attribute 
		void RemoveSimAccessoryMeshPosition3D(const FName& AttributeName);
		void RemoveSimAccessoryMeshNormal(const FName& AttributeName);
		void RemoveSimAccessoryMeshBoneIndices(const FName& AttributeName);
		void RemoveSimAccessoryMeshBoneWeights(const FName& AttributeName);

		//~ Sim Morph Targets Group
		TManagedArray<FString>* GetSimMorphTargetName() { return const_cast<TManagedArray<FString>*>(SimMorphTargetName); }
		TManagedArray<int32>* GetSimMorphTargetVerticesStart() { return const_cast<TManagedArray<int32>*>(SimMorphTargetVerticesStart); }
		TManagedArray<int32>* GetSimMorphTargetVerticesEnd() { return const_cast<TManagedArray<int32>*>(SimMorphTargetVerticesEnd); }

		//~ Sim Morph Targets Vertices Group
		TManagedArray<FVector3f>* GetSimMorphTargetPositionDelta() { return const_cast<TManagedArray<FVector3f>*>(SimMorphTargetPositionDelta); }
		TManagedArray<FVector3f>* GetSimMorphTargetTangentZDelta() { return const_cast<TManagedArray<FVector3f>*>(SimMorphTargetTangentZDelta); }
		TManagedArray<int32>* GetSimMorphTargetSimVertex3DIndex() { return const_cast<TManagedArray<int32>*>(SimMorphTargetSimVertex3DIndex); }

		//~ Render Faces Group
		TManagedArray<FIntVector3>* GetRenderIndices() { return const_cast<TManagedArray<FIntVector3>*>(RenderIndices); }

		//~ Render Vertices Group
		TManagedArray<FVector3f>* GetRenderPosition() { return const_cast<TManagedArray<FVector3f>*>(RenderPosition); }
		TManagedArray<FVector3f>* GetRenderNormal() { return const_cast<TManagedArray<FVector3f>*>(RenderNormal); }
		TManagedArray<FVector3f>* GetRenderTangentU() { return const_cast<TManagedArray<FVector3f>*>(RenderTangentU); }
		TManagedArray<FVector3f>* GetRenderTangentV() { return const_cast<TManagedArray<FVector3f>*>(RenderTangentV); }
		TManagedArray<TArray<FVector2f>>* GetRenderUVs() { return const_cast<TManagedArray<TArray<FVector2f>>*>(RenderUVs); }
		TManagedArray<FLinearColor>* GetRenderColor() { return const_cast<TManagedArray<FLinearColor>*>(RenderColor); }
		TManagedArray<TArray<int32>>* GetRenderBoneIndices() { return const_cast<TManagedArray<TArray<int32>>*>(RenderBoneIndices); }
		TManagedArray<TArray<float>>* GetRenderBoneWeights() { return const_cast<TManagedArray<TArray<float>>*>(RenderBoneWeights); }
		TManagedArray<TArray<FVector4f>>* GetRenderDeformerPositionBaryCoordsAndDist() { return const_cast<TManagedArray<TArray<FVector4f>>*>(RenderDeformerPositionBaryCoordsAndDist); }
		TManagedArray<TArray<FVector4f>>* GetRenderDeformerNormalBaryCoordsAndDist() { return const_cast<TManagedArray<TArray<FVector4f>>*>(RenderDeformerNormalBaryCoordsAndDist); }
		TManagedArray<TArray<FVector4f>>* GetRenderDeformerTangentBaryCoordsAndDist() { return const_cast<TManagedArray<TArray<FVector4f>>*>(RenderDeformerTangentBaryCoordsAndDist); }
		TManagedArray<TArray<FIntVector3>>* GetRenderDeformerSimIndices3D() { return const_cast<TManagedArray<TArray<FIntVector3>>*>(RenderDeformerSimIndices3D); }
		TManagedArray<TArray<float>>* GetRenderDeformerWeight() { return const_cast<TManagedArray<TArray<float>>*>(RenderDeformerWeight); }
		TManagedArray<float>* GetRenderDeformerSkinningBlend() { return const_cast<TManagedArray<float>*>(RenderDeformerSkinningBlend); }
		TManagedArray<float>* GetRenderCustomResizingBlend() { return const_cast<TManagedArray<float>*>(RenderCustomResizingBlend); }

		//~ Group Resizing Group
		TManagedArray<FString>* GetCustomResizingRegionSet() { return const_cast<TManagedArray<FString>*>(CustomResizingRegionSet); }
		TManagedArray<int32>* GetCustomResizingRegionType() { return const_cast<TManagedArray<int32>*>(CustomResizingRegionType); }

		//~ Sim Accessory Meshes Group
		TManagedArray<FName>* GetSimAccessoryMeshName() const { return const_cast<TManagedArray<FName>*>(SimAccessoryMeshName); }
		TManagedArray<FName>* GetSimAccessoryMeshPosition3DAttribute() const { return const_cast<TManagedArray<FName>*>(SimAccessoryMeshPosition3DAttribute); }
		TManagedArray<FName>* GetSimAccessoryMeshNormalAttribute() const { return const_cast<TManagedArray<FName>*>(SimAccessoryMeshNormalAttribute); }
		TManagedArray<FName>* GetSimAccessoryMeshBoneIndicesAttribute() const { return const_cast<TManagedArray<FName>*>(SimAccessoryMeshBoneIndicesAttribute); }
		TManagedArray<FName>* GetSimAccessoryMeshBoneWeightsAttribute() const { return const_cast<TManagedArray<FName>*>(SimAccessoryMeshBoneWeightsAttribute); }

	private:
		TSharedRef<FManagedArrayCollection> GetManagedArrayCollection()
		{
			return ConstCastSharedRef<FManagedArrayCollection>(ManagedArrayCollection);
		}
	};

	template<typename T>
	inline TConstArrayView<T> FConstClothCollection::GetElements(const TManagedArray<T>* ElementArray, const TManagedArray<int32>* StartArray, const TManagedArray<int32>* EndArray, int32 ArrayIndex)
	{
		if (StartArray && EndArray && ElementArray)
		{
			const int32 Start = (*StartArray)[ArrayIndex];
			const int32 End = (*EndArray)[ArrayIndex];
			if (Start != INDEX_NONE && End != INDEX_NONE)
			{
				return TConstArrayView<T>(ElementArray->GetData() + Start, End - Start + 1);
			}
			checkf(Start == End, TEXT("Only one boundary of the range is set to INDEX_NONE, when both should."));
		}
		return TConstArrayView<T>();
	}

	template<typename T>
	inline TArrayView<T> FConstClothCollection::GetElements(TManagedArray<T>* ElementArray, const TManagedArray<int32>* StartArray, const TManagedArray<int32>* EndArray, int32 ArrayIndex)
	{
		if (StartArray && EndArray && ElementArray)
		{
			const int32 Start = (*StartArray)[ArrayIndex];
			const int32 End = (*EndArray)[ArrayIndex];
			if (Start != INDEX_NONE && End != INDEX_NONE)
			{
				return TArrayView<T>(ElementArray->GetData() + Start, End - Start + 1);
			}
			checkf(Start == End, TEXT("Only one boundary of the range is set to INDEX_NONE, when both should."));
		}
		return TArrayView<T>();
	}

	template<typename T>
	inline TConstArrayView<T> FConstClothCollection::GetElements(const TManagedArray<T>* ElementArray)
	{
		if (ElementArray)
		{
			return TConstArrayView<T>(ElementArray->GetData(), ElementArray->Num());
		}
		return TConstArrayView<T>();
	}

	template<typename T>
	inline TArrayView<T> FConstClothCollection::GetElements(TManagedArray<T>* ElementArray)
	{
		if (ElementArray)
		{
			return TArrayView<T>(ElementArray->GetData(), ElementArray->Num());
		}
		return TArrayView<T>();
	}

	template<typename T>
	inline TConstArrayView<T> FConstClothCollection::GetSubElements(
		const TManagedArray<T>* SubElementArray,
		const TManagedArray<int32>* StartArray,
		const TManagedArray<int32>* EndArray,
		const TManagedArray<int32>* StartSubArray,
		const TManagedArray<int32>* EndSubArray,
		int32 ArrayIndex)
	{
		if (StartArray && EndArray && SubElementArray)
		{
			const TTuple<int32, int32> StartEnd = GetSubElementsStartEnd(StartArray, EndArray, StartSubArray, EndSubArray, ArrayIndex);
			const int32 Start = StartEnd.Get<0>();
			const int32 End = StartEnd.Get<1>();
			if (Start != INDEX_NONE && End != INDEX_NONE)
			{
				return TConstArrayView<T>(SubElementArray->GetData() + Start, End - Start + 1);
			}
			checkf(Start == End, TEXT("Only one boundary of the range is set to INDEX_NONE, when both should."));
		}
		return TConstArrayView<T>();
	}

	template<typename T>
	inline TArrayView<T> FConstClothCollection::GetSubElements(
		TManagedArray<T>* SubElementArray,
		const TManagedArray<int32>* StartArray,
		const TManagedArray<int32>* EndArray,
		const TManagedArray<int32>* StartSubArray,
		const TManagedArray<int32>* EndSubArray,
		int32 ArrayIndex)
	{
		if (StartArray && EndArray && SubElementArray)
		{
			const TTuple<int32, int32> StartEnd = GetSubElementsStartEnd(StartArray, EndArray, StartSubArray, EndSubArray, ArrayIndex);
			const int32 Start = StartEnd.Get<0>();
			const int32 End = StartEnd.Get<1>();
			if (Start != INDEX_NONE && End != INDEX_NONE)
			{
				return TArrayView<T>(SubElementArray->GetData() + Start, End - Start + 1);
			}
			checkf(Start == End, TEXT("Only one boundary of the range is set to INDEX_NONE, when both should."));
		}
		return TArrayView<T>();
	}

	template<typename T>
	inline void FConstClothCollection::CopyArrayViewData(const TArrayView<T>& To, const TConstArrayView<T>& From)
	{
		check(To.Num() == From.Num());
		for (int32 Index = 0; Index < To.Num(); ++Index)
		{
			To[Index] = From[Index];
		}
	}

	template<typename T UE_REQUIRES_DEFINITION(!std::is_array_v<T>)>
	inline void FConstClothCollection::CopyArrayViewDataAndApplyOffset(const TArrayView<T>& To, const TConstArrayView<T>& From, const T Offset)
	{
		check(To.Num() == From.Num());
		for (int32 Index = 0; Index < To.Num(); ++Index)
		{
			To[Index] = From[Index] + Offset;
		}
	}

	template<typename T>
	inline void FConstClothCollection::CopyArrayViewDataAndApplyOffset(const TArrayView<TArray<T>>& To, const TConstArrayView<TArray<T>>& From, const T Offset)
	{
		check(To.Num() == From.Num());
		for (int32 Index = 0; Index < To.Num(); ++Index)
		{
			To[Index] = From[Index];
			for (T& Value : To[Index])
			{
				Value += Offset;
			}
		}
	}

	template<typename T>
	inline uint32 FConstClothCollection::GetElementsTypeHash(const TManagedArray<T>* ElementArray)
	{
		if (ElementArray)
		{
			return GetTypeHash(*ElementArray);
		}
		return GetTypeHash(ElementArray);
	}
}  // End namespace UE::Chaos::ClothAsset
