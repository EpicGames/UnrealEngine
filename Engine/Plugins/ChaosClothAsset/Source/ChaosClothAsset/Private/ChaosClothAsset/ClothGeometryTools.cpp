// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ClothGeometryTools.h"
#include "ChaosClothAsset/ClothCollectionGroup.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "ChaosClothAsset/CollectionClothSelectionFacade.h"
#include "Chaos/CollectionEmbeddedSpringConstraintFacade.h"
#include "Chaos/CollectionPropertyFacade.h"
#include "Containers/Queue.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/DynamicVertexSkinWeightsAttribute.h"
#include "DynamicMesh/NonManifoldMappingSupport.h"
#include "Math/OrientedBox.h"
#include "Math/Vector.h"
#include "Util/IndexUtil.h"
#include "Utils/ClothingMeshUtils.h"
#include "Algo/RemoveIf.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "RenderMath.h"

namespace UE::Chaos::ClothAsset
{
	namespace Private::SimMeshBuilder
	{
		// Triangle islands to become patterns, although in this case all the seams are internal (same pattern)
		struct FIsland
		{
			TArray<int32> Indices;  // 3x number of triangles
			TArray<FVector2f> Positions2D;
			TArray<FVector3f> Positions3D;  // Same size as Positions
			TArray<FVector3f> Normals; // Empty or same size as Positions
			TArray<int32> PositionToSourceIndex; // Same size as Positions. Index in the original welded position array
		};

		enum class EIntersectCirclesResult
		{
			SingleIntersect,
			DoubleIntersect,
			Coincident,
			Separate,
			Contained
		};

		static EIntersectCirclesResult IntersectCircles(const FVector2f& C0, float R0, const FVector2f& C1, float R1, FVector2f& OutI0, FVector2f& OutI1)
		{
			const FVector2f C0C1 = C0 - C1;
			const float D = C0C1.Length();
			if (D < SMALL_NUMBER)
			{
				return EIntersectCirclesResult::Coincident;
			}
			else if (D > R0 + R1)
			{
				return EIntersectCirclesResult::Separate;
			}
			else if (D < FMath::Abs(R0 - R1))
			{
				return EIntersectCirclesResult::Contained;
			}
			const float SquareR0 = R0 * R0;
			const float SquareR1 = R1 * R1;
			const float SquareD = D * D;
			const float A = (SquareD - SquareR1 + SquareR0) / (2.f * D);

			OutI0 = OutI1 = C0 + A * (C1 - C0) / D;

			if (FMath::Abs(A - R0) < SMALL_NUMBER)
			{
				return EIntersectCirclesResult::SingleIntersect;
			}

			const float SquareA = A * A;
			const float H = FMath::Sqrt(SquareR0 - SquareA);

			const FVector2f N(C0C1.Y, -C0C1.X);

			OutI0 += N * H / D;
			OutI1 -= N * H / D;

			return EIntersectCirclesResult::DoubleIntersect;
		}
		;
		static FIntVector2 MakeSortedIntVector2(int32 Index0, int32 Index1)
		{
			return Index0 < Index1 ? FIntVector2(Index0, Index1) : FIntVector2(Index1, Index0);
		}

		static void UnwrapDynamicMesh(const UE::Geometry::FDynamicMesh3& DynamicMesh, bool bImportNormals, TArray<FIsland>& OutIslands)
		{
			using namespace UE::Geometry;
			const UE::Geometry::FDynamicMeshAttributeSet* const AttributeSet = DynamicMesh.Attributes();
			const UE::Geometry::FDynamicMeshNormalOverlay* const NormalOverlay = bImportNormals && AttributeSet ? AttributeSet->PrimaryNormals() : nullptr;

			OutIslands.Reset();
			constexpr float SquaredWeldingDistance = FMath::Square(0.01f);  // 0.1 mm

			// Build pattern islands. 
			const int32 NumTriangles = DynamicMesh.TriangleCount();
			TSet<int32> VisitedTriangles;
			VisitedTriangles.Reserve(NumTriangles);


			for (int32 SeedTriangle : DynamicMesh.TriangleIndicesItr())
			{
				if (VisitedTriangles.Contains(SeedTriangle))
				{
					continue;
				}
				const FIndex3i TriangleIndices = DynamicMesh.GetTriangle(SeedTriangle);
				const FIndex3i TriangleNormalElements = NormalOverlay ? NormalOverlay->GetTriangle(SeedTriangle) : FIndex3i::Invalid();

				const int32 SeedIndex0 = TriangleIndices[0];
				const int32 SeedIndex1 = TriangleIndices[1];

				const FVector3f Position0(DynamicMesh.GetVertex(SeedIndex0));
				const FVector3f Position1(DynamicMesh.GetVertex(SeedIndex1));
				const float Position01DistSq = FVector3f::DistSquared(Position0, Position1);

				if (Position01DistSq <= SquaredWeldingDistance)
				{
					continue;  // A degenerated triangle edge is not a good start
				}

				// Setup first visitor from seed, and add the first two points
				FIsland& Island = OutIslands.AddDefaulted_GetRef();

				Island.Positions3D.Add(Position0);
				Island.Positions3D.Add(Position1);
				if (NormalOverlay)
				{
					Island.Normals.Add(NormalOverlay->GetElement(TriangleNormalElements[0]));
					Island.Normals.Add(NormalOverlay->GetElement(TriangleNormalElements[1]));
				}
				Island.PositionToSourceIndex.Add(SeedIndex0);
				Island.PositionToSourceIndex.Add(SeedIndex1);

				const int32 SeedIndex2D0 = Island.Positions2D.Add(FVector2f::ZeroVector);
				const int32 SeedIndex2D1 = Island.Positions2D.Add(FVector2f(FMath::Sqrt(Position01DistSq), 0.f));

				struct FVisitor
				{
					int32 Triangle;
					FIndex2i OldEdge;
					FIndex2i NewEdge;
					FIndex2i NormalIndices;
					int32 CrossEdgePoint;  // Keep the opposite point to orientate degenerate cases
				} Visitor =
				{
					SeedTriangle,
					FIndex2i(SeedIndex0, SeedIndex1),
					FIndex2i(SeedIndex2D0, SeedIndex2D1),
					FIndex2i(TriangleNormalElements[0], TriangleNormalElements[1]),
					INDEX_NONE
				};

				VisitedTriangles.Add(SeedTriangle);

				TQueue<FVisitor> Visitors;
				do
				{
					const int32 Triangle = Visitor.Triangle;
					const int32 CrossEdgePoint = Visitor.CrossEdgePoint;
					const int32 OldIndex0 = Visitor.OldEdge.A;
					const int32 OldIndex1 = Visitor.OldEdge.B;
					const int32 NewIndex0 = Visitor.NewEdge.A;
					const int32 NewIndex1 = Visitor.NewEdge.B;
					const int32 NormalIndex0 = Visitor.NormalIndices.A;
					const int32 NormalIndex1 = Visitor.NormalIndices.B;

					// Find opposite index from this triangle edge

					const int32 OldIndex2 = IndexUtil::FindTriOtherVtxUnsafe(OldIndex0, OldIndex1, DynamicMesh.GetTriangle(Triangle));
					const int32 NormalIndex2 = NormalOverlay ? IndexUtil::FindTriOtherVtxUnsafe(NormalIndex0, NormalIndex1, NormalOverlay->GetTriangle(Triangle)) : INDEX_NONE;

					// Find the 2D intersection of the two connecting adjacent edges using the 3D reference length
					const FVector3f P0(DynamicMesh.GetVertexRef(OldIndex0));
					const FVector3f P1(DynamicMesh.GetVertexRef(OldIndex1));
					const FVector3f P2(DynamicMesh.GetVertexRef(OldIndex2));

					const float R0 = FVector3f::Dist(P0, P2);
					const float R1 = FVector3f::Dist(P1, P2);
					const FVector2f& C0 = Island.Positions2D[NewIndex0];
					const FVector2f& C1 = Island.Positions2D[NewIndex1];

					FVector2f I0, I1;
					const EIntersectCirclesResult IntersectCirclesResult = IntersectCircles(C0, R0, C1, R1, I0, I1);

					FVector2f C2;
					switch (IntersectCirclesResult)
					{
					case EIntersectCirclesResult::SingleIntersect:
						C2 = I0;  // Degenerated C2 is on (C0C1)
						break;
					case EIntersectCirclesResult::DoubleIntersect:
						C2 = (FVector2f::CrossProduct(C0 - C1, C0 - I0) > 0) ? I0 : I1;  // Keep correct winding order
						break;
					case EIntersectCirclesResult::Coincident:
						check(CrossEdgePoint != INDEX_NONE); // We can't start on a degenerated triangle
						C2 = C0 - (Island.Positions2D[CrossEdgePoint] - C0).GetSafeNormal() * R0;  // Degenerated C0 == C1, choose C2 on the opposite of the visitor opposite point
						break;
					case EIntersectCirclesResult::Separate: [[fallthrough]];
					case EIntersectCirclesResult::Contained:
						C2 = C0 - (C1 - C0).GetSafeNormal() * R0;  // Degenerated + some tolerance, C2 is on (C0C1)
						break;
					}

					// Add the new position found for the opposite point
					int32 NewIndex2 = INDEX_NONE;
					for (int32 UsedIndex = 0; UsedIndex < Island.Positions2D.Num(); ++UsedIndex)
					{
						if (Island.PositionToSourceIndex[UsedIndex] == OldIndex2 &&
							FVector2f::DistSquared(Island.Positions2D[UsedIndex], C2) <= SquaredWeldingDistance)
						{
							NewIndex2 = UsedIndex;  // Both Rest and 2D positions match, reuse this index
							break;
						}
					}

					if (NewIndex2 == INDEX_NONE)
					{
						NewIndex2 = Island.Positions2D.Add(C2);
						Island.Positions3D.Add(P2);
						if (NormalOverlay)
						{
							Island.Normals.Add(NormalOverlay->GetElement(NormalIndex2));
						}
						Island.PositionToSourceIndex.Add(OldIndex2);
					}

					// Add triangle to list of indices, unless it is degenerated to a segment
					if (NewIndex0 != NewIndex1 && NewIndex1 != NewIndex2 && NewIndex2 != NewIndex0)
					{
						Island.Indices.Add(NewIndex0);
						Island.Indices.Add(NewIndex1);
						Island.Indices.Add(NewIndex2);
					}

					// Add neighbor triangles to the queue
					const FIndex2i OldEdgeList[3] =
					{
						FIndex2i(OldIndex1, OldIndex0),  // Reversed as to keep the correct winding order
						FIndex2i(OldIndex2, OldIndex1),
						FIndex2i(OldIndex0, OldIndex2)
					};
					const FIndex3i NewEdgeList[3] =
					{
						FIndex3i(NewIndex1, NewIndex0, NewIndex2),  // Adds opposite point index
						FIndex3i(NewIndex2, NewIndex1, NewIndex0),
						FIndex3i(NewIndex0, NewIndex2, NewIndex1)
					};
					const FIndex2i NormalEdgeList[3] =
					{
						FIndex2i(NormalIndex1, NormalIndex0),
						FIndex2i(NormalIndex2, NormalIndex1),
						FIndex2i(NormalIndex0, NormalIndex2)
					};
					for (int32 Edge = 0; Edge < 3; ++Edge)
					{
						const int32 EdgeIndex0 = OldEdgeList[Edge].A;
						const int32 EdgeIndex1 = OldEdgeList[Edge].B;

						const FIndex2i EdgeT = DynamicMesh.GetEdgeT(DynamicMesh.FindEdgeFromTri(EdgeIndex0, EdgeIndex1, Triangle));
						const int32 NeighborTriangle = EdgeT.OtherElement(Triangle);
						if (NeighborTriangle != IndexConstants::InvalidID)
						{
							if (!VisitedTriangles.Contains(NeighborTriangle))
							{
								// Mark neighboring triangle as visited
								VisitedTriangles.Add(NeighborTriangle);

								// Enqueue next triangle
								Visitors.Enqueue(FVisitor
									{
										NeighborTriangle,
										OldEdgeList[Edge],
										FIndex2i(NewEdgeList[Edge].A, NewEdgeList[Edge].B),
										NormalEdgeList[Edge],
										NewEdgeList[Edge].C,  // Pass the cross edge 2D opposite point to help define orientation of any degenerated triangles
									});
							}

						}
					}
				} while (Visitors.Dequeue(Visitor));
			}
		}

		static void BuildIslandsFromDynamicMeshUVs(const UE::Geometry::FDynamicMeshUVOverlay& UVOverlay, const FVector2f& UVScale, bool bImportNormals, TArray<FIsland>& OutIslands)
		{
			using namespace UE::Geometry;

			const FDynamicMesh3* const DynamicMesh = UVOverlay.GetParentMesh();
			check(DynamicMesh);
			const UE::Geometry::FDynamicMeshAttributeSet* const AttributeSet = DynamicMesh->Attributes();
			const UE::Geometry::FDynamicMeshNormalOverlay* const NormalOverlay = bImportNormals && AttributeSet ? AttributeSet->PrimaryNormals() : nullptr;

			OutIslands.Reset();

			// Build pattern islands. 
			const int32 NumTriangles = DynamicMesh->TriangleCount();
			TSet<int32> VisitedTriangles;
			VisitedTriangles.Reserve(NumTriangles);

			// This is reused for each island, but only allocate once.
			TArray<int32> SourceElementIndexToNewIndex;

			for (int32 SeedTriangle : DynamicMesh->TriangleIndicesItr())
			{
				if (VisitedTriangles.Contains(SeedTriangle))
				{
					continue;
				}

				// Setup first visitor from seed
				FIsland& Island = OutIslands.AddDefaulted_GetRef();
				SourceElementIndexToNewIndex.Init(INDEX_NONE, UVOverlay.MaxElementID());

				struct FVisitor
				{
					int32 Triangle;
				} Visitor =
				{
					SeedTriangle
				};

				VisitedTriangles.Add(SeedTriangle);

				TQueue<FVisitor> Visitors;
				do
				{
					const int32 Triangle = Visitor.Triangle;
					const FIndex3i TriangleIndices = DynamicMesh->GetTriangle(Triangle);
					const FIndex3i TriangleUVElements = UVOverlay.GetTriangle(Triangle);
					const FIndex3i TriangleNormalElements = NormalOverlay ? NormalOverlay->GetTriangle(Triangle) : FIndex3i::Invalid();

					auto GetOrAddNewIndex = [&UVOverlay, &Island, &SourceElementIndexToNewIndex, &DynamicMesh, &UVScale, NormalOverlay](int32 ElementId, int32 VertexId, int32 NormalId)
					{
						int32& NewIndex = SourceElementIndexToNewIndex[ElementId];
						if (NewIndex == INDEX_NONE)
						{
							NewIndex = Island.Positions3D.Add(FVector3f(DynamicMesh->GetVertexRef(VertexId)));
							Island.Positions2D.Add((FVector2f(1.f) - UVOverlay.GetElement(ElementId)) * UVScale);  // The static mesh import uses 1 - UV for some reason
							if (NormalOverlay)
							{
								Island.Normals.Add(NormalOverlay->GetElement(NormalId));
							}
							Island.PositionToSourceIndex.Add(VertexId);
						}
						return NewIndex;
					};

					const int32 NewIndex0 = GetOrAddNewIndex(TriangleUVElements[0], TriangleIndices[0], TriangleNormalElements[0]);
					const int32 NewIndex1 = GetOrAddNewIndex(TriangleUVElements[1], TriangleIndices[1], TriangleNormalElements[1]);
					const int32 NewIndex2 = GetOrAddNewIndex(TriangleUVElements[2], TriangleIndices[2], TriangleNormalElements[2]);
					Island.Indices.Add(NewIndex0);
					Island.Indices.Add(NewIndex1);
					Island.Indices.Add(NewIndex2);

					TArray<int32> NeighborTriangles;
					for (int32 LocalVertexId = 0; LocalVertexId < 3; ++LocalVertexId)
					{
						NeighborTriangles.Reset();
						UVOverlay.GetElementTriangles(TriangleUVElements[LocalVertexId], NeighborTriangles);
						for (const int32 NeighborTriangle : NeighborTriangles)
						{
							if (!VisitedTriangles.Contains(NeighborTriangle))
							{
								// Mark neighboring triangle as visited
								VisitedTriangles.Add(NeighborTriangle);

								// Enqueue next triangle
								Visitors.Enqueue(FVisitor({ NeighborTriangle }));
							}
						}
					}
				} while (Visitors.Dequeue(Visitor));
			}
		}

		struct FSeam
		{
			TSet<FIntVector2> Stitches;
			FIntVector2 Patterns;
		};

		// Stitch together any vertices that were split, either via DynamicMesh NonManifoldMapping or UV Unwrap
		static void BuildSeams(const TArray<FIsland>& Islands, const UE::Geometry::FDynamicMesh3& DynamicMesh, int32 PatternIndexOffset, TArray<FSeam>& OutSeams)
		{
			OutSeams.Reset();

			const UE::Geometry::FNonManifoldMappingSupport NonManifoldMapping(DynamicMesh);

			TArray<TMap<int32, TArray<int32>>> IslandSourceIndexToPositions;
			IslandSourceIndexToPositions.SetNum(Islands.Num());

			for (int32 IslandIndex = 0; IslandIndex < Islands.Num(); ++IslandIndex)
			{
				const FIsland& Island = Islands[IslandIndex];
				TMap<int32, TArray<int32>>& SourceIndexToPositions = IslandSourceIndexToPositions[IslandIndex];

				// Build reverse lookup to PositionToSourceIndex
				SourceIndexToPositions.Reserve(Island.PositionToSourceIndex.Num());
				for (int32 PositionIndex = 0; PositionIndex < Island.PositionToSourceIndex.Num(); ++PositionIndex)
				{
					const int32 SourceIndex = NonManifoldMapping.GetOriginalNonManifoldVertexID(Island.PositionToSourceIndex[PositionIndex]);
					SourceIndexToPositions.FindOrAdd(SourceIndex).Add(PositionIndex);
				}

				// Find all internal seams
				FSeam InternalSeam;
				InternalSeam.Patterns = FIntVector2(IslandIndex + PatternIndexOffset);
				for (const TPair<int32, TArray<int32>>& Source : SourceIndexToPositions)
				{
					for (int32 FirstSourceArrayIdx = 0; FirstSourceArrayIdx < Source.Value.Num() - 1; ++FirstSourceArrayIdx)
					{
						for (int32 SecondSourceArrayIdx = FirstSourceArrayIdx + 1; SecondSourceArrayIdx < Source.Value.Num(); ++SecondSourceArrayIdx)
						{
							InternalSeam.Stitches.Emplace(MakeSortedIntVector2(Source.Value[FirstSourceArrayIdx], Source.Value[SecondSourceArrayIdx]));
						}
					}
				}
				if (InternalSeam.Stitches.Num())
				{
					OutSeams.Emplace(MoveTemp(InternalSeam));
				}

				for (int32 OtherIslandIndex = 0; OtherIslandIndex < IslandIndex; ++OtherIslandIndex)
				{
					// Find all seams between the two islands
					const TMap<int32, TArray<int32>>& OtherSourceIndexToPositions = IslandSourceIndexToPositions[OtherIslandIndex];

					FSeam Seam;
					Seam.Patterns = FIntVector2(OtherIslandIndex + PatternIndexOffset, IslandIndex + PatternIndexOffset);
					for (const TPair<int32, TArray<int32>>& FirstSource : SourceIndexToPositions)
					{
						if (const TArray<int32>* OtherSource = OtherSourceIndexToPositions.Find(FirstSource.Key))
						{
							for (const int32 FirstSourceVert : FirstSource.Value)
							{
								for (const int32 OtherSourceVert : *OtherSource)
								{
									Seam.Stitches.Emplace(FIntVector2(OtherSourceVert, FirstSourceVert));
								}
							}
						}
					}
					if (Seam.Stitches.Num())
					{
						OutSeams.Emplace(MoveTemp(Seam));
					}
				}
			}
		}

		static void AddSeam(FCollectionClothFacade& Cloth, const FSeam& Seam)
		{
			const int32 Pattern0Start = Cloth.GetSimPattern(Seam.Patterns[0]).GetSimVertices2DOffset();
			const int32 Pattern1Start = Cloth.GetSimPattern(Seam.Patterns[1]).GetSimVertices2DOffset();

			FCollectionClothSeamFacade SeamFacade = Cloth.AddGetSeam();
			TArray<FIntVector2> Stitches;
			Stitches.Reset(Seam.Stitches.Num());
			for (const FIntVector2& Stitch : Seam.Stitches)
			{
				Stitches.Emplace(Stitch[0] + Pattern0Start, Stitch[1] + Pattern1Start);
			}
			SeamFacade.Initialize(Stitches);
		}
	}// namespace Private::SimMeshBuilder

	// TODO: Move these functions to the cloth facade?
	bool FClothGeometryTools::HasSimMesh(const TSharedRef<const FManagedArrayCollection>& ClothCollection)
	{
		FCollectionClothConstFacade ClothFacade(ClothCollection);
		return ClothFacade.GetNumSimVertices2D() > 0 && ClothFacade.GetNumSimVertices3D() && ClothFacade.GetNumSimFaces() > 0;
	}

	bool FClothGeometryTools::HasRenderMesh(const TSharedRef<const FManagedArrayCollection>& ClothCollection)
	{
		FCollectionClothConstFacade ClothFacade(ClothCollection);
		return ClothFacade.GetNumRenderVertices() > 0 && ClothFacade.GetNumRenderFaces() > 0;
	}

	void FClothGeometryTools::DeleteRenderMesh(const TSharedRef<FManagedArrayCollection>& ClothCollection)
	{
		FCollectionClothFacade ClothFacade(ClothCollection);
		ClothFacade.SetNumRenderPatterns(0);
	}

	void FClothGeometryTools::DeleteSimMesh(const TSharedRef<FManagedArrayCollection>& ClothCollection)
	{
		::Chaos::Softs::FEmbeddedSpringFacade SpringFacade(ClothCollection.Get(), ClothCollectionGroup::SimVertices3D);
		SpringFacade.SetNumSpringConstraints(0);

		FCollectionClothFacade ClothFacade(ClothCollection);
		ClothFacade.SetNumSimPatterns(0);
		ClothFacade.RemoveAllSimVertices3D();
		ClothFacade.SetNumSeams(0);
	}

	void FClothGeometryTools::DeleteTethers(const TSharedRef<FManagedArrayCollection>& ClothCollection)
	{
		FCollectionClothFacade ClothFacade(ClothCollection);
		for (TArray<int32>& KinematicIndex : ClothFacade.GetTetherKinematicIndex())
		{
			KinematicIndex.Reset();
		}
		for (TArray<float>& ReferenceLength : ClothFacade.GetTetherReferenceLength())
		{
			ReferenceLength.Reset();
		}
	}

	void FClothGeometryTools::DeleteSelections(const TSharedRef<FManagedArrayCollection>& ClothCollection, const FName Group)
	{
		FCollectionClothSelectionFacade ClothSelectionFacade(ClothCollection);
		const TArray<FName> Names = ClothSelectionFacade.GetNames();

		for (const FName Name : Names)
		{
			if (Group == NAME_None || ClothSelectionFacade.GetSelectionGroup(Name) == Group)
			{
				ClothSelectionFacade.RemoveSelectionSet(Name);
			}
		}
	}

	void FClothGeometryTools::CopySimMeshToRenderMesh(const TSharedRef<FManagedArrayCollection>& ClothCollection, const FSoftObjectPath& RenderMaterialPathName, bool bSingleRenderPattern)
	{
		FCollectionClothFacade ClothFacade(ClothCollection);

		// Use 2D topology (unwelded mesh)

		// Render pattern data
		const int32 NumRenderPatterns = bSingleRenderPattern ? 1 : ClothFacade.GetNumSimPatterns();
		ClothFacade.SetNumRenderPatterns(NumRenderPatterns);
		const int32 TotalNumFaces = ClothFacade.GetNumSimFaces();
		const int32 TotalNumVertices = ClothFacade.GetNumSimVertices2D();
		for (int32 RenderPatternIndex = 0; RenderPatternIndex < NumRenderPatterns; ++RenderPatternIndex)
		{
			FCollectionClothRenderPatternFacade RenderPattern = ClothFacade.GetRenderPattern(RenderPatternIndex);
			RenderPattern.SetRenderMaterialSoftObjectPathName(RenderMaterialPathName);
			RenderPattern.SetNumRenderVertices(bSingleRenderPattern ? TotalNumVertices : ClothFacade.GetSimPattern(RenderPatternIndex).GetNumSimVertices2D());
			RenderPattern.SetNumRenderFaces(bSingleRenderPattern ? TotalNumFaces : ClothFacade.GetSimPattern(RenderPatternIndex).GetNumSimFaces());
		}

		// Calculate UVs scale and zero out tangents
		FVector2f MinPosition(TNumericLimits<float>::Max());
		FVector2f MaxPosition(TNumericLimits<float>::Lowest());

		const TConstArrayView<FVector2f> SimPosition2D = ClothFacade.GetSimPosition2D();
		const TConstArrayView<FVector3f> SimNormal = ClothFacade.GetSimNormal();
		const TArrayView<FVector3f> RenderTangentU = ClothFacade.GetRenderTangentU();
		const TArrayView<FVector3f> RenderTangentV = ClothFacade.GetRenderTangentV();

		for (int32 VertexIndex = 0; VertexIndex < TotalNumVertices; ++VertexIndex)
		{
			MinPosition = FVector2f::Min(MinPosition, SimPosition2D[VertexIndex]);
			MaxPosition = FVector2f::Max(MaxPosition, SimPosition2D[VertexIndex]);

			RenderTangentU[VertexIndex] = FVector3f::ZeroVector;
			RenderTangentV[VertexIndex] = FVector3f::ZeroVector;
		}
		const FVector2f UVScale = MaxPosition - MinPosition;
		const FVector2f UVInvScale(
			UVScale.X < SMALL_NUMBER ? 0.f : 1.f / UVScale.X,
			UVScale.Y < SMALL_NUMBER ? 0.f : 1.f / UVScale.Y);

		// Face group (and calculating render tangents)
		const TConstArrayView<FVector3f> SimPosition3D = ClothFacade.GetSimPosition3D();
		const TConstArrayView<int32> SimVertex3DLookup = ClothFacade.GetSimVertex3DLookup();
		const TConstArrayView<FIntVector3> SimIndices = ClothFacade.GetSimIndices2D();
		const TArrayView<FIntVector3> RenderIndices = ClothFacade.GetRenderIndices();
		for (int32 FaceIndex = 0; FaceIndex < TotalNumFaces; ++FaceIndex)
		{
			const FIntVector3& Face = SimIndices[FaceIndex];
			RenderIndices[FaceIndex] = Face;

			const FVector3f Pos01 = SimPosition3D[SimVertex3DLookup[Face[1]]] - SimPosition3D[SimVertex3DLookup[Face[0]]];
			const FVector3f Pos02 = SimPosition3D[SimVertex3DLookup[Face[2]]] - SimPosition3D[SimVertex3DLookup[Face[0]]];
			const FVector2f UV01 = SimPosition2D[Face[1]] - SimPosition2D[Face[0]];
			const FVector2f UV02 = SimPosition2D[Face[2]] - SimPosition2D[Face[0]];

			const float Denom = UV01.X * UV02.Y - UV01.Y * UV02.X;
			const float InvDenom = (FMath::Abs(Denom) < SMALL_NUMBER) ? 0.f : 1.f / Denom;
			const FVector3f TangentU = (Pos01 * UV02.Y - Pos02 * UV01.Y) * InvDenom;
			const FVector3f TangentV = (Pos02 * UV01.X - Pos01 * UV02.X) * InvDenom;

			for (int32 PointIndex = 0; PointIndex < 3; ++PointIndex)
			{
				RenderTangentU[Face[PointIndex]] += TangentU;
				RenderTangentV[Face[PointIndex]] += TangentV;
			}
		}

		// Vertex group
		const TArrayView<FVector3f> RenderPosition = ClothFacade.GetRenderPosition();
		const TArrayView<FVector3f> RenderNormal = ClothFacade.GetRenderNormal();
		const TArrayView<TArray<FVector2f>> RenderUVs = ClothFacade.GetRenderUVs();
		const TArrayView<FLinearColor> RenderColor = ClothFacade.GetRenderColor();
		const TArrayView<TArray<int32>> RenderBoneIndices = ClothFacade.GetRenderBoneIndices();
		const TArrayView<TArray<float>> RenderBoneWeights = ClothFacade.GetRenderBoneWeights();

		// NOTE: This sim data is stored on welded vertices.
		const TArrayView<TArray<int32>> SimBoneIndices = ClothFacade.GetSimBoneIndices();
		const TArrayView<TArray<float>> SimBoneWeights = ClothFacade.GetSimBoneWeights();

		for (int32 VertexIndex = 0; VertexIndex < TotalNumVertices; ++VertexIndex)
		{
			const int32 VertexIndex3D = SimVertex3DLookup[VertexIndex];

			RenderPosition[VertexIndex] = SimPosition3D[VertexIndex3D];
			RenderNormal[VertexIndex] = -SimNormal[VertexIndex3D];  // Simulation normals use reverse normals
			RenderUVs[VertexIndex] = { (SimPosition2D[VertexIndex] - MinPosition) * UVInvScale };
			RenderUVs[VertexIndex][0].Y = 1.f - RenderUVs[VertexIndex][0].Y;  // Reverse Y axis 
			RenderColor[VertexIndex] = FLinearColor::White;
			RenderTangentU[VertexIndex].Normalize();
			RenderTangentV[VertexIndex].Normalize();
			RenderBoneIndices[VertexIndex] = SimBoneIndices[VertexIndex3D];
			RenderBoneWeights[VertexIndex] = SimBoneWeights[VertexIndex3D];
		}

		// Bind to root bone
		constexpr bool bBindSimMesh = false;
		constexpr bool bBindRenderMesh = true;
		BindMeshToRootBone(ClothCollection, bBindSimMesh, bBindRenderMesh);
	}

	void FClothGeometryTools::ReverseMesh(
		const TSharedRef<FManagedArrayCollection>& ClothCollection,
		bool bReverseSimMeshNormals,
		bool bReverseSimMeshWindingOrder,
		bool bReverseRenderMeshNormals,
		bool bReverseRenderMeshWindingOrder,
		const TArray<int32>& SimPatternSelection,
		const TArray<int32>& RenderPatternSelection)
	{
		auto ReverseSimNormals = [](const TArrayView<FVector3f>& SimNormal)
			{
				for (int32 VertexIndex = 0; VertexIndex < SimNormal.Num(); ++VertexIndex)
				{
					SimNormal[VertexIndex] = -SimNormal[VertexIndex];
				}
			};
		auto ReverseRenderNormals = [](const TArrayView<FVector3f>& RenderNormal, const TArrayView<FVector3f>& RenderTangentU)
			{
				check(RenderNormal.Num() == RenderTangentU.Num())
				for (int32 VertexIndex = 0; VertexIndex < RenderNormal.Num(); ++VertexIndex)
				{
					RenderNormal[VertexIndex] = -RenderNormal[VertexIndex];      // Equivalent of rotating the normal basis
					RenderTangentU[VertexIndex] = -RenderTangentU[VertexIndex];  // around tangent V
				}
			};
		auto ReverseWindingOrder = [](const TArrayView<FIntVector3>& Indices)
			{
				for (int32 FaceIndex = 0; FaceIndex < Indices.Num(); ++FaceIndex)
				{
					Swap(Indices[FaceIndex][1], Indices[FaceIndex][2]);
				}
			};

		FCollectionClothFacade ClothFacade(ClothCollection);

		if (SimPatternSelection.IsEmpty())
		{
			if (bReverseSimMeshNormals)
			{
				ReverseSimNormals(ClothFacade.GetSimNormal());
			}

			if (bReverseSimMeshWindingOrder)
			{
				ReverseWindingOrder(ClothFacade.GetSimIndices2D());
				ReverseWindingOrder(ClothFacade.GetSimIndices3D());
			}
		}
		else
		{
			// Sim Normals live on welded vertices. We don't want to double flip normals that live in multiple patterns
			TBitArray AlreadyFlippedNormal;
			if (bReverseSimMeshNormals)
			{
				AlreadyFlippedNormal.Init(false, ClothFacade.GetNumSimVertices3D());
			}
			const TArrayView<FVector3f> AllSimNormals = ClothFacade.GetSimNormal();
			for (int32 PatternIndex = 0; PatternIndex < ClothFacade.GetNumSimPatterns(); ++PatternIndex)
			{
				if (SimPatternSelection.Find(PatternIndex) != INDEX_NONE)
				{
					FCollectionClothSimPatternFacade ClothPatternFacade = ClothFacade.GetSimPattern(PatternIndex);

					if (bReverseSimMeshNormals)
					{
						const TConstArrayView<int32> SimVertex3DLookup = static_cast<FCollectionClothSimPatternConstFacade&>(ClothPatternFacade).GetSimVertex3DLookup();
						for (int32 VertexIndex3D : SimVertex3DLookup)
						{
							if (!AlreadyFlippedNormal[VertexIndex3D])
							{
								AllSimNormals[VertexIndex3D] = -AllSimNormals[VertexIndex3D];
								AlreadyFlippedNormal[VertexIndex3D] = true;
							}
						}
					}

					if (bReverseSimMeshWindingOrder)
					{
						ReverseWindingOrder(ClothPatternFacade.GetSimIndices2D());
						ReverseWindingOrder(ClothPatternFacade.GetSimIndices3D());
					}
				}
			}
		}

		if (RenderPatternSelection.IsEmpty())
		{
			if (bReverseRenderMeshNormals)
			{
				ReverseRenderNormals(ClothFacade.GetRenderNormal(), ClothFacade.GetRenderTangentU());
			}
			if (bReverseRenderMeshWindingOrder)
			{
				ReverseWindingOrder(ClothFacade.GetRenderIndices());
			}
		}
		else
		{
			for (int32 PatternIndex = 0; PatternIndex < ClothFacade.GetNumRenderPatterns(); ++PatternIndex)
			{
				if (RenderPatternSelection.Find(PatternIndex) != INDEX_NONE)
				{
					FCollectionClothRenderPatternFacade ClothPatternFacade = ClothFacade.GetRenderPattern(PatternIndex);
					if (bReverseRenderMeshNormals)
					{
						ReverseRenderNormals(ClothPatternFacade.GetRenderNormal(), ClothPatternFacade.GetRenderTangentU());
					}
					if (bReverseRenderMeshWindingOrder)
					{
						ReverseWindingOrder(ClothPatternFacade.GetRenderIndices());
					}
				}
			}
		}
	}

	void FClothGeometryTools::RecalculateRenderMeshNormals(const TSharedRef<FManagedArrayCollection>& ClothCollection)
	{
		FCollectionClothFacade ClothFacade(ClothCollection);

		const TConstArrayView<FIntVector3> FaceIndices = ClothFacade.GetRenderIndices();
		const TConstArrayView<FVector3f> Positions = ClothFacade.GetRenderPosition();
		const TConstArrayView<TArray<FVector2f>> UVs = ClothFacade.GetRenderUVs();
		const TArrayView<FVector3f> Normals = ClothFacade.GetRenderNormal();
		const TArrayView<FVector3f> TangentUs = ClothFacade.GetRenderTangentU();
		const TArrayView<FVector3f> TangentVs = ClothFacade.GetRenderTangentV();

		check(Positions.Num() == UVs.Num());
		check(Positions.Num() == Normals.Num());
		check(Positions.Num() == TangentUs.Num());
		check(Positions.Num() == TangentVs.Num());

		// Record BasisDeterminantSigns to preserve them when calculating TangentVs.
		TArray<float> BasisDeterminantSigns;
		BasisDeterminantSigns.SetNumUninitialized(Normals.Num());
		for (int32 VertexIndex = 0; VertexIndex < Normals.Num(); ++VertexIndex)
		{
			BasisDeterminantSigns[VertexIndex] = GetBasisDeterminantSign(FVector3d(TangentUs[VertexIndex]), FVector3d(TangentVs[VertexIndex]), FVector3d(Normals[VertexIndex]));
		}

		FMemory::Memzero(Normals.GetData(), Normals.NumBytes());
		FMemory::Memzero(TangentUs.GetData(), TangentUs.NumBytes());

		for (int32 FaceIndex = 0; FaceIndex < FaceIndices.Num(); ++FaceIndex)
		{
			const FIntVector3& Face = FaceIndices[FaceIndex];

			const FVector3f& Pos0 = Positions[Face[0]];
			const FVector3f& Pos1 = Positions[Face[1]];
			const FVector3f& Pos2 = Positions[Face[2]];
			const FVector3f Pos01 = Pos1 - Pos0;
			const FVector3f Pos02 = Pos2 - Pos0;
			const FVector3f Normal = FVector3f::CrossProduct(Pos02, Pos01);

			FVector3f TangentU;
			if (UVs[Face[0]].Num() && UVs[Face[1]].Num() && UVs[Face[2]].Num())
			{
				const FVector2f UV01 = UVs[Face[1]][0] - UVs[Face[0]][0];
				const FVector2f UV02 = UVs[Face[2]][0] - UVs[Face[0]][0];
				const float Denom = UV01.X * UV02.Y - UV01.Y * UV02.X;
				const float InvDenom = (FMath::Abs(Denom) < SMALL_NUMBER) ? 0.f : 1.f / Denom;
				TangentU = (Pos01 * UV02.Y - Pos02 * UV01.Y) * InvDenom;
			}
			else
			{
				TangentU = (Pos1 + Pos2) * 0.5f - Pos0;
			}

			for (const int32 PointIndex : { 0, 1, 2 })
			{
				Normals[Face[PointIndex]] += Normal;
				TangentUs[Face[PointIndex]] += TangentU;
			}
		}

		const int32 NumVertices = Positions.Num();
		ParallelFor(NumVertices, [&Normals, &TangentUs, &TangentVs, &BasisDeterminantSigns](const int32 VertexIndex)
			{
				FVector3f& Normal = Normals[VertexIndex];
				FVector3f& TangentU = TangentUs[VertexIndex];
				FVector3f& TangentV = TangentVs[VertexIndex];

				Normal= Normal.GetSafeNormal();
				TangentU = TangentU.GetSafeNormal();
				TangentU = TangentU - FVector3f::DotProduct(TangentU, Normal) * Normal;
				TangentV = BasisDeterminantSigns[VertexIndex] * FVector3f::CrossProduct(Normal, TangentU);
				TangentV = TangentV.GetSafeNormal();
			}, NumVertices < 2000);
	}

	void FClothGeometryTools::BindMeshToRootBone(
		const TSharedRef<FManagedArrayCollection>& ClothCollection,
		bool bBindSimMesh,
		bool bBindRenderMesh)
	{
		if (!bBindSimMesh && !bBindRenderMesh)
		{
			return;
		}

		FCollectionClothFacade ClothFacade(ClothCollection);
		if (bBindSimMesh)
		{
			const int32 NumVertices = ClothFacade.GetNumSimVertices3D();
			TArrayView<TArray<int32>> BoneIndices = ClothFacade.GetSimBoneIndices();
			TArrayView<TArray<float>> BoneWeights = ClothFacade.GetSimBoneWeights();

			for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
			{
				BoneIndices[VertexIndex] = { 0 };
				BoneWeights[VertexIndex] = { 1.0f };
			}
		}

		if (bBindRenderMesh)
		{
			const int32 NumVertices = ClothFacade.GetNumRenderVertices();
			TArrayView<TArray<int32>> BoneIndices = ClothFacade.GetRenderBoneIndices();
			TArrayView<TArray<float>> BoneWeights = ClothFacade.GetRenderBoneWeights();

			for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
			{
				BoneIndices[VertexIndex] = { 0 };
				BoneWeights[VertexIndex] = { 1.0f };
			}
		}
	}


	void FClothGeometryTools::BuildSimMeshFromDynamicMeshes(
		const TSharedRef<FManagedArrayCollection>& ClothCollection,
		const UE::Geometry::FDynamicMesh3& Mesh2D,
		const UE::Geometry::FDynamicMesh3& Mesh3D,
		int32 PatternIndexLayerID,
		bool bTransferWeightMaps,
		bool bTransferSimSkinningData,
		bool bAppend,
		TMap<int, int32>& OutDynamicMeshToClothVertexMap)
	{
		using namespace UE::Geometry;

		if (!bAppend)
		{
			DeleteSimMesh(ClothCollection);
		}
		FCollectionClothFacade Cloth(ClothCollection);
		checkf(Cloth.IsValid(), TEXT("Invalid ClothCollection passed into BuildSimMeshFromDynamicMeshes"));

		check(Mesh2D.HasAttributes());
		const FDynamicMeshPolygroupAttribute* const PatternLayer = Mesh2D.Attributes()->GetPolygroupLayer((int)PatternIndexLayerID);
		check(PatternLayer);

		TArray<TArray<int>> PatternIndices;
		for (int FaceID = 0; FaceID < Mesh2D.MaxTriangleID(); ++FaceID)
		{
			const int32 PatternID = PatternLayer->GetValue(FaceID);
			if (PatternID >= PatternIndices.Num())
			{
				PatternIndices.SetNum(PatternID + 1);
			}

			const FIndex3i Tri = Mesh2D.GetTriangle(FaceID);
			PatternIndices[PatternID].Add(Tri[0]);
			PatternIndices[PatternID].Add(Tri[1]);
			PatternIndices[PatternID].Add(Tri[2]);
		}

		TMap<int, FIntVector2> MeshVertexToPatternAndVertex;

		for (int32 PatternID = 0; PatternID < PatternIndices.Num(); ++PatternID)
		{
			FCollectionClothSimPatternFacade Pattern = Cloth.AddGetSimPattern();
			const int32 PatternVertexOffset = Pattern.GetSimVertices2DOffset();

			TArray<FVector2f> Positions2D;
			TArray<FVector3f> Positions3D;

			const TArray<int>& InPatternIndexBuffer = PatternIndices[PatternID];
			
			TArray<int> LocalPatternIndexBuffer;

			for (const int VertexIndex : InPatternIndexBuffer)
			{
				int32 PatternVertexID;

				if (!MeshVertexToPatternAndVertex.Contains(VertexIndex))
				{
					const FVector3d InPosition2D = Mesh2D.GetVertex(VertexIndex);
					PatternVertexID = Positions2D.Add(FVector2f(InPosition2D[0], InPosition2D[1]));

					const FVector3d InPosition3D = Mesh3D.GetVertex(VertexIndex);
					Positions3D.Add(FVector3f(InPosition3D));

					MeshVertexToPatternAndVertex.Add({ VertexIndex, FIntVector2{PatternID, PatternVertexID}});
				}
				else
				{
					check(MeshVertexToPatternAndVertex[VertexIndex][0] == PatternID);
					PatternVertexID = MeshVertexToPatternAndVertex[VertexIndex][1];
				}

				LocalPatternIndexBuffer.Add(PatternVertexID);
			}

			Pattern.Initialize(Positions2D, Positions3D, LocalPatternIndexBuffer);
		}

		for (int InGlobalVertexIndex = 0; InGlobalVertexIndex < Mesh2D.MaxVertexID(); ++InGlobalVertexIndex)
		{
			const int32 PatternID = MeshVertexToPatternAndVertex[InGlobalVertexIndex][0];
			const int32 VertexID = MeshVertexToPatternAndVertex[InGlobalVertexIndex][1];
			const int32 ClothGlobalIndex = Cloth.GetSimPattern(PatternID).GetSimVertices2DOffset() + VertexID;

			OutDynamicMeshToClothVertexMap.Add({ InGlobalVertexIndex, ClothGlobalIndex });
		}

		// Copy skinning data
		if (bTransferSimSkinningData)
		{
			const UE::Geometry::FDynamicMeshVertexSkinWeightsAttribute* const SkinWeights = Mesh2D.Attributes() ? Mesh2D.Attributes()->GetSkinWeightsAttribute(FName("Default")) : nullptr;
			if (SkinWeights)
			{
				TArrayView<TArray<int32>> BoneIndices = Cloth.GetSimBoneIndices();
				TArrayView<TArray<float>> BoneWeights = Cloth.GetSimBoneWeights();
				for (int32 MeshVertexIndex : Mesh2D.VertexIndicesItr())
				{
					const int32 ClothVertexIndex = OutDynamicMeshToClothVertexMap[MeshVertexIndex];
					SkinWeights->GetValue(MeshVertexIndex, BoneIndices[ClothVertexIndex], BoneWeights[ClothVertexIndex]);
				}
			}
		}

		// Copy scalar weight maps
		if (bTransferWeightMaps && Mesh2D.Attributes())
		{
			const UE::Geometry::FDynamicMeshAttributeSet* const AttributeSet = Mesh2D.Attributes();
			for (int32 WeightMapLayerIndex = 0; WeightMapLayerIndex < AttributeSet->NumWeightLayers(); ++WeightMapLayerIndex)
			{
				if (const UE::Geometry::FDynamicMeshWeightAttribute* const WeightMapAttribute = AttributeSet->GetWeightLayer(WeightMapLayerIndex))
				{
					const FName WeightMapName = WeightMapAttribute->GetName();
					Cloth.AddWeightMap(WeightMapName);	// Does nothing if weight map already exists
					TArrayView<float> OutWeightMap = Cloth.GetWeightMap(WeightMapName);

					for (const int32 MeshVertexIndex : Mesh2D.VertexIndicesItr())
					{
						float VertexWeight;
						WeightMapAttribute->GetValue(MeshVertexIndex, &VertexWeight);

						const int32 ClothVertexIndex = OutDynamicMeshToClothVertexMap[MeshVertexIndex];
						OutWeightMap[ClothVertexIndex] = VertexWeight;
					}
				}
			}
		}

	}


	void FClothGeometryTools::BuildSimMeshFromDynamicMesh(
		const TSharedRef<FManagedArrayCollection>& ClothCollection,
		const UE::Geometry::FDynamicMesh3& DynamicMesh, int32 UVChannelIndex, const FVector2f& UVScale, bool bAppend, bool bImportNormals,
		TArray<int32>* OutSim2DToSourceIndex)
	{
		using namespace Private::SimMeshBuilder;

		if (!bAppend)
		{
			DeleteSimMesh(ClothCollection);

			if (OutSim2DToSourceIndex)
			{
				OutSim2DToSourceIndex->Reset();
			}
		}

		const UE::Geometry::FDynamicMeshAttributeSet* const AttributeSet = DynamicMesh.Attributes();
		const UE::Geometry::FDynamicMeshUVOverlay* const UVOverlay = AttributeSet ? AttributeSet->GetUVLayer(UVChannelIndex) : nullptr;
		const UE::Geometry::FDynamicMeshVertexSkinWeightsAttribute* SkinWeights = AttributeSet ? AttributeSet->GetSkinWeightsAttribute(FName("Default")) : nullptr;
		const UE::Geometry::FNonManifoldMappingSupport NonManifoldMapping(DynamicMesh);

		TArray<FIsland> Islands;
		if (UVOverlay)
		{
			BuildIslandsFromDynamicMeshUVs(*UVOverlay, UVScale, bImportNormals, Islands);
		}
		else
		{
			UnwrapDynamicMesh(DynamicMesh, bImportNormals, Islands);
		}

		FCollectionClothFacade Cloth(ClothCollection);
		const int32 PatternIndexOffset = bAppend ? Cloth.GetNumSimPatterns() : 0;
		for (FIsland& Island : Islands)
		{
			if (Island.Indices.Num() && Island.Positions2D.Num() && Island.Positions3D.Num())
			{
				FCollectionClothSimPatternFacade Pattern = Cloth.AddGetSimPattern();
				const int32 VertexOffset = Cloth.GetNumSimVertices3D();
				Pattern.Initialize(Island.Positions2D, Island.Positions3D, Island.Indices, INDEX_NONE, Island.Normals);

				// Copy skinning data
				if (SkinWeights)
				{
					TArrayView<TArray<int32>> BoneIndices = Cloth.GetSimBoneIndices();
					TArrayView<TArray<float>> BoneWeights = Cloth.GetSimBoneWeights();
					const int32 VertexCount = Island.Positions3D.Num();
					for (int32 VertexIndex = 0; VertexIndex < VertexCount; ++VertexIndex)
					{						
						SkinWeights->GetValue(Island.PositionToSourceIndex[VertexIndex], BoneIndices[VertexIndex + VertexOffset], BoneWeights[VertexIndex + VertexOffset]);
					}
				}

				// Copy scalar weight maps
				if (AttributeSet)
				{
					for (int32 WeightMapLayerIndex = 0; WeightMapLayerIndex < AttributeSet->NumWeightLayers(); ++WeightMapLayerIndex)
					{
						if (const UE::Geometry::FDynamicMeshWeightAttribute* const WeightMapAttribute = AttributeSet->GetWeightLayer(WeightMapLayerIndex))
						{
							const FName WeightMapName = WeightMapAttribute->GetName();
							Cloth.AddWeightMap(WeightMapName);	// Does nothing if weight map already exists
							TArrayView<float> OutWeightMap = Cloth.GetWeightMap(WeightMapName);

							for (int32 VertexIndex = 0; VertexIndex < Island.Positions3D.Num(); ++VertexIndex)
							{
								float VertexWeight;
								WeightMapAttribute->GetValue(Island.PositionToSourceIndex[VertexIndex], &VertexWeight);
								OutWeightMap[VertexIndex + VertexOffset] = VertexWeight;
							}
						}
					}
				}

				// Update OutSim2DToSourceIndex
				if (OutSim2DToSourceIndex)
				{
					OutSim2DToSourceIndex->Reserve(OutSim2DToSourceIndex->Num() + Island.PositionToSourceIndex.Num());
					for (int32 VertexIndex = 0; VertexIndex < Island.PositionToSourceIndex.Num(); ++VertexIndex)
					{
						OutSim2DToSourceIndex->Add(NonManifoldMapping.GetOriginalNonManifoldVertexID(Island.PositionToSourceIndex[VertexIndex]));
					}
				}
			}
		}

		TArray<FSeam> Seams;
		BuildSeams(Islands, DynamicMesh, PatternIndexOffset, Seams);  // Build the seam information as to be able to re-weld the mesh for simulation
		for (const FSeam& Seam : Seams)
		{
			AddSeam(Cloth, Seam);
		}
	}

	void FClothGeometryTools::CleanupAndCompactMesh(const TSharedRef<FManagedArrayCollection>& ClothCollection)
	{
		FCollectionClothFacade Cloth(ClothCollection);

		TArray<int32> SimPatternsToRemove;
		for (int32 PatternIndex = 0; PatternIndex < Cloth.GetNumSimPatterns(); ++PatternIndex)
		{
			FCollectionClothSimPatternFacade Pattern = Cloth.GetSimPattern(PatternIndex);
			{
				// Remove any triangles that are topologically degenerate
				TArray<int32> FacesToRemove;
				TConstArrayView<FIntVector3> SimIndices3D = Pattern.GetSimIndices3D();
				TConstArrayView<FIntVector3> SimIndices2D = Pattern.GetSimIndices2D();
				for (int32 FaceIndex = 0; FaceIndex < SimIndices3D.Num(); ++FaceIndex)
				{
					if (SimIndices3D[FaceIndex][0] == INDEX_NONE ||
						SimIndices3D[FaceIndex][1] == INDEX_NONE ||
						SimIndices3D[FaceIndex][2] == INDEX_NONE ||

						SimIndices2D[FaceIndex][0] == INDEX_NONE ||
						SimIndices2D[FaceIndex][1] == INDEX_NONE ||
						SimIndices2D[FaceIndex][2] == INDEX_NONE ||

						SimIndices3D[FaceIndex][0] == SimIndices3D[FaceIndex][1] ||
						SimIndices3D[FaceIndex][0] == SimIndices3D[FaceIndex][2] ||
						SimIndices3D[FaceIndex][1] == SimIndices3D[FaceIndex][2] ||

						SimIndices2D[FaceIndex][0] == SimIndices2D[FaceIndex][1] ||
						SimIndices2D[FaceIndex][0] == SimIndices2D[FaceIndex][2] ||
						SimIndices2D[FaceIndex][1] == SimIndices2D[FaceIndex][2])
					{
						FacesToRemove.Add(FaceIndex);
					}
				}

				if (FacesToRemove.Num())
				{
					Pattern.RemoveSimFaces(FacesToRemove);
				}
			}
			{
				// Remove any 2D vertices that are not used in a face.
				TConstArrayView<FIntVector3> SimIndices2D = Pattern.GetSimIndices2D();
				const int32 SimVertex2DOffset = Pattern.GetSimVertices2DOffset();
				TBitArray SimVertex2DToRemove;
				SimVertex2DToRemove.Init(true, Pattern.GetNumSimVertices2D());
				for (const FIntVector3& Face : SimIndices2D)
				{
					SimVertex2DToRemove[Face[0] - SimVertex2DOffset] = false;
					SimVertex2DToRemove[Face[1] - SimVertex2DOffset] = false;
					SimVertex2DToRemove[Face[2] - SimVertex2DOffset] = false;
				}

				TArray<int32> SimVertex2DToRemoveList;
				for (TConstSetBitIterator ToRemoveIter(SimVertex2DToRemove); ToRemoveIter; ++ToRemoveIter)
				{
					SimVertex2DToRemoveList.Add(ToRemoveIter.GetIndex());
				}

				if (SimVertex2DToRemoveList.Num())
				{
					Pattern.RemoveSimVertices2D(SimVertex2DToRemoveList);
				}
			}

			if (Pattern.IsEmpty())
			{
				SimPatternsToRemove.Add(PatternIndex);
			}
		}
		if (!SimPatternsToRemove.IsEmpty())
		{
			Cloth.RemoveSimPatterns(SimPatternsToRemove);
		}

		// Remove any unused 3D vertices
		{
			TConstArrayView<FIntVector3> SimIndices3D = Cloth.GetSimIndices3D();
			TBitArray SimVertex3DToRemove;
			SimVertex3DToRemove.Init(true, Cloth.GetNumSimVertices3D());
			for (const FIntVector3& Face : SimIndices3D)
			{
				SimVertex3DToRemove[Face[0]] = false;
				SimVertex3DToRemove[Face[1]] = false;
				SimVertex3DToRemove[Face[2]] = false;
			}

			TArray<int32> SimVertex3DToRemoveList;
			for (TConstSetBitIterator ToRemoveIter(SimVertex3DToRemove); ToRemoveIter; ++ToRemoveIter)
			{
				SimVertex3DToRemoveList.Add(ToRemoveIter.GetIndex());
			}

			if (SimVertex3DToRemoveList.Num())
			{
				Cloth.RemoveSimVertices3D(SimVertex3DToRemoveList);
			}
		}
		{
			// Clean up any references to vertices that no longer exist.
			// NOTE: should not need to clean up 2D vertices pointing to INDEX_NONE 3D vertices since this should have
			// meant the 2D vertex either was unused in the faces, or was associated with an invalid face (it should already be cleaned up).
			Cloth.CompactSimVertex2DLookup();

			TArrayView<TArray<int32>> TetherKinematicIndex = Cloth.GetTetherKinematicIndex();
			TArrayView<TArray<float>> TetherReferenceLength = Cloth.GetTetherReferenceLength();
			const int32 NumVertices = TetherKinematicIndex.Num();
			for (int32 VertexIdx = 0; VertexIdx < NumVertices; ++VertexIdx)
			{
				for (int32 TetherIdx = 0; TetherIdx < TetherKinematicIndex[VertexIdx].Num(); )
				{
					if (TetherKinematicIndex[VertexIdx][TetherIdx] == INDEX_NONE)
					{
						TetherKinematicIndex[VertexIdx].RemoveAtSwap(TetherIdx);
						TetherReferenceLength[VertexIdx].RemoveAtSwap(TetherIdx);
						continue;
					}
					++TetherIdx;
				}
			}

			// Clean up seams. Update stitches that refer to invalid indices.
			TArray<int32> SeamsToRemove;
			for (int32 SeamIndex = 0; SeamIndex < Cloth.GetNumSeams(); ++SeamIndex)
			{
				FCollectionClothSeamFacade Seam = Cloth.GetSeam(SeamIndex);
				Seam.CleanupAndCompact();
				if (Seam.GetNumSeamStitches() == 0)
				{
					SeamsToRemove.Add(SeamIndex);
				}
			}
			if (!SeamsToRemove.IsEmpty())
			{
				Cloth.RemoveSeams(SeamsToRemove);
			}
			Cloth.CompactSeamStitchLookup();
		}

		TArray<int32> RenderPatternsToRemove;
		for (int32 PatternIndex = 0; PatternIndex < Cloth.GetNumRenderPatterns(); ++PatternIndex)
		{
			FCollectionClothRenderPatternFacade Pattern = Cloth.GetRenderPattern(PatternIndex);
			{
				// Remove any triangles that are topologically degenerate
				TArray<int32> FacesToRemove;
				TConstArrayView<FIntVector3> RenderIndices = Pattern.GetRenderIndices();
				for (int32 FaceIndex = 0; FaceIndex < RenderIndices.Num(); ++FaceIndex)
				{
					if (RenderIndices[FaceIndex][0] == INDEX_NONE ||
						RenderIndices[FaceIndex][1] == INDEX_NONE ||
						RenderIndices[FaceIndex][2] == INDEX_NONE ||

						RenderIndices[FaceIndex][0] == RenderIndices[FaceIndex][1] ||
						RenderIndices[FaceIndex][0] == RenderIndices[FaceIndex][2] ||
						RenderIndices[FaceIndex][1] == RenderIndices[FaceIndex][2])
					{
						FacesToRemove.Add(FaceIndex);
					}
				}

				if (FacesToRemove.Num())
				{
					Pattern.RemoveRenderFaces(FacesToRemove);
				}
			}

			{
				// Remove any vertices that are not used in a face.
				TConstArrayView<FIntVector3> RenderIndices = Pattern.GetRenderIndices();
				const int32 RenderVertexOffset = Pattern.GetRenderVerticesOffset();
				TBitArray RenderVertexToRemove;
				RenderVertexToRemove.Init(true, Pattern.GetNumRenderVertices());
				for (const FIntVector3& Face : RenderIndices)
				{
					RenderVertexToRemove[Face[0] - RenderVertexOffset] = false;
					RenderVertexToRemove[Face[1] - RenderVertexOffset] = false;
					RenderVertexToRemove[Face[2] - RenderVertexOffset] = false;
				}

				TArray<int32> RenderVertexToRemoveList;
				for (TConstSetBitIterator ToRemoveIter(RenderVertexToRemove); ToRemoveIter; ++ToRemoveIter)
				{
					RenderVertexToRemoveList.Add(ToRemoveIter.GetIndex());
				}

				if (RenderVertexToRemoveList.Num())
				{
					Pattern.RemoveRenderVertices(RenderVertexToRemoveList);
				}
			}

			if (Pattern.IsEmpty())
			{
				RenderPatternsToRemove.Add(PatternIndex);
			}
		}
		if (!RenderPatternsToRemove.IsEmpty())
		{
			Cloth.RemoveRenderPatterns(RenderPatternsToRemove);
		}
#if DO_ENSURE
		for (int32 SeamIndex = 0; SeamIndex < Cloth.GetNumSeams(); ++SeamIndex)
		{
			Cloth.GetSeam(SeamIndex).ValidateSeam();
		}
#endif
		::Chaos::Softs::FEmbeddedSpringFacade SpringFacade(ClothCollection.Get(), ClothCollectionGroup::SimVertices3D);
		SpringFacade.CleanupAndCompactInvalidSprings();

		Cloth.CompactSimMorphTargets();
	}

	void FClothGeometryTools::StripUnusedUserAttributes(const TSharedRef<FManagedArrayCollection>& ClothCollection, bool bStripUnusedSimWeightMaps, const TArray<FName>& ExtraSimWeightMaps, bool bStripUnusedSets, const TArray<FName>& ExtraSets, bool bStripUnusedFaceIntMaps, const TArray<FName>& ExtraFaceIntMaps, bool bStripUnusedRenderWeightMaps, const TArray<FName>& ExtraRenderWeightMaps)
	{
		const ::Chaos::Softs::FCollectionPropertyConstFacade Properties(ClothCollection);
		FCollectionClothFacade Cloth(ClothCollection);

		auto IsNameUsedInProperties = [&Properties](const FName& WeightMapName)->bool
			{
				for (int32 PropertyIndex = 0; PropertyIndex < Properties.Num(); ++PropertyIndex)
				{
					if (Properties.GetStringValue(PropertyIndex) == WeightMapName)
					{
						return true;
					}
				}
				return false;
			};

		auto IsNameReserved = [&Properties](const FName& WeightMapName)->bool
			{
				return WeightMapName.ToString().StartsWith(TEXT("_"));
			};

		if (bStripUnusedSimWeightMaps)
		{
			const TArray<FName> WeightMapNames = Cloth.GetWeightMapNames();
			for (const FName& WeightMapName : WeightMapNames)
			{
				if (!(IsNameUsedInProperties(WeightMapName) || IsNameReserved(WeightMapName) || ExtraSimWeightMaps.Contains(WeightMapName)))
				{
					Cloth.RemoveWeightMap(WeightMapName);
				}
			}
		}

		if (bStripUnusedSets)
		{
			FCollectionClothSelectionFacade Selection(ClothCollection);
			const TArray<FName> SelectionNames = Selection.GetNames();
			for (const FName& SelectionName : SelectionNames)
			{
				if (!(IsNameUsedInProperties(SelectionName) || IsNameReserved(SelectionName) || ExtraSets.Contains(SelectionName)))
				{
					Selection.RemoveSelectionSet(SelectionName);
				}
			}
		}

		if (bStripUnusedFaceIntMaps)
		{
			const TArray<FName> FaceIntMapNames = Cloth.GetUserDefinedAttributeNames<int32>(ClothCollectionGroup::SimFaces);
			for (const FName& FaceIntMapName : FaceIntMapNames)
			{
				if (!(IsNameUsedInProperties(FaceIntMapName) || IsNameReserved(FaceIntMapName) || ExtraFaceIntMaps.Contains(FaceIntMapName)))
				{
					Cloth.RemoveUserDefinedAttribute(FaceIntMapName, ClothCollectionGroup::SimFaces);
				}
			}
		}

		if (bStripUnusedRenderWeightMaps)
		{
			const TArray<FName> WeightMapNames = Cloth.GetUserDefinedAttributeNames<float>(ClothCollectionGroup::RenderVertices);
			for (const FName& WeightMapName : WeightMapNames)
			{
				if (!(IsNameUsedInProperties(WeightMapName) || IsNameReserved(WeightMapName) || ExtraRenderWeightMaps.Contains(WeightMapName)))
				{
					Cloth.RemoveUserDefinedAttribute(WeightMapName, ClothCollectionGroup::RenderVertices);
				}
			}
		}
	}

	void FClothGeometryTools::BuildConnectedSeams(const TArray<FIntVector2>& InputStitches,
		const UE::Geometry::FDynamicMesh3& Mesh,
		TArray<TArray<FIntVector2>>& Seams)
	{
		TArray<FIntVector2> Stitches = InputStitches;

		// filter out any stitches referencing deleted vertices
		Stitches.SetNum(Algo::RemoveIf(Stitches, [](const FIntVector2& Stitch)
		{
			return Stitch[0] == INDEX_NONE || Stitch[1] == INDEX_NONE;
		}));

		while (Stitches.Num() > 0)
		{
			TArray<FIntVector2> Seam;

			const FIntVector2 FirstStitch = Stitches.Last();
			Seam.Add(FirstStitch);
			Stitches.RemoveAt(Stitches.Num() - 1);

			FIntVector2 CurrStitch = FirstStitch;
			bool bFoundNextStitch = true;
			bool bReverseSearch = false;
			while (Stitches.Num() > 0 && (bFoundNextStitch || !bReverseSearch))
			{
				bFoundNextStitch = false;

				for (int32 TestStitchIndex = 0; TestStitchIndex < Stitches.Num(); ++TestStitchIndex)
				{
					FIntVector2 TestStitch = Stitches[TestStitchIndex];

					// Stitch (A, B) is connected to stitch (C, D) if there exist edges {(A, C), (B, D)} *or* {(A, D), (B, C)} in the given DynamicMesh.

					const int32 A = CurrStitch[0];
					const int32 B = CurrStitch[1];
					const int32 C = TestStitch[0];
					const int32 D = TestStitch[1];

					if (Mesh.FindEdge(A, C) != UE::Geometry::FDynamicMesh3::InvalidID && Mesh.FindEdge(B, D) != UE::Geometry::FDynamicMesh3::InvalidID)
					{
						Seam.Add(TestStitch);
						bFoundNextStitch = true;
					}
					else if (Mesh.FindEdge(A, D) != UE::Geometry::FDynamicMesh3::InvalidID && Mesh.FindEdge(B, C) != UE::Geometry::FDynamicMesh3::InvalidID)
					{
						Swap(TestStitch[0], TestStitch[1]);
						Seam.Add(TestStitch);
						bFoundNextStitch = true;
					}

					if (bFoundNextStitch)
					{
						Stitches.RemoveAt(TestStitchIndex);
						CurrStitch = TestStitch;
						break;
					}

				}

				if (!bFoundNextStitch && !bReverseSearch)
				{
					Algo::Reverse(Seam);
					bReverseSearch = true;
					bFoundNextStitch = true;
					CurrStitch = FirstStitch;
				}
			}

			// Finished one connected set of seam edges
			Seams.Add(Seam);
		}
	}


	void FClothGeometryTools::BuildConnectedSeams2D(const TSharedRef<const FManagedArrayCollection>& ClothCollection,
		int32 SeamIndex,
		const UE::Geometry::FDynamicMesh3& Mesh,
		TArray<TArray<FIntVector2>>& Seams)
	{
		using namespace UE::Geometry;

		FNonManifoldMappingSupport NonManifold(Mesh);
		checkf(!NonManifold.IsNonManifoldVertexInSource(), TEXT("Cloth source is non-manifold. Cannot use FDynamicMesh to build connected seams"));

		const FCollectionClothConstFacade ClothFacade(ClothCollection);
		const FCollectionClothSeamConstFacade SeamFacade = ClothFacade.GetSeam(SeamIndex);
		
		const TArray<FIntVector2> Stitches(SeamFacade.GetSeamStitch2DEndIndices());

		BuildConnectedSeams(Stitches, Mesh, Seams);
	}

	void FClothGeometryTools::SampleVertices(const TConstArrayView<FVector3f> VertexPositions, float CullDiameterSq, TSet<int32>& OutVertexSet)
	{
		check(CullDiameterSq > 0.0f);

		TArray<bool> VertexIsValid;
		VertexIsValid.Init(true, VertexPositions.Num());

		for (int32 Index = 0; Index < VertexPositions.Num(); ++Index)
		{
			if (!VertexIsValid[Index])
			{
				continue;
			}
			OutVertexSet.Add(Index);

			const FVector3f& Pos0 = VertexPositions[Index];
			for (int32 CompareIndex = Index + 1; CompareIndex < VertexPositions.Num(); ++CompareIndex)
			{
				if (!VertexIsValid[CompareIndex])
				{
					continue;
				}
				if (FVector3f::DistSquared(Pos0, VertexPositions[CompareIndex]) < CullDiameterSq)
				{
					VertexIsValid[CompareIndex] = false;
				}
			}
		}
	}

	bool FClothGeometryTools::ConvertSelectionToNewGroupType(const TSharedRef<const FManagedArrayCollection>& ClothCollection, const FName& SelectionName, const FName& GroupName, bool bSecondarySelection, TSet<int32>& OutSelectionSet)
	{
		FCollectionClothSelectionConstFacade SelectionFacade(ClothCollection);
		FCollectionClothConstFacade ClothFacade(ClothCollection);
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (!SelectionFacade.IsValid() || !ClothFacade.IsValid() || (bSecondarySelection ? !SelectionFacade.HasSelectionSecondarySet(SelectionName) : !SelectionFacade.HasSelection(SelectionName)))
		{
			return false;
		}

		const TSet<int32>& OrigSelectionSet = bSecondarySelection ? SelectionFacade.GetSelectionSecondarySet(SelectionName) : SelectionFacade.GetSelectionSet(SelectionName);
		const FName OrigSelectionGroup = bSecondarySelection ? SelectionFacade.GetSelectionSecondaryGroup(SelectionName) : SelectionFacade.GetSelectionGroup(SelectionName);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		if (OrigSelectionGroup == GroupName)
		{
			OutSelectionSet = OrigSelectionSet;
			return true;
		}

		auto ConvertVerticesToFaces = [&OrigSelectionSet, &OutSelectionSet](const TConstArrayView<FIntVector3>& Indices)
		{
			OutSelectionSet.Reset();
			OutSelectionSet.Reserve(OrigSelectionSet.Num());
			for (int32 FaceIndex = 0; FaceIndex < Indices.Num(); ++FaceIndex)
			{
				const FIntVector3& Element = Indices[FaceIndex];
				if (OrigSelectionSet.Contains(Element[0]) &&
					OrigSelectionSet.Contains(Element[1]) &&
					OrigSelectionSet.Contains(Element[2]))
				{
					OutSelectionSet.Add(FaceIndex);
				}
			}
		};


		auto ConvertFacesToVertices = [&OrigSelectionSet, &OutSelectionSet](const TConstArrayView<FIntVector3>& Indices)
		{
			OutSelectionSet.Reset();
			OutSelectionSet.Reserve(OrigSelectionSet.Num());
			for (const int32 FaceIndex : OrigSelectionSet)
			{
				if (Indices.IsValidIndex(FaceIndex))
				{
					OutSelectionSet.Add(Indices[FaceIndex][0]);
					OutSelectionSet.Add(Indices[FaceIndex][1]);
					OutSelectionSet.Add(Indices[FaceIndex][2]);
				}
			}
		};	

		auto AppendElementRange = [&OutSelectionSet](const int32 StartIndex, const int32 NumElements)
		{
			OutSelectionSet.Reserve(OutSelectionSet.Num() + NumElements);
			for (int32 ElemIndex = StartIndex; ElemIndex < StartIndex + NumElements; ++ElemIndex)
			{
				OutSelectionSet.Add(ElemIndex);
			}
		};

		if (OrigSelectionGroup == ClothCollectionGroup::SimVertices2D)
		{
			if (GroupName == ClothCollectionGroup::SimFaces)
			{
				ConvertVerticesToFaces(ClothFacade.GetSimIndices2D());
				return true;
			}
			else if (GroupName == ClothCollectionGroup::SimVertices3D)
			{
				const TConstArrayView<int32> SimVertex3DLookup = ClothFacade.GetSimVertex3DLookup();
				OutSelectionSet.Reset();
				OutSelectionSet.Reserve(OrigSelectionSet.Num());
				for (const int32 OrigSelection : OrigSelectionSet)
				{
					if (SimVertex3DLookup.IsValidIndex(OrigSelection))
					{
						OutSelectionSet.Add(SimVertex3DLookup[OrigSelection]);
					}
				}
				return true;
			}
		}
		else if (OrigSelectionGroup == ClothCollectionGroup::SimVertices3D)
		{
			if (GroupName == ClothCollectionGroup::SimFaces)
			{
				ConvertVerticesToFaces(ClothFacade.GetSimIndices3D());
				return true;
			}
			else if (GroupName == ClothCollectionGroup::SimVertices2D)
			{
				const TConstArrayView<TArray<int32>> SimVertex2DLookup = ClothFacade.GetSimVertex2DLookup();
				OutSelectionSet.Reset();
				OutSelectionSet.Reserve(OrigSelectionSet.Num());
				for (const int32 OrigSelection : OrigSelectionSet)
				{
					if (SimVertex2DLookup.IsValidIndex(OrigSelection))
					{
						for (const int32 Vertex2D : SimVertex2DLookup[OrigSelection])
						{
							OutSelectionSet.Add(Vertex2D);
						}
					}
				}
				return true;
			}
		}
		else if (OrigSelectionGroup == ClothCollectionGroup::SimFaces)
		{
			if (GroupName == ClothCollectionGroup::SimVertices2D)
			{
				ConvertFacesToVertices(ClothFacade.GetSimIndices2D());
				return true;
			}
			else if (GroupName == ClothCollectionGroup::SimVertices3D)
			{
				ConvertFacesToVertices(ClothFacade.GetSimIndices3D());
				return true;
			}
		}
		else if (OrigSelectionGroup == ClothCollectionGroup::SimPatterns)
		{
			const int32 NumSimPatterns = ClothFacade.GetNumSimPatterns();
			if (GroupName == ClothCollectionGroup::SimFaces)
			{
				OutSelectionSet.Reset();
				for (const int32 PatternIndex : OrigSelectionSet)
				{
					if (PatternIndex >= 0 && PatternIndex < NumSimPatterns)
					{
						const FCollectionClothSimPatternConstFacade PatternFacade = ClothFacade.GetSimPattern(PatternIndex);
						AppendElementRange(PatternFacade.GetSimFacesOffset(), PatternFacade.GetNumSimFaces());
					}
				}
				return true;
			}
			else if (GroupName == ClothCollectionGroup::SimVertices2D)
			{
				OutSelectionSet.Reset();
				for (const int32 PatternIndex : OrigSelectionSet)
				{
					if (PatternIndex >= 0 && PatternIndex < NumSimPatterns)
					{
						const FCollectionClothSimPatternConstFacade PatternFacade = ClothFacade.GetSimPattern(PatternIndex);
						AppendElementRange(PatternFacade.GetSimVertices2DOffset(), PatternFacade.GetNumSimVertices2D());
					}
				}
				return true;
			}
			else if (GroupName == ClothCollectionGroup::SimVertices3D)
			{
				OutSelectionSet.Reset();
				for (const int32 PatternIndex : OrigSelectionSet)
				{
					if (PatternIndex >= 0 && PatternIndex < NumSimPatterns)
					{
						const FCollectionClothSimPatternConstFacade PatternFacade = ClothFacade.GetSimPattern(PatternIndex);
						OutSelectionSet.Reserve(OutSelectionSet.Num() + PatternFacade.GetNumSimVertices2D());

						const TConstArrayView<FIntVector> SimIndices3D = PatternFacade.GetSimIndices3D();
						for (const FIntVector& Indices : SimIndices3D)
						{
							OutSelectionSet.Add(Indices[0]);
							OutSelectionSet.Add(Indices[1]);
							OutSelectionSet.Add(Indices[2]);
						}
					}
				}
				return true;
			}
		}
		else if (OrigSelectionGroup == ClothCollectionGroup::RenderVertices)
		{
			if (GroupName == ClothCollectionGroup::RenderFaces)
			{
				ConvertVerticesToFaces(ClothFacade.GetRenderIndices());
				return true;
			}
		}
		else if (OrigSelectionGroup == ClothCollectionGroup::RenderFaces)
		{
			if (GroupName == ClothCollectionGroup::RenderVertices)
			{
				ConvertFacesToVertices(ClothFacade.GetRenderIndices());
				return true;
			}
		}
		else if (OrigSelectionGroup == ClothCollectionGroup::RenderPatterns)
		{
			const int32 NumRenderPatterns = ClothFacade.GetNumRenderPatterns();
			if (GroupName == ClothCollectionGroup::RenderFaces)
			{
				OutSelectionSet.Reset();
				for (const int32 PatternIndex : OrigSelectionSet)
				{
					if (PatternIndex >= 0 && PatternIndex < NumRenderPatterns)
					{
						const FCollectionClothRenderPatternConstFacade PatternFacade = ClothFacade.GetRenderPattern(PatternIndex);
						AppendElementRange(PatternFacade.GetRenderFacesOffset(), PatternFacade.GetNumRenderFaces());
					}
				}
				return true;
			}
			else if (GroupName == ClothCollectionGroup::RenderVertices)
			{
				OutSelectionSet.Reset();
				for (const int32 PatternIndex : OrigSelectionSet)
				{
					if (PatternIndex >= 0 && PatternIndex < NumRenderPatterns)
					{
						const FCollectionClothRenderPatternConstFacade PatternFacade = ClothFacade.GetRenderPattern(PatternIndex);
						AppendElementRange(PatternFacade.GetRenderVerticesOffset(), PatternFacade.GetNumRenderVertices());
					}
				}
				return true;
			}
		}

		return false;
	}

	bool FClothGeometryTools::ConvertSelectionToNewGroupType(const TSharedRef<const FManagedArrayCollection>& ClothCollection, const FName& SelectionName, const FName& GroupName, TSet<int32>& OutSelectionSet)
	{
		constexpr bool bSecondarySelection = false;
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return ConvertSelectionToNewGroupType(ClothCollection, SelectionName, GroupName, bSecondarySelection, OutSelectionSet);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void FClothGeometryTools::SelectAllInGroupType(const TSharedRef<FManagedArrayCollection>& ClothCollection, const FName& SelectionName, const FName& GroupName)
	{
		FCollectionClothSelectionFacade SelectionFacade(ClothCollection);
		SelectionFacade.DefineSchema();
		if (ClothCollection->HasGroup(GroupName))
		{
			const int32 GroupSize = ClothCollection->NumElements(GroupName);
			TSet<int32>& NewSelectionSet = SelectionFacade.FindOrAddSelectionSet(SelectionName, GroupName);
			NewSelectionSet.Reserve(GroupSize);
			for (int32 Index = 0; Index < GroupSize; ++Index)
			{
				NewSelectionSet.Add(Index);
			}
		}
	}

	void FClothGeometryTools::TransferWeightMap(
		const TConstArrayView<FVector3f>& SourcePositions,
		const TConstArrayView<FIntVector3>& InSourceIndices,
		const TConstArrayView<float>& SourceWeights,
		const TConstArrayView<FVector3f>& TargetPositions,
		const TConstArrayView<FVector3f>& TargetNormals,
		const TConstArrayView<FIntVector3>& InTargetIndices,
		const TArrayView<float>& TargetWeights)
	{
		check(TargetWeights.Num() == TargetPositions.Num());
		if (!ensure(SourcePositions.Num() <= 65536))
		{
			return;  // MeshToMeshVertData below is limited to 16bit unsigned int indexes
		}

		TArray<uint32> SourceIndices;
		SourceIndices.Reserve(InSourceIndices.Num() * 3);
		for (const FIntVector3& InSourceIndex : InSourceIndices)
		{
			SourceIndices.Add(InSourceIndex[0]);
			SourceIndices.Add(InSourceIndex[1]);
			SourceIndices.Add(InSourceIndex[2]);
		}
		TArray<uint32> TargetIndices;
		TargetIndices.Reserve(InTargetIndices.Num() * 3);
		for (const FIntVector3& InTargetIndex : InTargetIndices)
		{
			TargetIndices.Add(InTargetIndex[0]);
			TargetIndices.Add(InTargetIndex[1]);
			TargetIndices.Add(InTargetIndex[2]);
		}

		const ClothingMeshUtils::ClothMeshDesc SourceMeshDesc(SourcePositions, SourceIndices);
		const ClothingMeshUtils::ClothMeshDesc TargetMeshDesc(TargetPositions, TargetNormals, TargetIndices);

		TArray<FMeshToMeshVertData> MeshToMeshVertData;
		const FPointWeightMap* const MaxDistances = nullptr; // No need to update the vertex contribution on the transition maps
		constexpr bool bUseSmoothTransitions = false;  // Smooth transitions are only used at rendering for now and not during LOD transitions
		constexpr bool bUseMultipleInfluences = false;  // Multiple influences must not be used for LOD transitions
		constexpr float SkinningKernelRadius = 0.f;  // KernelRadius is only required when using multiple influences

		ClothingMeshUtils::GenerateMeshToMeshVertData(
			MeshToMeshVertData,
			TargetMeshDesc,
			SourceMeshDesc,
			MaxDistances,
			bUseSmoothTransitions,
			bUseMultipleInfluences,
			SkinningKernelRadius);

		check(MeshToMeshVertData.Num() == TargetWeights.Num());
		for (int32 Index = 0; Index < TargetWeights.Num(); ++Index)
		{
			const FMeshToMeshVertData& MeshToMeshVertDatum = MeshToMeshVertData[Index];

			const uint16 VertIndex0 = MeshToMeshVertDatum.SourceMeshVertIndices[0];
			const uint16 VertIndex1 = MeshToMeshVertDatum.SourceMeshVertIndices[1];
			const uint16 VertIndex2 = MeshToMeshVertDatum.SourceMeshVertIndices[2];

			TargetWeights[Index] = FMath::Clamp(
				SourceWeights[VertIndex0] * MeshToMeshVertDatum.PositionBaryCoordsAndDist[0] +
				SourceWeights[VertIndex1] * MeshToMeshVertDatum.PositionBaryCoordsAndDist[1] +
				SourceWeights[VertIndex2] * MeshToMeshVertDatum.PositionBaryCoordsAndDist[2], 0.f, 1.f);
		}
	}

	TSet<int32> FClothGeometryTools::GenerateKinematicVertices3D(const TSharedRef<FManagedArrayCollection>& ClothCollection, const FName& MaxDistanceMapName, const FVector2f& MaxDistanceValue, const FName& InputKinematicVertices, float KinematicDistanceThreshold)
	{
		TSet<int32> KinematicVertices;

		// Add InputKinematicVertices
		FCollectionClothSelectionConstFacade SelectionFacade(ClothCollection);
		if (InputKinematicVertices != NAME_None && SelectionFacade.IsValid() && SelectionFacade.HasSelection(InputKinematicVertices) && SelectionFacade.GetSelectionGroup(InputKinematicVertices) == UE::Chaos::ClothAsset::ClothCollectionGroup::SimVertices3D)
		{
			KinematicVertices = SelectionFacade.GetSelectionSet(InputKinematicVertices);
		}

		FCollectionClothFacade ClothFacade(ClothCollection);
		if (ClothFacade.IsValid())
		{
			if (ClothFacade.HasWeightMap(MaxDistanceMapName))
			{
				TConstArrayView<float> MaxDistanceMap = ClothFacade.GetWeightMap(MaxDistanceMapName);
				const FVector2f MaxDistanceOffsetRange(MaxDistanceValue[0], MaxDistanceValue[1] - MaxDistanceValue[0]);
				for (int32 Index = 0; Index < MaxDistanceMap.Num(); ++Index)
				{
					if (MaxDistanceOffsetRange[0] + MaxDistanceMap[Index] * MaxDistanceOffsetRange[1] < KinematicDistanceThreshold)
					{
						KinematicVertices.Add(Index);
					}
				}
			}
			else if (MaxDistanceValue[0] < KinematicDistanceThreshold)
			{
				KinematicVertices.Reserve(ClothFacade.GetNumSimVertices3D());
				for (int32 Index = 0; Index < ClothFacade.GetNumSimVertices3D(); ++Index)
				{
					KinematicVertices.Add(Index);
				}
			}
		}
		return KinematicVertices;
	}

	void FClothGeometryTools::ApplyProxyDeformer(const TSharedRef<FManagedArrayCollection>& ClothCollection, bool bIgnoreSkinningBlend)
	{
		FCollectionClothFacade ClothFacade(ClothCollection);
		if (ClothFacade.IsValid(EClothCollectionExtendedSchemas::RenderDeformer))
		{
			return ApplyProxyDeformer(ClothFacade, bIgnoreSkinningBlend, ClothFacade.GetSimPosition3D(), ClothFacade.GetSimNormal());
		}
	}

	void FClothGeometryTools::ApplyProxyDeformer(FCollectionClothFacade& ClothFacade, bool bIgnoreSkinningBlend, const TConstArrayView<FVector3f>& SimPositions, const TConstArrayView<FVector3f>& SimNormals)
	{
		if (ClothFacade.IsValid(EClothCollectionExtendedSchemas::RenderDeformer))
		{
			// This follows GpuSkinVertexFactor.ush
			for (int32 SectionIndex = 0; SectionIndex < ClothFacade.GetNumRenderPatterns(); ++SectionIndex)
			{
				FCollectionClothRenderPatternFacade RenderPatternFacade = ClothFacade.GetRenderPattern(SectionIndex);
				const int32 RenderDeformerNumInfluences = RenderPatternFacade.GetRenderDeformerNumInfluences();
				if (RenderDeformerNumInfluences > 0)
				{
					TArrayView<FVector3f> RenderPositions = RenderPatternFacade.GetRenderPosition();
					TArrayView<FVector3f> RenderNormal = RenderPatternFacade.GetRenderNormal();
					TArrayView<FVector3f> RenderTangentU = RenderPatternFacade.GetRenderTangentU();
					TArrayView<FVector3f> RenderTangentV = RenderPatternFacade.GetRenderTangentV();

					TConstArrayView<TArray<FVector4f>> PositionBaryCoordsAndDist = RenderPatternFacade.GetRenderDeformerPositionBaryCoordsAndDist();
					TConstArrayView<TArray<FVector4f>> NormalBaryCoordsAndDist = RenderPatternFacade.GetRenderDeformerNormalBaryCoordsAndDist();
					TConstArrayView<TArray<FVector4f>> TangentBaryCoordsAndDist = RenderPatternFacade.GetRenderDeformerTangentBaryCoordsAndDist();
					TConstArrayView<TArray<FIntVector3>> SimIndices3D = RenderPatternFacade.GetRenderDeformerSimIndices3D();
					TConstArrayView<TArray<float>> DeformerWeight = RenderPatternFacade.GetRenderDeformerWeight();
					TConstArrayView<float> SkinningBlend = RenderPatternFacade.GetRenderDeformerSkinningBlend();

					for (int32 Index = 0; Index < RenderPatternFacade.GetNumRenderVertices(); ++Index)
					{
						const float SkinningBlendValue = bIgnoreSkinningBlend ? 0.f : SkinningBlend[Index];
						if (SkinningBlendValue < 1)
						{
							FVector3f AveragedSimPosition(0.f);
							FVector3f NormalPosition(0.f);
							FVector3f TangentPosition(0.f);

							float SumWeights = 0.f;
							float SimulWeight = 0.f;
							for (int32 Influence = 0; Influence < RenderDeformerNumInfluences; ++Influence)
							{
								const FVector3f& SimPosA = SimPositions[SimIndices3D[Index][Influence].X];
								const FVector3f& SimPosB = SimPositions[SimIndices3D[Index][Influence].Y];
								const FVector3f& SimPosC = SimPositions[SimIndices3D[Index][Influence].Z];

								const FVector3f& SimNormalA = SimNormals[SimIndices3D[Index][Influence].X];
								const FVector3f& SimNormalB = SimNormals[SimIndices3D[Index][Influence].Y];
								const FVector3f& SimNormalC = SimNormals[SimIndices3D[Index][Influence].Z];

								SimulWeight += 1.f - SkinningBlendValue;

								float Weight = 1.f;
								if (RenderDeformerNumInfluences > 1)
								{
									Weight = DeformerWeight[Index][Influence];
									SumWeights += Weight;
								}
								else
								{
									Weight = 1.f;
									SumWeights = 1.f;
								}

								auto InterpolatePosition = [Weight, &SimPosA, &SimPosB, &SimPosC, &SimNormalA, &SimNormalB, &SimNormalC](const FVector4f& BaryCoord)->FVector3f
									{
										// Note: coordinates are calculated with inverted normals, so subtract them here.
										return Weight * (
											BaryCoord.X * (SimPosA - SimNormalA * BaryCoord.W) +
											BaryCoord.Y * (SimPosB - SimNormalB * BaryCoord.W) +
											BaryCoord.Z * (SimPosC - SimNormalC * BaryCoord.W));
									};

								const FVector4f& BaryCoordPos = PositionBaryCoordsAndDist[Index][Influence];
								AveragedSimPosition += InterpolatePosition(BaryCoordPos);

								const FVector4f& BaryCoordNormal = NormalBaryCoordsAndDist[Index][Influence];
								NormalPosition += InterpolatePosition(BaryCoordNormal);

								const FVector4f& BaryCoordTangent = TangentBaryCoordsAndDist[Index][Influence];
								TangentPosition += InterpolatePosition(BaryCoordTangent);
							}


							FVector3f Normal(0.f);
							FVector3f TangentU(0.f);
							FVector3f TangentV(0.f);
							if (SumWeights > UE_KINDA_SMALL_NUMBER)
							{
								float InvWeight = 1.f / SumWeights;
								AveragedSimPosition *= InvWeight;
								TangentPosition *= InvWeight;
								NormalPosition *= InvWeight;

								TangentU = (TangentPosition - AveragedSimPosition).GetSafeNormal();
								Normal = (NormalPosition - AveragedSimPosition).GetSafeNormal();

								TangentV = FVector3f::CrossProduct(Normal, TangentU).GetSafeNormal();

								// GetBasisDeterminantSign to determine if we need to flip TangentV.
								TangentV *= GetBasisDeterminantSign(FVector(RenderTangentU[Index]), FVector(RenderTangentV[Index]), FVector(RenderNormal[Index]));
							}
							else
							{
								SimulWeight = 0.f; // Fallback to skinned pos
							}

							if (RenderDeformerNumInfluences > 1)
							{
								SimulWeight /= RenderDeformerNumInfluences;
							}

							RenderPositions[Index] = FMath::Lerp(RenderPositions[Index], AveragedSimPosition, SimulWeight);
							RenderNormal[Index] = FMath::Lerp(RenderNormal[Index], Normal, SimulWeight).GetSafeNormal();
							RenderTangentU[Index] = FMath::Lerp(RenderTangentU[Index], TangentU, SimulWeight).GetSafeNormal();
							RenderTangentV[Index] = FMath::Lerp(RenderTangentV[Index], TangentV, SimulWeight).GetSafeNormal();
						}
					}
				}
			}
		}
	}
}  // End namespace UE::Chaos::ClothAsset
