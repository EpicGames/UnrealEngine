// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/CollectionClothFabricFacade.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "ChaosClothAsset/ClothCollection.h"
#include "ChaosClothAsset/ClothCollectionGroup.h"

namespace UE::Chaos::ClothAsset
{
	FCollectionClothFabricConstFacade::FAnisotropicData::FAnisotropicData(const float WeftValue, const float WarpValue, const float BiasValue) :
		Weft(WeftValue), Warp(WarpValue), Bias(BiasValue)
	{}
			
	FCollectionClothFabricConstFacade::FAnisotropicData::FAnisotropicData(const FVector3f& VectorDatas) :
		Weft(VectorDatas.X), Warp(VectorDatas.Y), Bias(VectorDatas.Z) 
	{}

	FCollectionClothFabricConstFacade::FAnisotropicData::FAnisotropicData(const float& FloatDatas) :
		Weft(FloatDatas), Warp(FloatDatas), Bias(FloatDatas)
	{}
			
	FVector3f FCollectionClothFabricConstFacade::FAnisotropicData::GetVectorDatas() const
	{
		return FVector3f(Weft, Warp, Bias);
	}
	
	FCollectionClothFabricConstFacade::FAnisotropicData FCollectionClothFabricConstFacade::GetBendingStiffness() const
	{
		return ClothCollection->GetFabricBendingStiffness() ? FCollectionClothFabricConstFacade::FAnisotropicData(
			ClothCollection->GetElements(ClothCollection->GetFabricBendingStiffness())[GetElementIndex()]) : FAnisotropicData(FDefaultFabric::BendingStiffness);
	}
	
	float FCollectionClothFabricConstFacade::GetBucklingRatio() const
	{
		return ClothCollection->GetFabricBucklingRatio() ? ClothCollection->GetElements(ClothCollection->GetFabricBucklingRatio())[GetElementIndex()] :
				FDefaultFabric::BucklingRatio;
	}
	
	FCollectionClothFabricConstFacade::FAnisotropicData FCollectionClothFabricConstFacade::GetBucklingStiffness() const
	{
		return ClothCollection->GetFabricBucklingStiffness() ? FCollectionClothFabricConstFacade::FAnisotropicData(
			ClothCollection->GetElements(ClothCollection->GetFabricBucklingStiffness())[GetElementIndex()]) : FAnisotropicData(FDefaultFabric::BucklingStiffness);
	}
	
	FCollectionClothFabricConstFacade::FAnisotropicData FCollectionClothFabricConstFacade::GetStretchStiffness() const
	{
		return ClothCollection->GetFabricStretchStiffness() ? FCollectionClothFabricConstFacade::FAnisotropicData(
			ClothCollection->GetElements(ClothCollection->GetFabricStretchStiffness())[GetElementIndex()]) : FAnisotropicData(FDefaultFabric::StretchStiffness);
	}
	
	float FCollectionClothFabricConstFacade::GetDensity() const
	{
		return ClothCollection->GetFabricDensity() ? ClothCollection->GetElements(ClothCollection->GetFabricDensity())[GetElementIndex()] : FDefaultFabric::Density;
	}
	
	float FCollectionClothFabricConstFacade::GetDamping() const
	{
		return ClothCollection->GetFabricDamping() ? ClothCollection->GetElements(ClothCollection->GetFabricDamping())[GetElementIndex()] : FDefaultFabric::Damping;
	}
	
	float FCollectionClothFabricConstFacade::GetFriction() const
	{
		return ClothCollection->GetFabricFriction() ? ClothCollection->GetElements(ClothCollection->GetFabricFriction())[GetElementIndex()] : FDefaultFabric::Friction;
	}
	
	float FCollectionClothFabricConstFacade::GetPressure() const
	{
		return ClothCollection->GetFabricPressure() ? ClothCollection->GetElements(ClothCollection->GetFabricPressure())[GetElementIndex()] : FDefaultFabric::Pressure;
	}
	
	int32 FCollectionClothFabricConstFacade::GetLayer() const
	{
		return ClothCollection->GetFabricLayer() ? ClothCollection->GetElements(ClothCollection->GetFabricLayer())[GetElementIndex()] : FDefaultFabric::Layer;
	}

	float FCollectionClothFabricConstFacade::GetCollisionThickness() const
	{
		return ClothCollection->GetFabricCollisionThickness() ? ClothCollection->GetElements(ClothCollection->GetFabricCollisionThickness())[GetElementIndex()] : FDefaultFabric::CollisionThickness;
	}
	
	FCollectionClothFabricConstFacade::FCollectionClothFabricConstFacade(const TSharedRef<const FConstClothCollection>& ClothCollection, int32 InFabricIndex)
		: ClothCollection(ClothCollection)
		, FabricIndex(InFabricIndex)
	{
		check(ClothCollection->IsValid());
		check(FabricIndex >= 0 && FabricIndex < ClothCollection->GetNumElements(ClothCollectionGroup::Fabrics));
	}

	void FCollectionClothFabricFacade::Initialize(const FAnisotropicData& BendingStiffness, const float BucklingRatio,
			const FAnisotropicData& BucklingStiffness, const FAnisotropicData& StretchStiffness,
			const float Density, const float Friction, const float Damping, const float Pressure, const int32 Layer, const float CollisionThickness)
	{
		const int32 ElementIndex = GetElementIndex();
		check((ElementIndex >= 0) && (ElementIndex < ClothCollection->GetNumElements(ClothCollectionGroup::Fabrics)));
		
		const TSharedRef<class FClothCollection> ClothCol = GetClothCollection();
		if (!ClothCol->IsValid(EClothCollectionExtendedSchemas::Fabrics))
		{
			ClothCol->DefineSchema(EClothCollectionExtendedSchemas::Fabrics);
		}
		
		ClothCol->GetElements(ClothCol->GetFabricBendingStiffness())[ElementIndex] = BendingStiffness.GetVectorDatas();
		ClothCol->GetElements(ClothCol->GetFabricBucklingRatio())[ElementIndex] = BucklingRatio;
		ClothCol->GetElements(ClothCol->GetFabricStretchStiffness())[ElementIndex] = StretchStiffness.GetVectorDatas();
		ClothCol->GetElements(ClothCol->GetFabricBucklingStiffness())[ElementIndex] = BucklingStiffness.GetVectorDatas();
		ClothCol->GetElements(ClothCol->GetFabricFriction())[ElementIndex] = Friction;
		ClothCol->GetElements(ClothCol->GetFabricDensity())[ElementIndex] = Density;
		ClothCol->GetElements(ClothCol->GetFabricDamping())[ElementIndex] = Damping;
		ClothCol->GetElements(ClothCol->GetFabricPressure())[ElementIndex] = Pressure;
		ClothCol->GetElements(ClothCol->GetFabricLayer())[ElementIndex] = Layer;
		ClothCol->GetElements(ClothCol->GetFabricCollisionThickness())[ElementIndex] = CollisionThickness;
	}

	void FCollectionClothFabricFacade::Initialize(const FCollectionClothFabricConstFacade& OtherFabricFacade,
		const float Pressure, const int32 Layer, const float CollisionThickness)
	{
		Initialize( OtherFabricFacade.GetBendingStiffness(), OtherFabricFacade.GetBucklingRatio(), OtherFabricFacade.GetBucklingStiffness(),
			OtherFabricFacade.GetStretchStiffness(), OtherFabricFacade.GetDensity(),
			OtherFabricFacade.GetFriction(), OtherFabricFacade.GetDamping(), Pressure, Layer, CollisionThickness);
	}
	
	void FCollectionClothFabricFacade::Initialize(const FCollectionClothFabricConstFacade& OtherFabricFacade)
	{
		Initialize( OtherFabricFacade.GetBendingStiffness(), OtherFabricFacade.GetBucklingRatio(), OtherFabricFacade.GetBucklingStiffness(),
			OtherFabricFacade.GetStretchStiffness(), OtherFabricFacade.GetDensity(),
			OtherFabricFacade.GetFriction(), OtherFabricFacade.GetDamping(), OtherFabricFacade.GetPressure(), OtherFabricFacade.GetLayer(), OtherFabricFacade.GetCollisionThickness());
	}

	FCollectionClothFabricFacade::FCollectionClothFabricFacade(const TSharedRef<FClothCollection>& ClothCollection, int32 InFabricIndex)
		: FCollectionClothFabricConstFacade(ClothCollection, InFabricIndex)
	{
	}

	void FCollectionClothFabricFacade::Reset()
	{
		SetDefaults();
	}

	void FCollectionClothFabricFacade::SetDefaults()
	{
		Initialize(FDefaultFabric::BendingStiffness, FDefaultFabric::BucklingRatio, FDefaultFabric::BucklingStiffness,
			FDefaultFabric::StretchStiffness, FDefaultFabric::Density, FDefaultFabric::Friction, FDefaultFabric::Damping,
			FDefaultFabric::Pressure, FDefaultFabric::Layer, FDefaultFabric::CollisionThickness);
	}

	TSharedRef<class FClothCollection> FCollectionClothFabricFacade::GetClothCollection()
	{
		return StaticCastSharedRef<class FClothCollection>(ConstCastSharedRef<class FConstClothCollection>(ClothCollection));
	}
}  // End namespace UE::Chaos::ClothAsset
