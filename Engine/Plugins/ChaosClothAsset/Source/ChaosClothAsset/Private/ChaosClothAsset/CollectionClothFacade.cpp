// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/CollectionClothFacade.h"
#include "ChaosClothAsset/ClothCollection.h"
#include "ChaosClothAsset/ClothCollectionGroup.h"
#include "ChaosClothAsset/ClothGeometryTools.h"
#include "Chaos/ChaosArchive.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/UE5MainStreamObjectVersion.h"

namespace UE::Chaos::ClothAsset
{
	FCollectionClothConstFacade::FCollectionClothConstFacade(const TSharedRef<const FManagedArrayCollection>& ManagedArrayCollection)
		: ClothCollection(MakeShared<FConstClothCollection>(ManagedArrayCollection))
	{
	}

	FCollectionClothConstFacade::FCollectionClothConstFacade(const TSharedRef<const FConstClothCollection>& ClothCollection)
		:ClothCollection(ClothCollection)
	{
	}

	bool FCollectionClothConstFacade::IsValid(EClothCollectionExtendedSchemas OptionalSchemas) const
	{
		return ClothCollection->IsValid(OptionalSchemas) && (EnumHasAnyFlags(OptionalSchemas, EClothCollectionExtendedSchemas::CookedOnly) || ClothCollection->GetNumElements(ClothCollectionGroup::Lods) == 1);
	}

	bool FCollectionClothConstFacade::HasValidSimulationData() const
	{
		return IsValid() &&
			GetNumSimPatterns() &&
			GetNumSimVertices2D() &&
			GetNumSimVertices3D() &&
			GetNumSimFaces();
	}

	bool FCollectionClothConstFacade::HasValidRenderData() const
	{
		return IsValid() &&
			GetNumRenderPatterns() &&
			GetNumRenderVertices() &&
			GetNumRenderFaces();
	}

	bool FCollectionClothConstFacade::HasValidData() const
	{
		return IsValid() &&
			GetNumSimPatterns() &&
			GetNumRenderPatterns() &&
			GetNumSimVertices2D() &&
			GetNumSimVertices3D() &&
			GetNumSimFaces() &&
			GetNumRenderVertices() &&
			GetNumRenderFaces();
	}

	uint32 FCollectionClothConstFacade::CalculateTypeHash(bool bIncludeWeightMaps, uint32 PreviousHash) const
	{
		check(IsValid());
		uint32 ResultHash = PreviousHash;

		//~ LOD (single per collection) Group
		ResultHash = HashCombineFast(ResultHash, GetTypeHash(GetPhysicsAssetSoftObjectPathName()));
		ResultHash = HashCombineFast(ResultHash, GetTypeHash(GetSkeletalMeshSoftObjectPathName()));
		ResultHash = HashCombineFast(ResultHash, GetTypeHash(GetReferenceBoneName()));
		ResultHash = HashCombineFast(ResultHash, GetTypeHash(GetCopyAllUserAttributesToSimModel()));

		//~ Solvers Group
		ResultHash = HashCombineFast(ResultHash, GetTypeHash(GetSolverGravity()));
		ResultHash = HashCombineFast(ResultHash, GetTypeHash(GetSolverAirDamping()));
		ResultHash = HashCombineFast(ResultHash, GetTypeHash(GetSolverSubSteps()));
		ResultHash = HashCombineFast(ResultHash, GetTypeHash(GetSolverTimeStep()));

		//~ Seam Stitches Group
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetSeamStitchStart()));
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetSeamStitchEnd()));

		//~ Seam Stitches Group
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetSeamStitch2DEndIndices()));
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetSeamStitch3DIndex()));

		//~ Sim Patterns Group
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetSimVertices2DStart()));
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetSimVertices2DEnd()));
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetSimFacesStart()));
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetSimFacesEnd()));
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetSimPatternFabric()));

		//~ Render Patterns Group
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetRenderVerticesStart()));
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetRenderVerticesEnd()));
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetRenderFacesStart()));
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetRenderFacesEnd()));
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetRenderMaterialSoftObjectPathName()));
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetRenderDeformerNumInfluences()));

		//~ Sim Faces Group
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetSimIndices2D()));
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetSimIndices3D()));

		//~ Sim Vertices 2D Group
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetSimPosition2D()));
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetSimVertex3DLookup()));
		
		//~ Sim Vertices 3D Group
    	ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetSimPosition3D()));
    	ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetSimNormal()));
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetSimBoneIndices()));
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetSimBoneWeights()));
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetTetherKinematicIndex()));
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetTetherReferenceLength()));
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetSimVertex2DLookup()));
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetSeamStitchLookup()));
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetSimCustomResizingBlend()));
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetPreResizedSimPosition3D()));

		//~ Sim Morph Targets Group
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetSimMorphTargetName()));
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetSimMorphTargetVerticesStart()));
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetSimMorphTargetVerticesEnd()));

		//~ Sim Morph Target Vertices Group
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetSimMorphTargetPositionDelta()));
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetSimMorphTargetTangentZDelta()));
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetSimMorphTargetSimVertex3DIndex()));
		
		//~ Fabrics Group
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetFabricBendingStiffness()));
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetFabricBucklingRatio()));
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetFabricBucklingStiffness()));
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetFabricDamping()));
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetFabricDensity()));
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetFabricFriction()));
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetFabricStretchStiffness()));
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetFabricPressure()));
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetFabricLayer()));
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetFabricCollisionThickness()));

		//~ Render Faces Group
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetRenderIndices()));
		
		//~ Render Vertices Group
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetRenderPosition()));
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetRenderNormal()));
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetRenderTangentU()));
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetRenderTangentV()));
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetRenderUVs()));
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetRenderColor()));
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetRenderBoneIndices()));
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetRenderBoneWeights()));
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetRenderDeformerPositionBaryCoordsAndDist()));
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetRenderDeformerNormalBaryCoordsAndDist()));
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetRenderDeformerTangentBaryCoordsAndDist()));
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetRenderDeformerSimIndices3D()));
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetRenderDeformerWeight()));
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetRenderDeformerSkinningBlend()));
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetRenderCustomResizingBlend()));

		//~ Resizing Binding Group
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetCustomResizingRegionSet()));
		ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetCustomResizingRegionType()));

		if (bIncludeWeightMaps)
		{
			ResultHash = CalculateWeightMapTypeHash(ResultHash);
		}
		return ResultHash;
	}

	uint32 FCollectionClothConstFacade::CalculateWeightMapTypeHash(uint32 PreviousHash) const
	{
		uint32 ResultHash = PreviousHash;
		const TArray<FName> WeightMapNames = GetWeightMapNames();
		for (const FName& WeightMapName : WeightMapNames)
		{
			ResultHash = HashCombineFast(ResultHash, GetTypeHash(WeightMapName));
			ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetUserDefinedAttribute<float>(WeightMapName, ClothCollectionGroup::SimVertices3D)));
		}
		return ResultHash;
	}

	template<typename T UE_REQUIRES_DEFINITION(TIsUserAttributeType<T>::Value)>
	uint32 FCollectionClothConstFacade::CalculateUserDefinedAttributesTypeHash(const FName& GroupName, uint32 PreviousHash) const
	{
		uint32 ResultHash = PreviousHash;
		const TArray<FName> AttributesNames = GetUserDefinedAttributeNames<T>(GroupName);
		for (const FName& AttributesName : AttributesNames)
		{
			ResultHash = HashCombineFast(ResultHash, GetTypeHash(AttributesName));
			ResultHash = HashCombineFast(ResultHash, ClothCollection->GetElementsTypeHash(ClothCollection->GetUserDefinedAttribute<T>(AttributesName, GroupName)));
		}
		return ResultHash;
	}
	template CHAOSCLOTHASSET_API uint32 FCollectionClothConstFacade::CalculateUserDefinedAttributesTypeHash<int32>(const FName& GroupName, uint32 PreviousHash) const;
	template CHAOSCLOTHASSET_API uint32 FCollectionClothConstFacade::CalculateUserDefinedAttributesTypeHash<float>(const FName& GroupName, uint32 PreviousHash) const;
	template CHAOSCLOTHASSET_API uint32 FCollectionClothConstFacade::CalculateUserDefinedAttributesTypeHash<FVector3f>(const FName& GroupName, uint32 PreviousHash) const;
	template CHAOSCLOTHASSET_API uint32 FCollectionClothConstFacade::CalculateUserDefinedAttributesTypeHash<TArray<int32>>(const FName& GroupName, uint32 PreviousHash) const;

	const FSoftObjectPath& FCollectionClothConstFacade::GetPhysicsAssetSoftObjectPathName() const
	{
		static const FSoftObjectPath EmptyString;
		return ClothCollection->GetPhysicsAssetSoftObjectPathName() && ClothCollection->GetNumElements(ClothCollectionGroup::Lods) > 0 ? (*ClothCollection->GetPhysicsAssetSoftObjectPathName())[0] : EmptyString;
	}

	const FSoftObjectPath& FCollectionClothConstFacade::GetSkeletalMeshSoftObjectPathName() const
	{
		static const FSoftObjectPath EmptyString;
		return ClothCollection->GetSkeletalMeshSoftObjectPathName() && ClothCollection->GetNumElements(ClothCollectionGroup::Lods) > 0 ? (*ClothCollection->GetSkeletalMeshSoftObjectPathName())[0] : EmptyString;
	}

	FName FCollectionClothConstFacade::GetReferenceBoneName() const
	{
		return ClothCollection->GetReferenceBoneName() && ClothCollection->GetNumElements(ClothCollectionGroup::Lods) > 0 ? (*ClothCollection->GetReferenceBoneName())[0] : NAME_None;
	}

	bool FCollectionClothConstFacade::GetCopyAllUserAttributesToSimModel() const
	{
		return ClothCollection->GetCopyAllUserAttributesToSimModel() && ClothCollection->GetNumElements(ClothCollectionGroup::Lods) > 0 ? (*ClothCollection->GetCopyAllUserAttributesToSimModel())[0] : false;
	}

	bool FCollectionClothConstFacade::HasSolverElement() const
	{
		return ClothCollection->GetNumElements(ClothCollectionGroup::Solvers) == 1;
	}

	const FVector3f& FCollectionClothConstFacade::GetSolverGravity() const
	{
		return (ClothCollection->GetSolverGravity() && HasSolverElement()) ? (*ClothCollection->GetSolverGravity())[0] : FDefaultSolver::Gravity;
	}

	float FCollectionClothConstFacade::GetSolverAirDamping() const
	{
		return (ClothCollection->GetSolverAirDamping() && HasSolverElement()) ? (*ClothCollection->GetSolverAirDamping())[0] : FDefaultSolver::AirDamping;
	}

	int32 FCollectionClothConstFacade::GetSolverSubSteps() const
	{
		return (ClothCollection->GetSolverSubSteps() && HasSolverElement()) ? (*ClothCollection->GetSolverSubSteps())[0] : FDefaultSolver::SubSteps;
	}

	float FCollectionClothConstFacade::GetSolverTimeStep() const
	{
		return (ClothCollection->GetSolverTimeStep() && HasSolverElement()) ? (*ClothCollection->GetSolverTimeStep())[0] : FDefaultSolver::TimeStep;
	}

	int32 FCollectionClothConstFacade::GetNumSimVertices2D() const
	{
		return ClothCollection->GetNumElements(ClothCollectionGroup::SimVertices2D);
	}

	TConstArrayView<FVector2f> FCollectionClothConstFacade::GetSimPosition2D() const
	{
		return ClothCollection->GetElements(ClothCollection->GetSimPosition2D());
	}

	TConstArrayView<int32> FCollectionClothConstFacade::GetSimVertex3DLookup() const
	{
		return ClothCollection->GetElements(ClothCollection->GetSimVertex3DLookup());
	}

	TConstArrayView<int32> FCollectionClothConstFacade::GetSimImportVertexID() const
	{
		return ClothCollection->GetElements(ClothCollection->GetSimImportVertexID());
	}

	int32 FCollectionClothConstFacade::GetNumSimVertices3D() const
	{
		return ClothCollection->GetNumElements(ClothCollectionGroup::SimVertices3D);
	}

	TConstArrayView<FVector3f> FCollectionClothConstFacade::GetSimPosition3D() const
	{
		return ClothCollection->GetElements(ClothCollection->GetSimPosition3D());
	}

	TConstArrayView<FVector3f> FCollectionClothConstFacade::GetPreResizedSimPosition3D() const
	{
		return ClothCollection->GetElements(ClothCollection->GetPreResizedSimPosition3D());
	}

	TConstArrayView<FVector3f> FCollectionClothConstFacade::GetSimNormal() const
	{
		return ClothCollection->GetElements(ClothCollection->GetSimNormal());
	}

	TConstArrayView<TArray<int32>> FCollectionClothConstFacade::GetSimBoneIndices() const
	{
		return ClothCollection->GetElements(ClothCollection->GetSimBoneIndices());
	}

	TConstArrayView<TArray<float>> FCollectionClothConstFacade::GetSimBoneWeights() const
	{
		return ClothCollection->GetElements(ClothCollection->GetSimBoneWeights());
	}

	TConstArrayView<TArray<int32>> FCollectionClothConstFacade::GetTetherKinematicIndex() const
	{
		return ClothCollection->GetElements(ClothCollection->GetTetherKinematicIndex());
	}

	TConstArrayView<TArray<float>> FCollectionClothConstFacade::GetTetherReferenceLength() const
	{
		return ClothCollection->GetElements(ClothCollection->GetTetherReferenceLength());
	}

	TConstArrayView<TArray<int32>> FCollectionClothConstFacade::GetSimVertex2DLookup() const
	{
		return ClothCollection->GetElements(ClothCollection->GetSimVertex2DLookup());
	}

	TConstArrayView<TArray<int32>> FCollectionClothConstFacade::GetSeamStitchLookup() const
	{
		return ClothCollection->GetElements(ClothCollection->GetSeamStitchLookup());
	}

	TConstArrayView<float> FCollectionClothConstFacade::GetSimCustomResizingBlend() const
	{
		return ClothCollection->GetElements(ClothCollection->GetSimCustomResizingBlend());
	}

	int32 FCollectionClothConstFacade::GetNumSimFaces() const
	{
		return ClothCollection->GetNumElements(ClothCollectionGroup::SimFaces);
	}

	TConstArrayView<FIntVector3> FCollectionClothConstFacade::GetSimIndices2D() const
	{
		return ClothCollection->GetElements(ClothCollection->GetSimIndices2D());
	}

	TConstArrayView<FIntVector3> FCollectionClothConstFacade::GetSimIndices3D() const
	{
		return ClothCollection->GetElements(ClothCollection->GetSimIndices3D());
	}

	int32 FCollectionClothConstFacade::GetNumSimPatterns() const
	{
		return ClothCollection->GetNumElements(ClothCollectionGroup::SimPatterns);
	}

	FCollectionClothSimPatternConstFacade FCollectionClothConstFacade::GetSimPattern(int32 PatternIndex) const
	{
		return FCollectionClothSimPatternConstFacade(ClothCollection, PatternIndex);
	}

	int32 FCollectionClothConstFacade::FindSimPatternByVertex2D(int32 Vertex2DIndex) const
	{
		return ClothCollection->GetArrayIndexForContainedElement(ClothCollection->GetSimVertices2DStart(), ClothCollection->GetSimVertices2DEnd(), Vertex2DIndex);
	}

	int32 FCollectionClothConstFacade::FindSimPatternByFaceIndex(int32 FaceIndex) const
	{
		return ClothCollection->GetArrayIndexForContainedElement(ClothCollection->GetSimFacesStart(), ClothCollection->GetSimFacesEnd(), FaceIndex);
	}

	int32 FCollectionClothConstFacade::GetNumSimMorphTargets() const
	{
		return ClothCollection->GetNumElements(ClothCollectionGroup::SimMorphTargets);
	}

	int32 FCollectionClothConstFacade::FindSimMorphTargetIndexByName(const FString& MorphTargetName) const
	{
		TConstArrayView<FString> SimMorphTargetNames = GetSimMorphTargetName();
		for(int32 Index = 0; Index < SimMorphTargetNames.Num(); ++Index)
		{
			if (SimMorphTargetNames[Index].Equals(MorphTargetName))
			{
				return Index;
			}
		}
		return INDEX_NONE;
	}

	FCollectionClothSimMorphTargetConstFacade FCollectionClothConstFacade::GetSimMorphTarget(int32 MorphTargetIndex) const
	{
		return FCollectionClothSimMorphTargetConstFacade(ClothCollection, MorphTargetIndex);
	}

	TConstArrayView<FString> FCollectionClothConstFacade::GetSimMorphTargetName() const
	{
		return ClothCollection->GetElements(ClothCollection->GetSimMorphTargetName());
	}

	int32 FCollectionClothConstFacade::GetNumSimMorphTargetVertices() const
	{
		return ClothCollection->GetNumElements(ClothCollectionGroup::SimMorphTargetVertices);
	}

	TConstArrayView<FVector3f> FCollectionClothConstFacade::GetSimMorphTargetPositionDelta() const
	{
		return ClothCollection->GetElements(ClothCollection->GetSimMorphTargetPositionDelta());
	}

	TConstArrayView<FVector3f> FCollectionClothConstFacade::GetSimMorphTargetTangentZDelta() const
	{
		return ClothCollection->GetElements(ClothCollection->GetSimMorphTargetTangentZDelta());
	}

	TConstArrayView<int32> FCollectionClothConstFacade::GetSimMorphTargetSimVertex3DIndex() const
	{
		return ClothCollection->GetElements(ClothCollection->GetSimMorphTargetSimVertex3DIndex());
	}

	int32 FCollectionClothConstFacade::GetNumSimAccessoryMeshes() const
	{
		return ClothCollection->GetNumElements(ClothCollectionGroup::SimAccessoryMeshes);
	}

	int32 FCollectionClothConstFacade::FindSimAccessoryMeshIndexByName(const FName& AccessoryMeshName) const
	{
		TConstArrayView<FName> SimAccessoryMeshNames = GetSimAccessoryMeshName();
		for (int32 Index = 0; Index < SimAccessoryMeshNames.Num(); ++Index)
		{
			if (SimAccessoryMeshNames[Index] == AccessoryMeshName)
			{
				return Index;
			}
		}
		return INDEX_NONE;
	}

	FCollectionClothSimAccessoryMeshConstFacade FCollectionClothConstFacade::GetSimAccessoryMesh(int32 MeshIndex) const
	{
		return FCollectionClothSimAccessoryMeshConstFacade(ClothCollection, MeshIndex);
	}

	TConstArrayView<FName> FCollectionClothConstFacade::GetSimAccessoryMeshName() const
	{
		return ClothCollection->GetElements(ClothCollection->GetSimAccessoryMeshName());
	}

	TConstArrayView<FName> FCollectionClothConstFacade::GetSimAccessoryMeshPosition3DAttribute() const
	{
		return ClothCollection->GetElements(ClothCollection->GetSimAccessoryMeshPosition3DAttribute());
	}

	TConstArrayView<FName> FCollectionClothConstFacade::GetSimAccessoryMeshNormalAttribute() const
	{
		return ClothCollection->GetElements(ClothCollection->GetSimAccessoryMeshNormalAttribute());
	}

	TConstArrayView<FName> FCollectionClothConstFacade::GetSimAccessoryMeshBoneIndicesAttribute() const
	{
		return ClothCollection->GetElements(ClothCollection->GetSimAccessoryMeshBoneIndicesAttribute());
	}

	TConstArrayView<FName> FCollectionClothConstFacade::GetSimAccessoryMeshBoneWeightsAttribute() const
	{
		return ClothCollection->GetElements(ClothCollection->GetSimAccessoryMeshBoneWeightsAttribute());
	}

	int32 FCollectionClothConstFacade::GetNumRenderPatterns() const
	{
		return ClothCollection->GetNumElements(ClothCollectionGroup::RenderPatterns);
	}

	FCollectionClothRenderPatternConstFacade FCollectionClothConstFacade::GetRenderPattern(int32 PatternIndex) const
	{
		return FCollectionClothRenderPatternConstFacade(ClothCollection, PatternIndex);
	}

	TConstArrayView<int32> FCollectionClothConstFacade::GetRenderDeformerNumInfluences() const
	{
		return ClothCollection->GetElements(ClothCollection->GetRenderDeformerNumInfluences());
	}

	TConstArrayView<FSoftObjectPath> FCollectionClothConstFacade::GetRenderMaterialSoftObjectPathName() const
	{
		return ClothCollection->GetElements(ClothCollection->GetRenderMaterialSoftObjectPathName());
	}

	int32 FCollectionClothConstFacade::FindRenderPatternByVertex(int32 VertexIndex) const
	{
		return ClothCollection->GetArrayIndexForContainedElement(ClothCollection->GetRenderVerticesStart(), ClothCollection->GetRenderVerticesEnd(), VertexIndex);
	}

	int32 FCollectionClothConstFacade::FindRenderPatternByFaceIndex(int32 FaceIndex) const
	{
		return ClothCollection->GetArrayIndexForContainedElement(ClothCollection->GetRenderFacesStart(), ClothCollection->GetRenderFacesEnd(), FaceIndex);
	}

	int32 FCollectionClothConstFacade::GetNumSeams() const
	{
		return ClothCollection->GetNumElements(ClothCollectionGroup::Seams);
	}

	FCollectionClothSeamConstFacade FCollectionClothConstFacade::GetSeam(int32 SeamIndex) const
	{
		return FCollectionClothSeamConstFacade(ClothCollection, SeamIndex);
	}

	int32 FCollectionClothConstFacade::GetNumFabrics() const
	{
		return ClothCollection->GetNumElements(ClothCollectionGroup::Fabrics);
	}

	FCollectionClothFabricConstFacade FCollectionClothConstFacade::GetFabric(int32 FabricIndex) const
	{
		return FCollectionClothFabricConstFacade(ClothCollection, FabricIndex);
	}

	int32 FCollectionClothConstFacade::GetNumRenderVertices() const
	{
		return ClothCollection->GetNumElements(ClothCollectionGroup::RenderVertices);
	}

	TConstArrayView<FVector3f> FCollectionClothConstFacade::GetRenderPosition() const
	{
		return ClothCollection->GetElements(ClothCollection->GetRenderPosition());
	}

	TConstArrayView<FVector3f> FCollectionClothConstFacade::GetRenderNormal() const
	{
		return ClothCollection->GetElements(ClothCollection->GetRenderNormal());
	}

	TConstArrayView<FVector3f> FCollectionClothConstFacade::GetRenderTangentU() const
	{
		return ClothCollection->GetElements(ClothCollection->GetRenderTangentU());
	}

	TConstArrayView<FVector3f> FCollectionClothConstFacade::GetRenderTangentV() const
	{
		return ClothCollection->GetElements(ClothCollection->GetRenderTangentV());
	}

	TConstArrayView<TArray<FVector2f>> FCollectionClothConstFacade::GetRenderUVs() const
	{
		return ClothCollection->GetElements(ClothCollection->GetRenderUVs());
	}

	TConstArrayView<FLinearColor> FCollectionClothConstFacade::GetRenderColor() const
	{
		return ClothCollection->GetElements(ClothCollection->GetRenderColor());
	}

	TConstArrayView<TArray<int32>> FCollectionClothConstFacade::GetRenderBoneIndices() const
	{
		return ClothCollection->GetElements(ClothCollection->GetRenderBoneIndices());
	}

	TConstArrayView<TArray<float>> FCollectionClothConstFacade::GetRenderBoneWeights() const
	{
		return ClothCollection->GetElements(ClothCollection->GetRenderBoneWeights());
	}

	TConstArrayView<TArray<FVector4f>> FCollectionClothConstFacade::GetRenderDeformerPositionBaryCoordsAndDist() const
	{
		return ClothCollection->GetElements(ClothCollection->GetRenderDeformerPositionBaryCoordsAndDist());
	}

	TConstArrayView<TArray<FVector4f>> FCollectionClothConstFacade::GetRenderDeformerNormalBaryCoordsAndDist() const
	{
		return ClothCollection->GetElements(ClothCollection->GetRenderDeformerNormalBaryCoordsAndDist());
	}

	TConstArrayView<TArray<FVector4f>> FCollectionClothConstFacade::GetRenderDeformerTangentBaryCoordsAndDist() const
	{
		return ClothCollection->GetElements(ClothCollection->GetRenderDeformerTangentBaryCoordsAndDist());
	}

	TConstArrayView<TArray<FIntVector3>> FCollectionClothConstFacade::GetRenderDeformerSimIndices3D() const
	{
		return ClothCollection->GetElements(ClothCollection->GetRenderDeformerSimIndices3D());
	}

	TConstArrayView<TArray<float>> FCollectionClothConstFacade::GetRenderDeformerWeight() const
	{
		return ClothCollection->GetElements(ClothCollection->GetRenderDeformerWeight());
	}

	TConstArrayView<float> FCollectionClothConstFacade::GetRenderDeformerSkinningBlend() const
	{
		return ClothCollection->GetElements(ClothCollection->GetRenderDeformerSkinningBlend());
	}

	TConstArrayView<float> FCollectionClothConstFacade::GetRenderCustomResizingBlend() const
	{
		return ClothCollection->GetElements(ClothCollection->GetRenderCustomResizingBlend());
	}

	int32 FCollectionClothConstFacade::GetNumRenderFaces() const
	{
		return ClothCollection->GetNumElements(ClothCollectionGroup::RenderFaces);
	}

	TConstArrayView<FIntVector3> FCollectionClothConstFacade::GetRenderIndices() const
	{
		return ClothCollection->GetElements(ClothCollection->GetRenderIndices());
	}

	int32 FCollectionClothConstFacade::GetNumCustomResizingRegions() const
	{
		return ClothCollection->GetNumElements(ClothCollectionGroup::CustomResizingRegions);
	}

	TConstArrayView<FString> FCollectionClothConstFacade::GetCustomResizingRegionSet() const
	{
		return ClothCollection->GetElements(ClothCollection->GetCustomResizingRegionSet());
	}

	TConstArrayView<int32> FCollectionClothConstFacade::GetCustomResizingRegionType() const
	{
		return ClothCollection->GetElements(ClothCollection->GetCustomResizingRegionType());
	}

	bool FCollectionClothConstFacade::HasWeightMap(const FName& Name) const
	{
		return ClothCollection->HasUserDefinedAttribute<float>(Name, ClothCollectionGroup::SimVertices3D);
	}

	TArray<FName> FCollectionClothConstFacade::GetWeightMapNames() const
	{
		return ClothCollection->GetUserDefinedAttributeNames<float>(ClothCollectionGroup::SimVertices3D);
	}

	TConstArrayView<float> FCollectionClothConstFacade::GetWeightMap(const FName& Name) const
	{
		return ClothCollection->GetElements(ClothCollection->GetUserDefinedAttribute<float>(Name, ClothCollectionGroup::SimVertices3D));
	}


	template<typename T UE_REQUIRES_DEFINITION(TIsUserAttributeType<T>::Value)>
	bool FCollectionClothConstFacade::HasUserDefinedAttribute(const FName& Name, const FName& GroupName) const
	{
		return ClothCollection->HasUserDefinedAttribute<T>(Name, GroupName);
	}
	template CHAOSCLOTHASSET_API bool FCollectionClothConstFacade::HasUserDefinedAttribute<int32>(const FName& Name, const FName& GroupName) const;
	template CHAOSCLOTHASSET_API bool FCollectionClothConstFacade::HasUserDefinedAttribute<float>(const FName& Name, const FName& GroupName) const;
	template CHAOSCLOTHASSET_API bool FCollectionClothConstFacade::HasUserDefinedAttribute<FVector3f>(const FName& Name, const FName& GroupName) const;
	template CHAOSCLOTHASSET_API bool FCollectionClothConstFacade::HasUserDefinedAttribute<TArray<int32>>(const FName& Name, const FName& GroupName) const;

	template<typename T UE_REQUIRES_DEFINITION(TIsUserAttributeType<T>::Value)>
	TArray<FName> FCollectionClothConstFacade::GetUserDefinedAttributeNames(const FName& GroupName) const
	{
		return ClothCollection->GetUserDefinedAttributeNames<T>(GroupName);
	}
	template CHAOSCLOTHASSET_API TArray<FName> FCollectionClothConstFacade::GetUserDefinedAttributeNames<int32>(const FName& GroupName) const;
	template CHAOSCLOTHASSET_API TArray<FName> FCollectionClothConstFacade::GetUserDefinedAttributeNames<float>(const FName& GroupName) const;
	template CHAOSCLOTHASSET_API TArray<FName> FCollectionClothConstFacade::GetUserDefinedAttributeNames<FVector3f>(const FName& GroupName) const;
	template CHAOSCLOTHASSET_API TArray<FName> FCollectionClothConstFacade::GetUserDefinedAttributeNames<TArray<int32>>(const FName& GroupName) const;

	template<typename T UE_REQUIRES_DEFINITION(TIsUserAttributeType<T>::Value)>
	TConstArrayView<T> FCollectionClothConstFacade::GetUserDefinedAttribute(const FName& Name, const FName& GroupName) const
	{
		return ClothCollection->GetElements(ClothCollection->GetUserDefinedAttribute<T>(Name, GroupName));
	}
	template CHAOSCLOTHASSET_API TConstArrayView<int32> FCollectionClothConstFacade::GetUserDefinedAttribute<int32>(const FName& Name, const FName& GroupName) const;
	template CHAOSCLOTHASSET_API TConstArrayView<float> FCollectionClothConstFacade::GetUserDefinedAttribute<float>(const FName& Name, const FName& GroupName) const;
	template CHAOSCLOTHASSET_API TConstArrayView<FVector3f> FCollectionClothConstFacade::GetUserDefinedAttribute<FVector3f>(const FName& Name, const FName& GroupName) const;
	template CHAOSCLOTHASSET_API TConstArrayView<TArray<int32>> FCollectionClothConstFacade::GetUserDefinedAttribute<TArray<int32>>(const FName& Name, const FName& GroupName) const;

	bool FCollectionClothConstFacade::IsValidClothCollectionGroupName(const FName& GroupName)
	{
		return FClothCollection::IsValidClothCollectionGroupName(GroupName);
	}

	TArray<FName> FCollectionClothConstFacade::GetValidClothCollectionGroupName()
	{
		return FClothCollection::GetValidClothCollectionGroupName();
	}

	void FCollectionClothConstFacade::BuildSimulationMesh(TArray<FVector3f>& Positions, TArray<FVector3f>& Normals, TArray<uint32>& Indices, TArray<FVector2f>& PatternsPositions, TArray<uint32>& PatternsIndices,
		TArray<uint32>& PatternToWeldedIndices, TArray<TArray<int32>>* OptionalWeldedToPatternIndices) const
	{
		Positions = GetSimPosition3D();
		Normals = GetSimNormal();
		PatternsPositions = GetSimPosition2D();
		PatternToWeldedIndices = GetSimVertex3DLookup();
		if (OptionalWeldedToPatternIndices)
		{
			*OptionalWeldedToPatternIndices = GetSimVertex2DLookup();
		}

		for (const FIntVector3& Face : GetSimIndices2D())
		{
			PatternsIndices.Add(Face[0]);
			PatternsIndices.Add(Face[1]);
			PatternsIndices.Add(Face[2]);
		}

		// It's possible that welding created degenerate 3D triangles. Remove them when copying over. Keep 2D and 3D triangles in sync
		PatternsIndices.Reset(GetNumSimFaces() * 3);
		Indices.Reset(GetNumSimFaces() * 3);
		for(int32 FaceIdx = 0; FaceIdx < GetNumSimFaces(); ++FaceIdx)
		{
			const FIntVector3 Face3D = GetSimIndices3D()[FaceIdx];

			if (Face3D[0] != Face3D[1] && Face3D[0] != Face3D[2] && Face3D[1] != Face3D[2])
			{
				Indices.Add(Face3D[0]);
				Indices.Add(Face3D[1]);
				Indices.Add(Face3D[2]);

				const FIntVector3 Face2D = GetSimIndices2D()[FaceIdx];
				PatternsIndices.Add(Face2D[0]);
				PatternsIndices.Add(Face2D[1]);
				PatternsIndices.Add(Face2D[2]);
			}
		}
	}

	FCollectionClothFacade::FCollectionClothFacade(const TSharedRef<FManagedArrayCollection>& ManagedArrayCollection)
		: FCollectionClothConstFacade(MakeShared<FClothCollection>(ManagedArrayCollection))
	{
	}

	FCollectionClothFacade::FCollectionClothFacade(const TSharedRef<FClothCollection>& InClothCollection)
		: FCollectionClothConstFacade(InClothCollection)
	{
	}

	void FCollectionClothFacade::DefineSchema(EClothCollectionExtendedSchemas OptionalSchemas)
	{
		GetClothCollection()->DefineSchema(OptionalSchemas);
		SetDefaults();
	}

	void FCollectionClothFacade::PostSerialize(const FArchive& Ar)
	{
		GetClothCollection()->PostSerialize(Ar);
	}

	void FCollectionClothFacade::Reset()
	{
		GetClothCollection()->SetNumElements(0, ClothCollectionGroup::Lods);  // Reset all Lods data
		GetClothCollection()->SetNumElements(1, ClothCollectionGroup::Lods);  //
		RemoveAllSimVertices3D();
		SetNumSimPatterns(0);
		SetNumRenderPatterns(0);
		SetNumSeams(0); // Do this after removing SimVertices3D and SimPatterns. Otherwise, Seams will do a bunch of unnecessary work to unseam stuff.
		SetNumFabrics(0); 
		if (HasSolverElement())
		{
			GetClothCollection()->SetNumElements(0, ClothCollectionGroup::Solvers);  // Reset all Solvers data
			GetClothCollection()->SetNumElements(1, ClothCollectionGroup::Solvers);  //
		}
		SetNumSimMorphTargets(0);
		SetNumCustomResizingRegions(0);
		SetNumSimAccessoryMeshes(0);
	}

	void FCollectionClothFacade::Initialize(const FCollectionClothConstFacade& Other)
	{
		Reset();
		
		// Solvers Group
		if (Other.IsValid(EClothCollectionExtendedSchemas::Solvers) && Other.HasSolverElement())
		{
			SetSolverGravity(Other.GetSolverGravity());
			SetSolverAirDamping(Other.GetSolverAirDamping());
			SetSolverSubSteps(Other.GetSolverSubSteps());
			SetSolverTimeStep(Other.GetSolverTimeStep());
		}
		
		Append(Other);
	}

	void FCollectionClothFacade::Append(const FCollectionClothConstFacade& Other)
	{

#if DO_ENSURE
		for (int32 SeamIndex = 0; SeamIndex < GetNumSeams(); ++SeamIndex)
		{
			GetSeam(SeamIndex).ValidateSeam();
		}
#endif

		// LODs Group 
		// Keep original data unless our data is empty. Then take Other's data.
		if (GetPhysicsAssetSoftObjectPathName().IsNull())
		{
			SetPhysicsAssetSoftObjectPathName(Other.GetPhysicsAssetSoftObjectPathName());
		}
		if (GetSkeletalMeshSoftObjectPathName().IsNull())
		{
			SetSkeletalMeshSoftObjectPathName(Other.GetSkeletalMeshSoftObjectPathName());
		}
		if (GetReferenceBoneName() == NAME_None)
		{
			SetReferenceBoneName(Other.GetReferenceBoneName());
		}
		if (GetCopyAllUserAttributesToSimModel() == false)
		{
			SetCopyAllUserAttributesToSimModel(Other.GetCopyAllUserAttributesToSimModel());
		}

		const bool bOtherHasResizingData = Other.IsValid(EClothCollectionExtendedSchemas::Resizing);
		if (bOtherHasResizingData)
		{
			DefineSchema(EClothCollectionExtendedSchemas::Resizing);
		}

		// Very important order of operations to ensure indices don't get messed up:
		// 1) Append 3D Vertices, but don't set 2D Lookups or SeamStitch Lookups because those indices don't exist yet.
		// 2) Append Sim Patterns (includes 2D Vertices, have 3D dependency)
		// 3) Append Seams (have 2D and 3D dependency)
		// 4) Append 2DLookups (2D dep)  and SeamStitchLookups (SeamStitch Dep)

		// Sim Vertices 3D Group
		const int32 StartNumSimVertices3D = GetNumSimVertices3D();
		const int32 OtherNumSimVertices3D = Other.GetNumSimVertices3D();
		GetClothCollection()->SetNumElements(StartNumSimVertices3D + OtherNumSimVertices3D, ClothCollectionGroup::SimVertices3D);
		FClothCollection::CopyArrayViewData(GetSimPosition3D().Right(OtherNumSimVertices3D), Other.GetSimPosition3D());
		FClothCollection::CopyArrayViewData(GetSimNormal().Right(OtherNumSimVertices3D), Other.GetSimNormal());
		FClothCollection::CopyArrayViewData(GetSimBoneIndices().Right(OtherNumSimVertices3D), Other.GetSimBoneIndices());
		FClothCollection::CopyArrayViewData(GetSimBoneWeights().Right(OtherNumSimVertices3D), Other.GetSimBoneWeights());
		FClothCollection::CopyArrayViewDataAndApplyOffset(GetTetherKinematicIndex().Right(OtherNumSimVertices3D), Other.GetTetherKinematicIndex(), StartNumSimVertices3D);
		FClothCollection::CopyArrayViewData(GetTetherReferenceLength().Right(OtherNumSimVertices3D), Other.GetTetherReferenceLength());
		if (bOtherHasResizingData)
		{
			FClothCollection::CopyArrayViewData(GetSimCustomResizingBlend().Right(OtherNumSimVertices3D), Other.GetSimCustomResizingBlend());
			FClothCollection::CopyArrayViewData(GetPreResizedSimPosition3D().Right(OtherNumSimVertices3D), Other.GetPreResizedSimPosition3D());
		}

		// Fabrics Group
		const int32 StartNumFabrics = GetNumFabrics();
		const int32 OtherNumFabrics = Other.GetNumFabrics();
		SetNumFabrics(StartNumFabrics + OtherNumFabrics);
		for (int32 FabricIndex = 0; FabricIndex < OtherNumFabrics; ++FabricIndex)
		{
			GetFabric(FabricIndex + StartNumFabrics).Initialize(Other.GetFabric(FabricIndex));
		}

		// Sim Patterns Group
		const bool bHadImportData = IsValid(EClothCollectionExtendedSchemas::Import);
		const int32 StartNumSimVertices2D = GetNumSimVertices2D();
		const int32 StartNumSimPatterns = GetNumSimPatterns();
		const int32 OtherNumSimPatterns = Other.GetNumSimPatterns();
		SetNumSimPatterns(StartNumSimPatterns + OtherNumSimPatterns);
		for (int32 PatternIndex = 0; PatternIndex < OtherNumSimPatterns; ++PatternIndex)
		{
			GetSimPattern(StartNumSimPatterns + PatternIndex).Initialize(Other.GetSimPattern(PatternIndex), StartNumSimVertices3D, StartNumFabrics);
		}
		const bool bOtherHasImportData = Other.IsValid(EClothCollectionExtendedSchemas::Import);
		if (bOtherHasImportData && !bHadImportData)
		{
			DefineSchema(EClothCollectionExtendedSchemas::Import);
			TArrayView<int32> ImportVertexID = GetSimImportVertexID();
			for (int32 Index = 0; Index < StartNumSimVertices2D; ++Index)
			{
				ImportVertexID[Index] = INDEX_NONE;
			}
		}

		// Seams Group
		const int32 StartNumSeamStitches = ClothCollection->GetNumElements(ClothCollectionGroup::SeamStitches);
		const int32 StartNumSeams = GetNumSeams();
		const int32 OtherNumSeams = Other.GetNumSeams();
		SetNumSeams(StartNumSeams + OtherNumSeams);
		for (int32 SeamIndex = 0; SeamIndex < OtherNumSeams; ++SeamIndex)
		{
			Other.GetSeam(SeamIndex).ValidateSeam();
			GetSeam(SeamIndex + StartNumSeams).Initialize(Other.GetSeam(SeamIndex), StartNumSimVertices2D, StartNumSimVertices3D);
		}
		
		// Sim Vertices 3D Group (lookups)
		FClothCollection::CopyArrayViewDataAndApplyOffset(
			GetClothCollection()->GetElements(GetClothCollection()->GetSimVertex2DLookup()).Right(OtherNumSimVertices3D), 
			Other.GetSimVertex2DLookup(), StartNumSimVertices2D);
		FClothCollection::CopyArrayViewDataAndApplyOffset(
			GetClothCollection()->GetElements(GetClothCollection()->GetSeamStitchLookup()).Right(OtherNumSimVertices3D), 
			Other.GetSeamStitchLookup(), StartNumSeamStitches);

#if DO_ENSURE
		for (int32 SeamIndex = 0; SeamIndex < OtherNumSeams; ++SeamIndex)
		{
			GetSeam(SeamIndex + StartNumSeams).ValidateSeam();
		}
#endif

		// Sim Morph Targets Group
		const int32 StartNumSimMorphTargets = GetNumSimMorphTargets();
		const int32 OtherNumSimMorphTargets = Other.GetNumSimMorphTargets();
		SetNumSimMorphTargets(StartNumSimMorphTargets + OtherNumSimMorphTargets);
		for (int32 TargetIndex = 0; TargetIndex < OtherNumSimMorphTargets; ++TargetIndex)
		{
			GetSimMorphTarget(TargetIndex + StartNumSimMorphTargets).Initialize(Other.GetSimMorphTarget(TargetIndex), StartNumSimVertices3D);
		}

		// Render Patterns Group
		const int32 StartNumRenderPatterns = GetNumRenderPatterns();
		const int32 OtherNumRenderPatterns = Other.GetNumRenderPatterns();
		SetNumRenderPatterns(StartNumRenderPatterns + OtherNumRenderPatterns);
		for (int32 PatternIndex = 0; PatternIndex < OtherNumRenderPatterns; ++PatternIndex)
		{
			GetRenderPattern(PatternIndex + StartNumRenderPatterns).Initialize(Other.GetRenderPattern(PatternIndex), StartNumSimVertices3D);
		}

		// Resizing Binding Group
		if (bOtherHasResizingData)
		{
			const int32 StartNumResizingBindingGroups = GetNumCustomResizingRegions();
			const int32 OtherNumResizingBindingGroups = Other.GetNumCustomResizingRegions();
			SetNumCustomResizingRegions(StartNumResizingBindingGroups + OtherNumResizingBindingGroups);
			FClothCollection::CopyArrayViewData(GetCustomResizingRegionSet().Right(OtherNumResizingBindingGroups), Other.GetCustomResizingRegionSet());
			FClothCollection::CopyArrayViewData(GetCustomResizingRegionType().Right(OtherNumResizingBindingGroups), Other.GetCustomResizingRegionType());
		}

		// Sim Accessory Meshes
		// Merge accessory meshes with the same name.
		// Fill in with existing sim mesh data when an accessory mesh exists in one end of the merge but not the other.
		const int32 StartNumAccessoryMeshes = GetNumSimAccessoryMeshes();
		const int32 OtherNumAccessoryMeshes = Other.GetNumSimAccessoryMeshes();
		TConstArrayView<FName> OtherSimAccessoryMeshNames = Other.GetSimAccessoryMeshName();
		for (int32 ExistingMeshIndex = 0; ExistingMeshIndex < StartNumAccessoryMeshes; ++ExistingMeshIndex)
		{
			FCollectionClothSimAccessoryMeshFacade MeshFacade = GetSimAccessoryMesh(ExistingMeshIndex);
			const int32 OtherMatchingMeshIndex = OtherSimAccessoryMeshNames.IndexOfByKey(MeshFacade.GetSimAccessoryMeshName());
			if (OtherMatchingMeshIndex != INDEX_NONE)
			{
				FCollectionClothSimAccessoryMeshConstFacade OtherMeshFacade = Other.GetSimAccessoryMesh(OtherMatchingMeshIndex);
				FClothCollection::CopyArrayViewData(MeshFacade.GetSimAccessoryMeshPosition3D().Right(OtherNumSimVertices3D), OtherMeshFacade.GetSimAccessoryMeshPosition3D());
				FClothCollection::CopyArrayViewData(MeshFacade.GetSimAccessoryMeshNormal().Right(OtherNumSimVertices3D), OtherMeshFacade.GetSimAccessoryMeshNormal());
				FClothCollection::CopyArrayViewData(MeshFacade.GetSimAccessoryMeshBoneIndices().Right(OtherNumSimVertices3D), OtherMeshFacade.GetSimAccessoryMeshBoneIndices());
				FClothCollection::CopyArrayViewData(MeshFacade.GetSimAccessoryMeshBoneWeights().Right(OtherNumSimVertices3D), OtherMeshFacade.GetSimAccessoryMeshBoneWeights());
			}
			else
			{
				FClothCollection::CopyArrayViewData(MeshFacade.GetSimAccessoryMeshPosition3D().Right(OtherNumSimVertices3D), Other.GetSimPosition3D());
				FClothCollection::CopyArrayViewData(MeshFacade.GetSimAccessoryMeshNormal().Right(OtherNumSimVertices3D), Other.GetSimNormal());
				FClothCollection::CopyArrayViewData(MeshFacade.GetSimAccessoryMeshBoneIndices().Right(OtherNumSimVertices3D), Other.GetSimBoneIndices());
				FClothCollection::CopyArrayViewData(MeshFacade.GetSimAccessoryMeshBoneWeights().Right(OtherNumSimVertices3D), Other.GetSimBoneWeights());
			}
		}
		for (int32 OtherMeshIndex = 0; OtherMeshIndex < OtherNumAccessoryMeshes; ++OtherMeshIndex)
		{
			FCollectionClothSimAccessoryMeshConstFacade OtherMeshFacade = Other.GetSimAccessoryMesh(OtherMeshIndex);
			const int32 MatchingMeshIndex = GetSimAccessoryMeshName().IndexOfByKey(OtherMeshFacade.GetSimAccessoryMeshName());
			if (MatchingMeshIndex == INDEX_NONE)
			{
				FCollectionClothSimAccessoryMeshFacade MeshFacade = AddGetSimAccessoryMesh();
				// Initialize with sim mesh data and then copy over other accessory mesh data.
				MeshFacade.Initialize(OtherMeshFacade.GetSimAccessoryMeshName(), GetSimPosition3D(), GetSimNormal(), GetSimBoneIndices(), GetSimBoneWeights());
				FClothCollection::CopyArrayViewData(MeshFacade.GetSimAccessoryMeshPosition3D().Right(OtherNumSimVertices3D), OtherMeshFacade.GetSimAccessoryMeshPosition3D());
				FClothCollection::CopyArrayViewData(MeshFacade.GetSimAccessoryMeshNormal().Right(OtherNumSimVertices3D), OtherMeshFacade.GetSimAccessoryMeshNormal());
				FClothCollection::CopyArrayViewData(MeshFacade.GetSimAccessoryMeshBoneIndices().Right(OtherNumSimVertices3D), OtherMeshFacade.GetSimAccessoryMeshBoneIndices());
				FClothCollection::CopyArrayViewData(MeshFacade.GetSimAccessoryMeshBoneWeights().Right(OtherNumSimVertices3D), OtherMeshFacade.GetSimAccessoryMeshBoneWeights());				

			} // Else case handled above
		}

		// Weight maps
		const TArray<FName> WeightMapNames = Other.GetWeightMapNames();
		for (const FName& WeightMapName : WeightMapNames)
		{
			AddWeightMap(WeightMapName);
			FClothCollection::CopyArrayViewData(GetWeightMap(WeightMapName).Right(OtherNumSimVertices3D), Other.GetWeightMap(WeightMapName));
		}

		// Copy user defined attributes
		static const TArray<FName> ClothCollectionGroups =
		{
			ClothCollectionGroup::SimFaces,  // Face int maps (self collision layers)
			ClothCollectionGroup::RenderPatterns,  // RecomputeTangents
			ClothCollectionGroup::RenderVertices  
		};

		for (const FName& Group : ClothCollectionGroups)
		{
			const int32 OtherNumElements = Other.ClothCollection->GetNumElements(Group);

			const TArray<FName> IntAttributeNames = Other.GetUserDefinedAttributeNames<int32>(Group);
			for (const FName& AttributeName : IntAttributeNames)
			{
				AddUserDefinedAttribute<int32>(AttributeName, Group);
				FClothCollection::CopyArrayViewData(GetUserDefinedAttribute<int32>(AttributeName, Group).Right(OtherNumElements), Other.GetUserDefinedAttribute<int32>(AttributeName, Group));
			}
			const TArray<FName> FloatAttributeNames = Other.GetUserDefinedAttributeNames<float>(Group);
			for (const FName& AttributeName : FloatAttributeNames)
			{
				AddUserDefinedAttribute<float>(AttributeName, Group);
				FClothCollection::CopyArrayViewData(GetUserDefinedAttribute<float>(AttributeName, Group).Right(OtherNumElements), Other.GetUserDefinedAttribute<float>(AttributeName, Group));
			}
			const TArray<FName> VectorAttributeNames = Other.GetUserDefinedAttributeNames<FVector3f>(Group);
			for (const FName& AttributeName : VectorAttributeNames)
			{
				AddUserDefinedAttribute<FVector3f>(AttributeName, Group);
				FClothCollection::CopyArrayViewData(GetUserDefinedAttribute<FVector3f>(AttributeName, Group).Right(OtherNumElements), Other.GetUserDefinedAttribute<FVector3f>(AttributeName, Group));
			}
		}
	}

	void FCollectionClothFacade::InitializeCookedOnly(const FCollectionClothConstFacade& Other)
	{
		constexpr bool bCookedOnly = true;
		SetNumSimMorphTargets(0, bCookedOnly);

		// Sim Morph Targets Group
		const int32 OtherNumSimMorphTargets = Other.GetNumSimMorphTargets();
		SetNumSimMorphTargets(OtherNumSimMorphTargets, bCookedOnly);
		for (int32 TargetIndex = 0; TargetIndex < OtherNumSimMorphTargets; ++TargetIndex)
		{
			GetSimMorphTarget(TargetIndex).Initialize(Other.GetSimMorphTarget(TargetIndex), 0);
		}

		// Sim Vertices 3D Group
		GetClothCollection()->SetNumElements(Other.GetNumSimVertices3D(), ClothCollectionGroup::SimVertices3D);
		FClothCollection::CopyArrayViewData(GetPreResizedSimPosition3D().Right(Other.GetNumSimVertices3D()), Other.GetPreResizedSimPosition3D());

		// Sim AccessoryMeshes
		SetNumSimAccessoryMeshes(0, bCookedOnly);
		const int32 OtherNumSimAccessoryMeshes = Other.GetNumSimAccessoryMeshes();
		SetNumSimAccessoryMeshes(OtherNumSimAccessoryMeshes, bCookedOnly);
		for (int32 MeshIndex = 0; MeshIndex < OtherNumSimAccessoryMeshes; ++MeshIndex)
		{
			GetSimAccessoryMesh(MeshIndex).Initialize(Other.GetSimAccessoryMesh(MeshIndex));
		}
	}

	void FCollectionClothFacade::SetPhysicsAssetSoftObjectPathName(const FSoftObjectPath& PathName)
	{
		if (ClothCollection->GetNumElements(ClothCollectionGroup::Lods) && GetClothCollection()->GetPhysicsAssetSoftObjectPathName())
		{
			(*GetClothCollection()->GetPhysicsAssetSoftObjectPathName())[0] = PathName;
		}
	}
	
	void FCollectionClothFacade::SetSkeletalMeshSoftObjectPathName(const FSoftObjectPath& PathName)
	{
		if (ClothCollection->GetNumElements(ClothCollectionGroup::Lods) && GetClothCollection()->GetSkeletalMeshSoftObjectPathName())
		{
			(*GetClothCollection()->GetSkeletalMeshSoftObjectPathName())[0] = PathName;
		}
	}

	void FCollectionClothFacade::SetReferenceBoneName(const FName& BoneName)
	{
		if (ClothCollection->GetNumElements(ClothCollectionGroup::Lods))
		{
			(*GetClothCollection()->GetReferenceBoneName())[0] = BoneName;
		}
	}

	void FCollectionClothFacade::SetCopyAllUserAttributesToSimModel(bool bValue)
	{
		if (ClothCollection->GetNumElements(ClothCollectionGroup::Lods) && GetClothCollection()->GetCopyAllUserAttributesToSimModel())
		{
			(*GetClothCollection()->GetCopyAllUserAttributesToSimModel())[0] = bValue;
		}
	}
	
	void FCollectionClothFacade::SetSolverGravity(const FVector3f& SolverGravity)
	{
		if (!IsValid(EClothCollectionExtendedSchemas::Solvers))
		{
			DefineSchema(EClothCollectionExtendedSchemas::Solvers);
		}
		(*GetClothCollection()->GetSolverGravity())[0] = SolverGravity;
	}
	
	void FCollectionClothFacade::SetSolverAirDamping(const float SolverAirDamping)
	{
		if (!IsValid(EClothCollectionExtendedSchemas::Solvers))
		{
			DefineSchema(EClothCollectionExtendedSchemas::Solvers);
		}
		(*GetClothCollection()->GetSolverAirDamping())[0] = SolverAirDamping;
	}
	
	void FCollectionClothFacade::SetSolverTimeStep(const float SolverTimeStep)
	{
		if (!IsValid(EClothCollectionExtendedSchemas::Solvers))
		{
			DefineSchema(EClothCollectionExtendedSchemas::Solvers);
		}
		(*GetClothCollection()->GetSolverTimeStep())[0] = SolverTimeStep;
	}
	
	void FCollectionClothFacade::SetSolverSubSteps(const int32 SolverSubSteps)
	{
		if (!IsValid(EClothCollectionExtendedSchemas::Solvers))
		{
			DefineSchema(EClothCollectionExtendedSchemas::Solvers);
		}
		(*GetClothCollection()->GetSolverSubSteps())[0] = SolverSubSteps;
	}

	TArrayView<FVector2f> FCollectionClothFacade::GetSimPosition2D()
	{
		return GetClothCollection()->GetElements(GetClothCollection()->GetSimPosition2D());
	}

	TArrayView<int32> FCollectionClothFacade::GetSimVertex3DLookupPrivate()
	{
		return GetClothCollection()->GetElements(GetClothCollection()->GetSimVertex3DLookup());
	}

	TArrayView<int32> FCollectionClothFacade::GetSimImportVertexID()
	{
		if (!IsValid(EClothCollectionExtendedSchemas::Import))
		{
			DefineSchema(EClothCollectionExtendedSchemas::Import);
		}
		return GetClothCollection()->GetElements(GetClothCollection()->GetSimImportVertexID());
	}

	TArrayView<FVector3f> FCollectionClothFacade::GetSimPosition3D()
	{
		return GetClothCollection()->GetElements(GetClothCollection()->GetSimPosition3D());
	}

	TArrayView<FVector3f> FCollectionClothFacade::GetPreResizedSimPosition3D()
	{
		return GetClothCollection()->GetElements(GetClothCollection()->GetPreResizedSimPosition3D());
	}

	TArrayView<FVector3f> FCollectionClothFacade::GetSimNormal()
	{
		return GetClothCollection()->GetElements(GetClothCollection()->GetSimNormal());
	}

	TArrayView<TArray<int32>> FCollectionClothFacade::GetSimBoneIndices()
	{
		return GetClothCollection()->GetElements(GetClothCollection()->GetSimBoneIndices());
	}

	TArrayView<TArray<float>> FCollectionClothFacade::GetSimBoneWeights()
	{
		return GetClothCollection()->GetElements(GetClothCollection()->GetSimBoneWeights());
	}

	TArrayView<TArray<int32>> FCollectionClothFacade::GetTetherKinematicIndex()
	{
		return GetClothCollection()->GetElements(GetClothCollection()->GetTetherKinematicIndex());
	}

	TArrayView<TArray<float>> FCollectionClothFacade::GetTetherReferenceLength()
	{
		return GetClothCollection()->GetElements(GetClothCollection()->GetTetherReferenceLength());
	}
	
	TArrayView<float> FCollectionClothFacade::GetSimCustomResizingBlend()
	{
		return GetClothCollection()->GetElements(GetClothCollection()->GetSimCustomResizingBlend());
	}

	TArrayView<TArray<int32>> FCollectionClothFacade::GetSeamStitchLookupPrivate()
	{
		return GetClothCollection()->GetElements(GetClothCollection()->GetSeamStitchLookup());
	}

	TArrayView<TArray<int32>> FCollectionClothFacade::GetSimVertex2DLookupPrivate()
	{
		return GetClothCollection()->GetElements(GetClothCollection()->GetSimVertex2DLookup());
	}

	void FCollectionClothFacade::RemoveSimVertices3D(int32 InNumSimVertices)
	{
		const int32 NumSimVertices = GetNumSimVertices3D();
		check(InNumSimVertices >= NumSimVertices);
		GetClothCollection()->SetNumElements(NumSimVertices - InNumSimVertices, ClothCollectionGroup::SimVertices3D);
	}

	void FCollectionClothFacade::RemoveSimVertices3D(const TArray<int32>& SortedDeletionList)
	{
		GetClothCollection()->RemoveElements(ClothCollectionGroup::SimVertices3D, SortedDeletionList);
	}

	void FCollectionClothFacade::CompactSimVertex2DLookup()
	{
		TArrayView<TArray<int32>> SimVertex2DLookup = GetSimVertex2DLookupPrivate();
		for (TArray<int32>& VertexLookup : SimVertex2DLookup)
		{
			for (int32 Idx = 0; Idx < VertexLookup.Num();)
			{
				if (VertexLookup[Idx] == INDEX_NONE)
				{
					VertexLookup.RemoveAtSwap(Idx);
					continue;
				}
				++Idx;
			}
		}
	}

	void FCollectionClothFacade::CompactSeamStitchLookup()
	{
		TArrayView<TArray<int32>> SeamStitchLookup = GetSeamStitchLookupPrivate();
		for (TArray<int32>& StitchLookup : SeamStitchLookup)
		{
			for (int32 Idx = 0; Idx < StitchLookup.Num();)
			{
				if (StitchLookup[Idx] == INDEX_NONE)
				{
					StitchLookup.RemoveAtSwap(Idx);
					continue;
				}
				++Idx;
			}
		}
	}

	TArrayView<FIntVector3> FCollectionClothFacade::GetSimIndices2D()
	{
		return GetClothCollection()->GetElements(GetClothCollection()->GetSimIndices2D());
	}

	TArrayView<FIntVector3> FCollectionClothFacade::GetSimIndices3D()
	{
		return GetClothCollection()->GetElements(GetClothCollection()->GetSimIndices3D());
	}

	void FCollectionClothFacade::SetNumSimPatterns(int32 InNumPatterns)
	{
		const int32 NumPatterns = GetNumSimPatterns();

		for (int32 PatternIndex = InNumPatterns; PatternIndex < NumPatterns; ++PatternIndex)
		{
			GetSimPattern(PatternIndex).Reset();
		}

		GetClothCollection()->SetNumElements(InNumPatterns, ClothCollectionGroup::SimPatterns);

		for (int32 PatternIndex = NumPatterns; PatternIndex < InNumPatterns; ++PatternIndex)
		{
			GetSimPattern(PatternIndex).SetDefaults();
		}
	}

	int32 FCollectionClothFacade::AddSimPattern()
	{
		const int32 PatternIndex = GetNumSimPatterns();
		SetNumSimPatterns(PatternIndex + 1);
		return PatternIndex;
	}

	FCollectionClothSimPatternFacade FCollectionClothFacade::GetSimPattern(int32 PatternIndex)
	{
		return FCollectionClothSimPatternFacade(GetClothCollection(), PatternIndex);
	}

	void FCollectionClothFacade::RemoveSimPatterns(const TArray<int32>& SortedDeletionList)
	{
		for (const int32 PatternToRemove : SortedDeletionList)
		{
			GetSimPattern(PatternToRemove).Reset();
		}

		GetClothCollection()->RemoveElements(ClothCollectionGroup::SimPatterns, SortedDeletionList);
	}

	void FCollectionClothFacade::SetNumSimMorphTargets(int32 InNumMorphTargets, bool bCookedOnly)
	{
		const EClothCollectionExtendedSchemas OptionalSchemas = bCookedOnly ? EClothCollectionExtendedSchemas::CookedOnly : EClothCollectionExtendedSchemas::SimMorphTargets;
		if (!IsValid(OptionalSchemas))
		{
			DefineSchema(OptionalSchemas);
		}

		const int32 NumMorphTargets = GetNumSimMorphTargets();

		for (int32 TargetIndex = InNumMorphTargets; TargetIndex < NumMorphTargets; ++TargetIndex)
		{
			GetSimMorphTarget(TargetIndex).Reset();
		}

		GetClothCollection()->SetNumElements(InNumMorphTargets, ClothCollectionGroup::SimMorphTargets);

		for (int32 TargetIndex = NumMorphTargets; TargetIndex < InNumMorphTargets; ++TargetIndex)
		{
			GetSimMorphTarget(TargetIndex).SetDefaults();
		}
	}

	int32 FCollectionClothFacade::AddSimMorphTarget()
	{
		const int32 TargetIndex = GetNumSimMorphTargets();
		SetNumSimMorphTargets(TargetIndex + 1);
		return TargetIndex;
	}

	FCollectionClothSimMorphTargetFacade FCollectionClothFacade::GetSimMorphTarget(int32 MorphTargetIndex)
	{
		return FCollectionClothSimMorphTargetFacade(GetClothCollection(), MorphTargetIndex);
	}

	void FCollectionClothFacade::RemoveSimMorphTargets(const TArray<int32>& SortedDeletionList)
	{
		for (const int32 TargetToRemove : SortedDeletionList)
		{
			GetSimMorphTarget(TargetToRemove).Reset();
		}
		GetClothCollection()->RemoveElements(ClothCollectionGroup::SimMorphTargets, SortedDeletionList);
	}

	void FCollectionClothFacade::CompactSimMorphTargets()
	{
		TArray<int32> SortedDeletionList;
		for (int32 TargetIndex = 0; TargetIndex < GetNumSimMorphTargets(); ++TargetIndex)
		{
			FCollectionClothSimMorphTargetFacade MorphTarget = GetSimMorphTarget(TargetIndex);
			MorphTarget.Compact();
			if (MorphTarget.IsEmpty())
			{
				SortedDeletionList.Add(TargetIndex);
			}
		}
		RemoveSimMorphTargets(SortedDeletionList);
	}

	TArrayView<FVector3f> FCollectionClothFacade::GetSimMorphTargetPositionDelta()
	{
		return GetClothCollection()->GetElements(GetClothCollection()->GetSimMorphTargetPositionDelta());
	}

	TArrayView<FVector3f> FCollectionClothFacade::GetSimMorphTargetTangentZDelta()
	{
		return GetClothCollection()->GetElements(GetClothCollection()->GetSimMorphTargetTangentZDelta());
	}

	TArrayView<int32> FCollectionClothFacade::GetSimMorphTargetSimVertex3DIndex()
	{
		return GetClothCollection()->GetElements(GetClothCollection()->GetSimMorphTargetSimVertex3DIndex());
	}

	void FCollectionClothFacade::RemoveSimMorphTargetVertices3D(const TArray<int32>& SortedDeletionList)
	{
		GetClothCollection()->RemoveElements(ClothCollectionGroup::SimMorphTargetVertices, SortedDeletionList);
	}

	void FCollectionClothFacade::SetNumSimAccessoryMeshes(int32 InNumMeshes, bool bCookedOnly)
	{
		const EClothCollectionExtendedSchemas OptionalSchemas = bCookedOnly ? EClothCollectionExtendedSchemas::CookedOnly : EClothCollectionExtendedSchemas::SimAccessoryMeshes;
		if (!IsValid(OptionalSchemas))
		{
			DefineSchema(OptionalSchemas);
		}

		const int32 NumMeshes = GetNumSimAccessoryMeshes();

		for (int32 MeshIndex = InNumMeshes; MeshIndex < NumMeshes; ++MeshIndex)
		{
			GetSimAccessoryMesh(MeshIndex).Reset();
		}

		GetClothCollection()->SetNumElements(InNumMeshes, ClothCollectionGroup::SimAccessoryMeshes);

		for (int32 MeshIndex = NumMeshes; MeshIndex < InNumMeshes; ++MeshIndex)
		{
			GetSimAccessoryMesh(MeshIndex).SetDefaults();
		}
	}

	int32 FCollectionClothFacade::AddSimAccessoryMesh()
	{
		const int32 MeshIndex = GetNumSimAccessoryMeshes();
		SetNumSimAccessoryMeshes(MeshIndex + 1);
		return MeshIndex;
	}

	FCollectionClothSimAccessoryMeshFacade FCollectionClothFacade::GetSimAccessoryMesh(int32 MeshIndex)
	{
		return FCollectionClothSimAccessoryMeshFacade(GetClothCollection(), MeshIndex);
	}

	void FCollectionClothFacade::RemoveSimAccessoryMeshes(const TArray<int32>& SortedDeletionList)
	{
		for (const int32 MeshToRemove : SortedDeletionList)
		{
			GetSimAccessoryMesh(MeshToRemove).Reset();
		}
		GetClothCollection()->RemoveElements(ClothCollectionGroup::SimAccessoryMeshes, SortedDeletionList);
	}

	TArrayView<FName> FCollectionClothFacade::GetSimAccessoryMeshName()
	{
		return GetClothCollection()->GetElements(GetClothCollection()->GetSimAccessoryMeshName());
	}

	TArrayView<FName> FCollectionClothFacade::GetSimAccessoryMeshPosition3DAttribute()
	{
		return GetClothCollection()->GetElements(GetClothCollection()->GetSimAccessoryMeshPosition3DAttribute());
	}

	TArrayView<FName> FCollectionClothFacade::GetSimAccessoryMeshNormalAttribute()
	{
		return GetClothCollection()->GetElements(GetClothCollection()->GetSimAccessoryMeshNormalAttribute());
	}

	TArrayView<FName> FCollectionClothFacade::GetSimAccessoryMeshBoneIndicesAttribute()
	{
		return GetClothCollection()->GetElements(GetClothCollection()->GetSimAccessoryMeshBoneIndicesAttribute());
	}

	TArrayView<FName> FCollectionClothFacade::GetSimAccessoryMeshBoneWeightsAttribute()
	{
		return GetClothCollection()->GetElements(GetClothCollection()->GetSimAccessoryMeshBoneWeightsAttribute());
	}

	void FCollectionClothFacade::SetNumRenderPatterns(int32 InNumPatterns)
	{
		const int32 NumPatterns = GetNumRenderPatterns();

		for (int32 PatternIndex = InNumPatterns; PatternIndex < NumPatterns; ++PatternIndex)
		{
			GetRenderPattern(PatternIndex).Reset();
		}

		GetClothCollection()->SetNumElements(InNumPatterns, ClothCollectionGroup::RenderPatterns);

		for (int32 PatternIndex = NumPatterns; PatternIndex < InNumPatterns; ++PatternIndex)
		{
			GetRenderPattern(PatternIndex).SetDefaults();
		}
	}

	int32 FCollectionClothFacade::AddRenderPattern()
	{
		const int32 PatternIndex = GetNumRenderPatterns();
		SetNumRenderPatterns(PatternIndex + 1);
		return PatternIndex;
	}

	FCollectionClothRenderPatternFacade FCollectionClothFacade::GetRenderPattern(int32 PatternIndex)
	{
		return FCollectionClothRenderPatternFacade(GetClothCollection(), PatternIndex);
	}

	void FCollectionClothFacade::RemoveRenderPatterns(const TArray<int32>& SortedDeletionList)
	{
		for (const int32 PatternToRemove : SortedDeletionList)
		{
			GetRenderPattern(PatternToRemove).Reset();
		}

		GetClothCollection()->RemoveElements(ClothCollectionGroup::RenderPatterns, SortedDeletionList);
	}

	TArrayView<int32> FCollectionClothFacade::GetRenderDeformerNumInfluences()
	{
		return GetClothCollection()->GetElements(GetClothCollection()->GetRenderDeformerNumInfluences());
	}

	TArrayView<FSoftObjectPath> FCollectionClothFacade::GetRenderMaterialSoftObjectPathName()
	{
		return GetClothCollection()->GetElements(GetClothCollection()->GetRenderMaterialSoftObjectPathName());
	}

	void FCollectionClothFacade::SetNumSeams(int32 InNumSeams)
	{
		const int32 NumSeams = GetNumSeams();

		for (int32 SeamIndex = InNumSeams; SeamIndex < NumSeams; ++SeamIndex)
		{
			GetSeam(SeamIndex).Reset();
		}

		GetClothCollection()->SetNumElements(InNumSeams, ClothCollectionGroup::Seams);

		for (int32 SeamIndex = NumSeams; SeamIndex < InNumSeams; ++SeamIndex)
		{
			GetSeam(SeamIndex).SetDefaults();
		}
	}

	int32 FCollectionClothFacade::AddSeam()
	{
		const int32 SeamIndex = GetNumSeams();
		SetNumSeams(SeamIndex + 1);
		return SeamIndex;
	}

	FCollectionClothSeamFacade FCollectionClothFacade::GetSeam(int32 SeamIndex)
	{
		return FCollectionClothSeamFacade(GetClothCollection(), SeamIndex);
	}

	void FCollectionClothFacade::RemoveSeams(const TArray<int32>& SortedDeletionList)
	{
		for (const int32 SeamToRemove : SortedDeletionList)
		{
			GetSeam(SeamToRemove).Reset();
		}
		GetClothCollection()->RemoveElements(ClothCollectionGroup::Seams, SortedDeletionList);
	}

	void FCollectionClothFacade::SetNumFabrics(int32 InNumFabrics)
	{
		const int32 NumFabrics = GetNumFabrics();

		for (int32 FabricIndex = InNumFabrics; FabricIndex < NumFabrics; ++FabricIndex)
		{
			GetFabric(FabricIndex).Reset();
		}

		GetClothCollection()->SetNumElements(InNumFabrics, ClothCollectionGroup::Fabrics);

		for (int32 FabricIndex = NumFabrics; FabricIndex < InNumFabrics; ++FabricIndex)
		{
			GetFabric(FabricIndex).SetDefaults();
		}
	}

	int32 FCollectionClothFacade::AddFabric()
	{
		const int32 FabricIndex = GetNumFabrics();
		SetNumFabrics(FabricIndex + 1);
		return FabricIndex;
	}

	FCollectionClothFabricFacade FCollectionClothFacade::GetFabric(int32 FabricIndex)
	{
		return FCollectionClothFabricFacade(GetClothCollection(), FabricIndex);
	}

	void FCollectionClothFacade::RemoveFabrics(const TArray<int32>& SortedDeletionList)
	{
		for (const int32 FabricToRemove : SortedDeletionList)
		{
			GetFabric(FabricToRemove).Reset();
		}
		GetClothCollection()->RemoveElements(ClothCollectionGroup::Fabrics, SortedDeletionList);
	}

	//~ Render Vertices Group
	TArrayView<FVector3f> FCollectionClothFacade::GetRenderPosition()
	{
		return GetClothCollection()->GetElements(GetClothCollection()->GetRenderPosition());
	}

	TArrayView<FVector3f> FCollectionClothFacade::GetRenderNormal()
	{
		return GetClothCollection()->GetElements(GetClothCollection()->GetRenderNormal());
	}

	TArrayView<FVector3f> FCollectionClothFacade::GetRenderTangentU()
	{
		return GetClothCollection()->GetElements(GetClothCollection()->GetRenderTangentU());
	}

	TArrayView<FVector3f> FCollectionClothFacade::GetRenderTangentV()
	{
		return GetClothCollection()->GetElements(GetClothCollection()->GetRenderTangentV());
	}

	TArrayView<TArray<FVector2f>> FCollectionClothFacade::GetRenderUVs()
	{
		return GetClothCollection()->GetElements(GetClothCollection()->GetRenderUVs());
	}

	TArrayView<FLinearColor> FCollectionClothFacade::GetRenderColor()
	{
		return GetClothCollection()->GetElements(GetClothCollection()->GetRenderColor());
	}

	TArrayView<TArray<int32>> FCollectionClothFacade::GetRenderBoneIndices()
	{
		return GetClothCollection()->GetElements(GetClothCollection()->GetRenderBoneIndices());
	}

	TArrayView<TArray<float>> FCollectionClothFacade::GetRenderBoneWeights()
	{
		return GetClothCollection()->GetElements(GetClothCollection()->GetRenderBoneWeights());
	}

	TArrayView<TArray<FVector4f>> FCollectionClothFacade::GetRenderDeformerPositionBaryCoordsAndDist()
	{
		return GetClothCollection()->GetElements(GetClothCollection()->GetRenderDeformerPositionBaryCoordsAndDist());
	}

	TArrayView<TArray<FVector4f>> FCollectionClothFacade::GetRenderDeformerNormalBaryCoordsAndDist()
	{
		return GetClothCollection()->GetElements(GetClothCollection()->GetRenderDeformerNormalBaryCoordsAndDist());
	}

	TArrayView<TArray<FVector4f>> FCollectionClothFacade::GetRenderDeformerTangentBaryCoordsAndDist()
	{
		return GetClothCollection()->GetElements(GetClothCollection()->GetRenderDeformerTangentBaryCoordsAndDist());
	}

	TArrayView<TArray<FIntVector3>> FCollectionClothFacade::GetRenderDeformerSimIndices3D()
	{
		return GetClothCollection()->GetElements(GetClothCollection()->GetRenderDeformerSimIndices3D());
	}

	TArrayView<TArray<float>> FCollectionClothFacade::GetRenderDeformerWeight()
	{
		return GetClothCollection()->GetElements(GetClothCollection()->GetRenderDeformerWeight());
	}

	TArrayView<float> FCollectionClothFacade::GetRenderDeformerSkinningBlend()
	{
		return GetClothCollection()->GetElements(GetClothCollection()->GetRenderDeformerSkinningBlend());
	}

	TArrayView<float> FCollectionClothFacade::GetRenderCustomResizingBlend()
	{
		return GetClothCollection()->GetElements(GetClothCollection()->GetRenderCustomResizingBlend());
	}

	TArrayView<FIntVector3> FCollectionClothFacade::GetRenderIndices()
	{
		return GetClothCollection()->GetElements(GetClothCollection()->GetRenderIndices());
	}

	void FCollectionClothFacade::SetNumCustomResizingRegions(int32 InNumGroups)
	{
		if (!IsValid(EClothCollectionExtendedSchemas::Resizing))
		{
			DefineSchema(EClothCollectionExtendedSchemas::Resizing);
		}

		const int32 NumGroups = GetNumCustomResizingRegions();

		GetClothCollection()->SetNumElements(InNumGroups, ClothCollectionGroup::CustomResizingRegions);

		for (int32 Index = NumGroups; Index < InNumGroups; ++Index)
		{
			GetCustomResizingRegionSet()[Index] = FString();
		}
	}

	TArrayView<FString> FCollectionClothFacade::GetCustomResizingRegionSet()
	{
		return GetClothCollection()->GetElements(GetClothCollection()->GetCustomResizingRegionSet());
	}

	TArrayView<int32> FCollectionClothFacade::GetCustomResizingRegionType()
	{
		return GetClothCollection()->GetElements(GetClothCollection()->GetCustomResizingRegionType());
	}

	void FCollectionClothFacade::AddWeightMap(const FName& Name)
	{
		check(IsValid());
		GetClothCollection()->AddUserDefinedAttribute<float>(Name, ClothCollectionGroup::SimVertices3D);
	}

	void FCollectionClothFacade::RemoveWeightMap(const FName& Name)
	{
		check(IsValid());
		GetClothCollection()->RemoveUserDefinedAttribute(Name, ClothCollectionGroup::SimVertices3D);
	}

	TArrayView<float> FCollectionClothFacade::GetWeightMap(const FName& Name)
	{
		return GetClothCollection()->GetElements(GetClothCollection()->GetUserDefinedAttribute<float>(Name, ClothCollectionGroup::SimVertices3D));
	}

	template<typename T UE_REQUIRES_DEFINITION(TIsUserAttributeType<T>::Value)>
	bool FCollectionClothFacade::AddUserDefinedAttribute(const FName& Name, const FName& GroupName, const FName& GroupDependency)
	{
		check(IsValid());
		return GetClothCollection()->FindOrAddUserDefinedAttribute<T>(Name, GroupName, GroupDependency) != nullptr;
	}
	template CHAOSCLOTHASSET_API bool FCollectionClothFacade::AddUserDefinedAttribute<int32>(const FName& Name, const FName& GroupName, const FName& GroupDependency);
	template CHAOSCLOTHASSET_API bool FCollectionClothFacade::AddUserDefinedAttribute<float>(const FName& Name, const FName& GroupName, const FName& GroupDependency);
	template CHAOSCLOTHASSET_API bool FCollectionClothFacade::AddUserDefinedAttribute<FVector3f>(const FName& Name, const FName& GroupName, const FName& GroupDependency);
	template CHAOSCLOTHASSET_API bool FCollectionClothFacade::AddUserDefinedAttribute<TArray<int32>>(const FName& Name, const FName& GroupName, const FName& GroupDependency);

	void FCollectionClothFacade::RemoveUserDefinedAttribute(const FName& Name, const FName& GroupName)
	{
		check(IsValid());
		GetClothCollection()->RemoveUserDefinedAttribute(Name, GroupName);
	}

	template<typename T UE_REQUIRES_DEFINITION(TIsUserAttributeType<T>::Value)>
	TArrayView<T> FCollectionClothFacade::GetUserDefinedAttribute(const FName& Name, const FName& GroupName)
	{
		return GetClothCollection()->GetElements(GetClothCollection()->GetUserDefinedAttribute<T>(Name, GroupName));
	}
	template CHAOSCLOTHASSET_API TArrayView<int32> FCollectionClothFacade::GetUserDefinedAttribute<int32>(const FName& Name, const FName& GroupName);
	template CHAOSCLOTHASSET_API TArrayView<float> FCollectionClothFacade::GetUserDefinedAttribute<float>(const FName& Name, const FName& GroupName);
	template CHAOSCLOTHASSET_API TArrayView<FVector3f> FCollectionClothFacade::GetUserDefinedAttribute<FVector3f>(const FName& Name, const FName& GroupName);
	template CHAOSCLOTHASSET_API TArrayView<TArray<int32>> FCollectionClothFacade::GetUserDefinedAttribute<TArray<int32>>(const FName& Name, const FName& GroupName);

	void FCollectionClothFacade::SetDefaults()
	{
		GetClothCollection()->SetNumElements(1, ClothCollectionGroup::Lods);

		if (IsValid(EClothCollectionExtendedSchemas::Solvers))
		{
			GetClothCollection()->SetNumElements(1, ClothCollectionGroup::Solvers);
			
			SetSolverGravity(FDefaultSolver::Gravity);
			SetSolverAirDamping(FDefaultSolver::AirDamping);
			SetSolverSubSteps(FDefaultSolver::SubSteps);
			SetSolverTimeStep(FDefaultSolver::TimeStep);
		}
	}

	TSharedRef<class FClothCollection> FCollectionClothFacade::GetClothCollection()
	{
		return StaticCastSharedRef<class FClothCollection>(ConstCastSharedRef<class FConstClothCollection>(ClothCollection));
	}

} // End namespace UE::Chaos::ClothAsset
