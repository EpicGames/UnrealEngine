// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/CollectionClothRenderPatternFacade.h"
#include "ChaosClothAsset/ClothCollection.h"
#include "ChaosClothAsset/ClothCollectionGroup.h"
#include "ChaosClothAsset/ClothGeometryTools.h"
#include "UObject/SoftObjectPath.h"

namespace UE::Chaos::ClothAsset
{
	int32 FCollectionClothRenderPatternConstFacade::GetRenderDeformerNumInfluences() const
	{
		return ClothCollection->GetRenderDeformerNumInfluences() && ClothCollection->GetNumElements(ClothCollectionGroup::RenderPatterns) > GetElementIndex() ?
			(*ClothCollection->GetRenderDeformerNumInfluences())[GetElementIndex()] : 0;
	}

	const FSoftObjectPath& FCollectionClothRenderPatternConstFacade::GetRenderMaterialSoftObjectPathName() const
	{
		static const FSoftObjectPath EmptyString;
		return ClothCollection->GetRenderMaterialSoftObjectPathName() && ClothCollection->GetNumElements(ClothCollectionGroup::RenderPatterns) > GetElementIndex() ?
			(*ClothCollection->GetRenderMaterialSoftObjectPathName())[GetElementIndex()] : EmptyString;
	}

	int32 FCollectionClothRenderPatternConstFacade::GetNumRenderVertices() const
	{
		return ClothCollection->GetNumElements(
			ClothCollection->GetRenderVerticesStart(),
			ClothCollection->GetRenderVerticesEnd(),
			GetElementIndex());
	}

	int32 FCollectionClothRenderPatternConstFacade::GetRenderVerticesOffset() const
	{
		return ClothCollection->GetElementsOffset(
			ClothCollection->GetRenderVerticesStart(),
			GetBaseElementIndex(),
			GetElementIndex());
	}

	TConstArrayView<FVector3f> FCollectionClothRenderPatternConstFacade::GetRenderPosition() const
	{
		return ClothCollection->GetElements(
			ClothCollection->GetRenderPosition(),
			ClothCollection->GetRenderVerticesStart(),
			ClothCollection->GetRenderVerticesEnd(),
			GetElementIndex());
	}

	TConstArrayView<FVector3f> FCollectionClothRenderPatternConstFacade::GetRenderNormal() const
	{
		return ClothCollection->GetElements(
			ClothCollection->GetRenderNormal(),
			ClothCollection->GetRenderVerticesStart(),
			ClothCollection->GetRenderVerticesEnd(),
			GetElementIndex());
	}

	TConstArrayView<FVector3f> FCollectionClothRenderPatternConstFacade::GetRenderTangentU() const
	{
		return ClothCollection->GetElements(
			ClothCollection->GetRenderTangentU(),
			ClothCollection->GetRenderVerticesStart(),
			ClothCollection->GetRenderVerticesEnd(),
			GetElementIndex());
	}

	TConstArrayView<FVector3f> FCollectionClothRenderPatternConstFacade::GetRenderTangentV() const
	{
		return ClothCollection->GetElements(
			ClothCollection->GetRenderTangentV(),
			ClothCollection->GetRenderVerticesStart(),
			ClothCollection->GetRenderVerticesEnd(),
			GetElementIndex());
	}

	TConstArrayView<TArray<FVector2f>> FCollectionClothRenderPatternConstFacade::GetRenderUVs() const
	{
		return ClothCollection->GetElements(
			ClothCollection->GetRenderUVs(),
			ClothCollection->GetRenderVerticesStart(),
			ClothCollection->GetRenderVerticesEnd(),
			GetElementIndex());
	}

	TConstArrayView<FLinearColor> FCollectionClothRenderPatternConstFacade::GetRenderColor() const
	{
		return ClothCollection->GetElements(
			ClothCollection->GetRenderColor(),
			ClothCollection->GetRenderVerticesStart(),
			ClothCollection->GetRenderVerticesEnd(),
			GetElementIndex());
	}

	TConstArrayView<TArray<int32>> FCollectionClothRenderPatternConstFacade::GetRenderBoneIndices() const
	{
		return ClothCollection->GetElements(
			ClothCollection->GetRenderBoneIndices(),
			ClothCollection->GetRenderVerticesStart(),
			ClothCollection->GetRenderVerticesEnd(),
			GetElementIndex());
	}

	TConstArrayView<TArray<float>> FCollectionClothRenderPatternConstFacade::GetRenderBoneWeights() const
	{
		return ClothCollection->GetElements(
			ClothCollection->GetRenderBoneWeights(),
			ClothCollection->GetRenderVerticesStart(),
			ClothCollection->GetRenderVerticesEnd(),
			GetElementIndex());
	}

	TConstArrayView<TArray<FVector4f>> FCollectionClothRenderPatternConstFacade::GetRenderDeformerPositionBaryCoordsAndDist() const
	{
		return ClothCollection->GetElements(
			ClothCollection->GetRenderDeformerPositionBaryCoordsAndDist(),
			ClothCollection->GetRenderVerticesStart(),
			ClothCollection->GetRenderVerticesEnd(),
			GetElementIndex());
	}

	TConstArrayView<TArray<FVector4f>> FCollectionClothRenderPatternConstFacade::GetRenderDeformerNormalBaryCoordsAndDist() const
	{
		return ClothCollection->GetElements(
			ClothCollection->GetRenderDeformerNormalBaryCoordsAndDist(),
			ClothCollection->GetRenderVerticesStart(),
			ClothCollection->GetRenderVerticesEnd(),
			GetElementIndex());
	}

	TConstArrayView<TArray<FVector4f>> FCollectionClothRenderPatternConstFacade::GetRenderDeformerTangentBaryCoordsAndDist() const
	{
		return ClothCollection->GetElements(
			ClothCollection->GetRenderDeformerTangentBaryCoordsAndDist(),
			ClothCollection->GetRenderVerticesStart(),
			ClothCollection->GetRenderVerticesEnd(),
			GetElementIndex());
	}

	TConstArrayView<TArray<FIntVector3>> FCollectionClothRenderPatternConstFacade::GetRenderDeformerSimIndices3D() const
	{
		return ClothCollection->GetElements(
			ClothCollection->GetRenderDeformerSimIndices3D(),
			ClothCollection->GetRenderVerticesStart(),
			ClothCollection->GetRenderVerticesEnd(),
			GetElementIndex());
	}

	TConstArrayView<TArray<float>> FCollectionClothRenderPatternConstFacade::GetRenderDeformerWeight() const
	{
		return ClothCollection->GetElements(
			ClothCollection->GetRenderDeformerWeight(),
			ClothCollection->GetRenderVerticesStart(),
			ClothCollection->GetRenderVerticesEnd(),
			GetElementIndex());
	}

	TConstArrayView<float> FCollectionClothRenderPatternConstFacade::GetRenderDeformerSkinningBlend() const
	{
		return ClothCollection->GetElements(
			ClothCollection->GetRenderDeformerSkinningBlend(),
			ClothCollection->GetRenderVerticesStart(),
			ClothCollection->GetRenderVerticesEnd(),
			GetElementIndex());
	}

	TConstArrayView<float> FCollectionClothRenderPatternConstFacade::GetRenderCustomResizingBlend() const
	{
		return ClothCollection->GetElements(
			ClothCollection->GetRenderCustomResizingBlend(),
			ClothCollection->GetRenderVerticesStart(),
			ClothCollection->GetRenderVerticesEnd(),
			GetElementIndex());
	}

	int32 FCollectionClothRenderPatternConstFacade::GetNumRenderFaces() const
	{
		return ClothCollection->GetNumElements(
			ClothCollection->GetRenderFacesStart(),
			ClothCollection->GetRenderFacesEnd(),
			GetElementIndex());
	}

	int32 FCollectionClothRenderPatternConstFacade::GetRenderFacesOffset() const
	{
		return ClothCollection->GetElementsOffset(
			ClothCollection->GetRenderFacesStart(),
			GetBaseElementIndex(),
			GetElementIndex());
	}

	TConstArrayView<FIntVector3> FCollectionClothRenderPatternConstFacade::GetRenderIndices() const
	{
		return ClothCollection->GetElements(
			ClothCollection->GetRenderIndices(),
			ClothCollection->GetRenderFacesStart(),
			ClothCollection->GetRenderFacesEnd(),
			GetElementIndex());
	}

	bool FCollectionClothRenderPatternConstFacade::IsEmpty() const
	{
		return GetNumRenderVertices() == 0 && GetNumRenderFaces() == 0;
	}

	FCollectionClothRenderPatternConstFacade::FCollectionClothRenderPatternConstFacade(const TSharedRef<const FConstClothCollection>& ClothCollection, int32 InPatternIndex)
		: ClothCollection(ClothCollection)
		, PatternIndex(InPatternIndex)
	{
		check(ClothCollection->IsValid());
		check(PatternIndex >= 0 && PatternIndex < ClothCollection->GetNumElements(ClothCollectionGroup::RenderPatterns));
	}

	void FCollectionClothRenderPatternFacade::Reset()
	{
		SetNumRenderVertices(0);
		SetNumRenderFaces(0);
		SetDefaults();
	}

	void FCollectionClothRenderPatternFacade::Initialize(const FCollectionClothRenderPatternConstFacade& Other, int32 SimVertex3DOffset)
	{
		Reset();

		//~ Render Patterns Group
		SetRenderDeformerNumInfluences(Other.GetRenderDeformerNumInfluences());  // Must be called prior to copying the Render Deformer as it also sets the optional RenderDeformer schema
		SetRenderMaterialSoftObjectPathName(Other.GetRenderMaterialSoftObjectPathName());

		//~ Render Vertices Group
		SetNumRenderVertices(Other.GetNumRenderVertices());
		FClothCollection::CopyArrayViewData(GetRenderPosition(), Other.GetRenderPosition());
		FClothCollection::CopyArrayViewData(GetRenderNormal(), Other.GetRenderNormal());
		FClothCollection::CopyArrayViewData(GetRenderTangentU(), Other.GetRenderTangentU());
		FClothCollection::CopyArrayViewData(GetRenderTangentV(), Other.GetRenderTangentV());
		FClothCollection::CopyArrayViewData(GetRenderUVs(), Other.GetRenderUVs());
		FClothCollection::CopyArrayViewData(GetRenderColor(), Other.GetRenderColor());
		FClothCollection::CopyArrayViewData(GetRenderBoneIndices(), Other.GetRenderBoneIndices());
		FClothCollection::CopyArrayViewData(GetRenderBoneWeights(), Other.GetRenderBoneWeights());

		if (GetClothCollection()->IsValid(EClothCollectionExtendedSchemas::RenderDeformer) && Other.ClothCollection->IsValid(EClothCollectionExtendedSchemas::RenderDeformer))
		{
			FClothCollection::CopyArrayViewData(GetRenderDeformerPositionBaryCoordsAndDist(), Other.GetRenderDeformerPositionBaryCoordsAndDist());
			FClothCollection::CopyArrayViewData(GetRenderDeformerNormalBaryCoordsAndDist(), Other.GetRenderDeformerNormalBaryCoordsAndDist());
			FClothCollection::CopyArrayViewData(GetRenderDeformerTangentBaryCoordsAndDist(), Other.GetRenderDeformerTangentBaryCoordsAndDist());
			FClothCollection::CopyArrayViewDataAndApplyOffset(GetRenderDeformerSimIndices3D(), Other.GetRenderDeformerSimIndices3D(), FIntVector3(SimVertex3DOffset));
			FClothCollection::CopyArrayViewData(GetRenderDeformerWeight(), Other.GetRenderDeformerWeight());
			FClothCollection::CopyArrayViewData(GetRenderDeformerSkinningBlend(), Other.GetRenderDeformerSkinningBlend());
		}

		if (GetClothCollection()->IsValid(EClothCollectionExtendedSchemas::Resizing) && Other.ClothCollection->IsValid(EClothCollectionExtendedSchemas::Resizing))
		{
			FClothCollection::CopyArrayViewData(GetRenderCustomResizingBlend(), Other.GetRenderCustomResizingBlend());
		}

		//~ Render Faces Group
		const int32 RenderVertexOffset = GetRenderVerticesOffset() - Other.GetRenderVerticesOffset();
		SetNumRenderFaces(Other.GetNumRenderFaces());
		FClothCollection::CopyArrayViewDataAndApplyOffset(GetRenderIndices(), Other.GetRenderIndices(), FIntVector3(RenderVertexOffset));
	}

	void FCollectionClothRenderPatternFacade::SetRenderDeformerNumInfluences(int32 NumInfluences)
	{
		check(NumInfluences >= 0);
		if (!GetClothCollection()->IsValid(EClothCollectionExtendedSchemas::RenderDeformer))
		{
			if (NumInfluences == 0)
			{
				return;  // Not having the schema is the same as having zero influences
			}
			GetClothCollection()->DefineSchema(EClothCollectionExtendedSchemas::RenderDeformer);
		}
		(*GetClothCollection()->GetRenderDeformerNumInfluences())[GetElementIndex()] = NumInfluences;
	}

	void FCollectionClothRenderPatternFacade::SetRenderMaterialSoftObjectPathName(const FSoftObjectPath& PathName)
	{
		(*GetClothCollection()->GetRenderMaterialSoftObjectPathName())[GetElementIndex()] = PathName;
	}

	void FCollectionClothRenderPatternFacade::SetNumRenderVertices(int32 NumRenderVertices)
	{
		GetClothCollection()->SetNumElements(
			NumRenderVertices,
			ClothCollectionGroup::RenderVertices,
			GetClothCollection()->GetRenderVerticesStart(),
			GetClothCollection()->GetRenderVerticesEnd(),
			GetElementIndex());
	}

	void FCollectionClothRenderPatternFacade::RemoveRenderVertices(const TArray<int32>& SortedDeletionList)
	{
		TArray<int32> GlobalIndexSortedDeletionList;
		const int32 Offset = GetRenderVerticesOffset();
		GlobalIndexSortedDeletionList.SetNumUninitialized(SortedDeletionList.Num());
		for (int32 Idx = 0; Idx < SortedDeletionList.Num(); ++Idx)
		{
			GlobalIndexSortedDeletionList[Idx] = SortedDeletionList[Idx] + Offset;
		}

		GetClothCollection()->RemoveElements(
			ClothCollectionGroup::RenderVertices,
			GlobalIndexSortedDeletionList,
			GetClothCollection()->GetRenderVerticesStart(),
			GetClothCollection()->GetRenderVerticesEnd(),
			GetElementIndex());
	}

	TArrayView<FVector3f> FCollectionClothRenderPatternFacade::GetRenderPosition()
	{
		return GetClothCollection()->GetElements(
			GetClothCollection()->GetRenderPosition(),
			GetClothCollection()->GetRenderVerticesStart(),
			GetClothCollection()->GetRenderVerticesEnd(),
			GetElementIndex());
	}

	TArrayView<FVector3f> FCollectionClothRenderPatternFacade::GetRenderNormal()
	{
		return GetClothCollection()->GetElements(
			GetClothCollection()->GetRenderNormal(),
			GetClothCollection()->GetRenderVerticesStart(),
			GetClothCollection()->GetRenderVerticesEnd(),
			GetElementIndex());
	}

	TArrayView<FVector3f> FCollectionClothRenderPatternFacade::GetRenderTangentU()
	{
		return GetClothCollection()->GetElements(
			GetClothCollection()->GetRenderTangentU(),
			GetClothCollection()->GetRenderVerticesStart(),
			GetClothCollection()->GetRenderVerticesEnd(),
			GetElementIndex());
	}

	TArrayView<FVector3f> FCollectionClothRenderPatternFacade::GetRenderTangentV()
	{
		return GetClothCollection()->GetElements(
			GetClothCollection()->GetRenderTangentV(),
			GetClothCollection()->GetRenderVerticesStart(),
			GetClothCollection()->GetRenderVerticesEnd(),
			GetElementIndex());
	}

	TArrayView<TArray<FVector2f>> FCollectionClothRenderPatternFacade::GetRenderUVs()
	{
		return GetClothCollection()->GetElements(
			GetClothCollection()->GetRenderUVs(),
			GetClothCollection()->GetRenderVerticesStart(),
			GetClothCollection()->GetRenderVerticesEnd(),
			GetElementIndex());
	}

	TArrayView<FLinearColor> FCollectionClothRenderPatternFacade::GetRenderColor()
	{
		return GetClothCollection()->GetElements(
			GetClothCollection()->GetRenderColor(),
			GetClothCollection()->GetRenderVerticesStart(),
			GetClothCollection()->GetRenderVerticesEnd(),
			GetElementIndex());
	}

	TArrayView<TArray<int32>> FCollectionClothRenderPatternFacade::GetRenderBoneIndices()
	{
		return GetClothCollection()->GetElements(
			GetClothCollection()->GetRenderBoneIndices(),
			GetClothCollection()->GetRenderVerticesStart(),
			GetClothCollection()->GetRenderVerticesEnd(),
			GetElementIndex());
	}

	TArrayView<TArray<float>> FCollectionClothRenderPatternFacade::GetRenderBoneWeights()
	{
		return GetClothCollection()->GetElements(
			GetClothCollection()->GetRenderBoneWeights(),
			GetClothCollection()->GetRenderVerticesStart(),
			GetClothCollection()->GetRenderVerticesEnd(),
			GetElementIndex());
	}

	TArrayView<TArray<FVector4f>> FCollectionClothRenderPatternFacade::GetRenderDeformerPositionBaryCoordsAndDist()
	{
		return GetClothCollection()->GetElements(
			GetClothCollection()->GetRenderDeformerPositionBaryCoordsAndDist(),
			GetClothCollection()->GetRenderVerticesStart(),
			GetClothCollection()->GetRenderVerticesEnd(),
			GetElementIndex());
	}

	TArrayView<TArray<FVector4f>> FCollectionClothRenderPatternFacade::GetRenderDeformerNormalBaryCoordsAndDist()
	{
		return GetClothCollection()->GetElements(
			GetClothCollection()->GetRenderDeformerNormalBaryCoordsAndDist(),
			GetClothCollection()->GetRenderVerticesStart(),
			GetClothCollection()->GetRenderVerticesEnd(),
			GetElementIndex());
	}

	TArrayView<TArray<FVector4f>> FCollectionClothRenderPatternFacade::GetRenderDeformerTangentBaryCoordsAndDist()
	{
		return GetClothCollection()->GetElements(
			GetClothCollection()->GetRenderDeformerTangentBaryCoordsAndDist(),
			GetClothCollection()->GetRenderVerticesStart(),
			GetClothCollection()->GetRenderVerticesEnd(),
			GetElementIndex());
	}

	TArrayView<TArray<FIntVector3>> FCollectionClothRenderPatternFacade::GetRenderDeformerSimIndices3D()
	{
		return GetClothCollection()->GetElements(
			GetClothCollection()->GetRenderDeformerSimIndices3D(),
			GetClothCollection()->GetRenderVerticesStart(),
			GetClothCollection()->GetRenderVerticesEnd(),
			GetElementIndex());
	}

	TArrayView<TArray<float>> FCollectionClothRenderPatternFacade::GetRenderDeformerWeight()
	{
		return GetClothCollection()->GetElements(
			GetClothCollection()->GetRenderDeformerWeight(),
			GetClothCollection()->GetRenderVerticesStart(),
			GetClothCollection()->GetRenderVerticesEnd(),
			GetElementIndex());
	}

	TArrayView<float> FCollectionClothRenderPatternFacade::GetRenderDeformerSkinningBlend()
	{
		return GetClothCollection()->GetElements(
			GetClothCollection()->GetRenderDeformerSkinningBlend(),
			GetClothCollection()->GetRenderVerticesStart(),
			GetClothCollection()->GetRenderVerticesEnd(),
			GetElementIndex());
	}

	TArrayView<float> FCollectionClothRenderPatternFacade::GetRenderCustomResizingBlend()
	{
		return GetClothCollection()->GetElements(
			GetClothCollection()->GetRenderCustomResizingBlend(),
			GetClothCollection()->GetRenderVerticesStart(),
			GetClothCollection()->GetRenderVerticesEnd(),
			GetElementIndex());
	}

	void FCollectionClothRenderPatternFacade::SetNumRenderFaces(int32 NumRenderFaces)
	{
		GetClothCollection()->SetNumElements(
			NumRenderFaces,
			ClothCollectionGroup::RenderFaces,
			GetClothCollection()->GetRenderFacesStart(),
			GetClothCollection()->GetRenderFacesEnd(),
			GetElementIndex());
	}

	void FCollectionClothRenderPatternFacade::RemoveRenderFaces(const TArray<int32>& SortedDeletionList)
	{
		TArray<int32> GlobalIndexSortedDeletionList;
		const int32 Offset = GetRenderFacesOffset();
		GlobalIndexSortedDeletionList.SetNumUninitialized(SortedDeletionList.Num());
		for (int32 Idx = 0; Idx < SortedDeletionList.Num(); ++Idx)
		{
			GlobalIndexSortedDeletionList[Idx] = SortedDeletionList[Idx] + Offset;
		}

		GetClothCollection()->RemoveElements(
			ClothCollectionGroup::RenderFaces,
			GlobalIndexSortedDeletionList,
			GetClothCollection()->GetRenderFacesStart(),
			GetClothCollection()->GetRenderFacesEnd(),
			GetElementIndex());
	}

	TArrayView<FIntVector3> FCollectionClothRenderPatternFacade::GetRenderIndices()
	{
		return GetClothCollection()->GetElements(
			GetClothCollection()->GetRenderIndices(),
			GetClothCollection()->GetRenderFacesStart(),
			GetClothCollection()->GetRenderFacesEnd(),
			GetElementIndex());
	}

	FCollectionClothRenderPatternFacade::FCollectionClothRenderPatternFacade(const TSharedRef<FClothCollection>& ClothCollection, int32 InPatternIndex)
		: FCollectionClothRenderPatternConstFacade(ClothCollection, InPatternIndex)
	{
	}

	void FCollectionClothRenderPatternFacade::SetDefaults()
	{
		const int32 ElementIndex = GetElementIndex();

		(*GetClothCollection()->GetRenderVerticesStart())[ElementIndex] = INDEX_NONE;
		(*GetClothCollection()->GetRenderVerticesEnd())[ElementIndex] = INDEX_NONE;
		(*GetClothCollection()->GetRenderFacesStart())[ElementIndex] = INDEX_NONE;
		(*GetClothCollection()->GetRenderFacesEnd())[ElementIndex] = INDEX_NONE;
	}

	TSharedRef<class FClothCollection> FCollectionClothRenderPatternFacade::GetClothCollection()
	{
		return StaticCastSharedRef<class FClothCollection>(ConstCastSharedRef<class FConstClothCollection>(ClothCollection));
	}
}  // End namespace UE::Chaos::ClothAsset
