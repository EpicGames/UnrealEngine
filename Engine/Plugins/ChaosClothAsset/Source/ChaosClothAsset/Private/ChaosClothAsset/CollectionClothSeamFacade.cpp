// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/CollectionClothSeamFacade.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "ChaosClothAsset/ClothCollection.h"
#include "ChaosClothAsset/ClothCollectionGroup.h"
#include "ChaosClothAsset/ClothGeometryTools.h"
#include "Containers/Map.h"

namespace UE::Chaos::ClothAsset
{
#if DO_ENSURE
static bool bEnableSeamChecks = false;
static FAutoConsoleVariableRef CVarEnableSeamChecks(TEXT("p.ChaosCloth.EnableSeamChecks"), bEnableSeamChecks, TEXT("Enable seam validation checks"));
#endif
	namespace Private
	{
		//
		// Welding
		//
		typedef TMap<int32, int32> FWeldingGroup; // Key = Index, Value = Weight;

		static int32 WeldingMappedValue(const TMap<int32, int32>& WeldingMap, int32 Index)
		{
			const int32* Value = WeldingMap.Find(Index);
			if (!Value)
			{
				return Index;
			}
			return *Value;
		}

		static void UpdateWeldingMap(TMap<int32, int32>& WeldingMap, TMap<int32, FWeldingGroup>& WeldingGroups, int32 Index0, int32 Index1, const TConstArrayView<TArray<int32>>& SimVertex2DLookup)
		{
			// Update WeldingMap
			int32 Key0 = WeldingMappedValue(WeldingMap, Index0); // These might be swapped in the welding process
			int32 Key1 = WeldingMappedValue(WeldingMap, Index1);

			auto CountValidIndices = [](const TArray<int32>& Indices)
			{
				int32 Count = 0;
				for (int32 Index : Indices)
				{
					if (Index != INDEX_NONE)
					{
						++Count;
					}
				}
				return Count;
			};


			// Only process pairs that are not already redirected to the same index
			if (Key0 != Key1)
			{
				// Make sure Index0 points to the the smallest redirected index, so that merges are done into the correct group
				if (Key0 > Key1)
				{
					Swap(Key0, Key1);
					Swap(Index0, Index1);
				}

				// Find the group for Index0 if any
				FWeldingGroup* WeldingGroup0 = WeldingGroups.Find(Key0);
				if (!WeldingGroup0)
				{
					// No existing group, create a new one
					check(Key0 == Index0);  // No group means this index can't already have been redirected  // TODO: Make this a checkSlow
					const int32 Weight0 = CountValidIndices(SimVertex2DLookup[Index0]);
					check(Weight0 > 0);
					WeldingGroup0 = &WeldingGroups.Add(Key0);
					WeldingGroup0->Add({ Index0, Weight0 });
				}

				// Find the group for Index1, if it exists merge the two groups
				if (FWeldingGroup* const WeldingGroup1 = WeldingGroups.Find(Key1))
				{
					// Update group1 redirected indices with the new key
					for (TPair<int32, int32>& IndexAndWeight : *WeldingGroup1)
					{
						WeldingMap.FindOrAdd(IndexAndWeight.Get<0>()) = Key0;  // Could be a source index for which there isn't a WeldingMap entry and needs adding
					}

					// Merge group0 & group1
					WeldingGroup0->Append(*WeldingGroup1);

					// Remove group1
					WeldingGroups.Remove(Key1);

					// Sanity check
					check(WeldingGroup0->Contains(Key0) && WeldingGroup0->Contains(Key1));  // TODO: Make this a checkSlow
				}
				else
				{
					// Otherwise add Index1 to Index0's group
					check(Key1 == Index1);  // No group means this index can't already have been redirected  // TODO: Make this a checkSlow
					const int32 Weight1 = CountValidIndices(SimVertex2DLookup[Index1]);
					check(Weight1 > 0);
					WeldingMap.Add(Index1, Key0);
					WeldingGroup0->Add({ Index1, Weight1 });
				}
			}
		}

		// This is used for SimVertex3D <--> SimVertex2D, as well as SimVertex3D <--> SeamStitch
		static void UpdateWeldingLookups(const TMap<int32, FWeldingGroup>& WeldingGroups, const TArrayView<int32>& SimVertex3DLookup, const TArrayView<TArray<int32>>& ReverseLookup)
		{
			for (TMap<int32, FWeldingGroup>::TConstIterator GroupIter = WeldingGroups.CreateConstIterator(); GroupIter; ++GroupIter)
			{
				const int32 PrimaryIndex3D = GroupIter.Key();
				TArray<int32>& PrimaryReverseLookup = ReverseLookup[PrimaryIndex3D];
				for (const TPair<int32, int32>& IndexAndWeight : GroupIter.Value())
				{
					const TArray<int32>& MergingReverseLookup = ReverseLookup[IndexAndWeight.Get<0>()];
					for (const int32 ReverseIndex : MergingReverseLookup)
					{
						if (ReverseIndex != INDEX_NONE)
						{
							// All elements that used to point to us need to point to PrimaryIndex3D
							SimVertex3DLookup[ReverseIndex] = PrimaryIndex3D;
							PrimaryReverseLookup.AddUnique(ReverseIndex);
						}
					}
				}
			}
		}

		template<typename T>
		void WeldByWeightedAverage(const TMap<int32, FWeldingGroup>& WeldingGroups, const TArrayView<T>& Values)
		{
			for (TMap<int32, FWeldingGroup>::TConstIterator GroupIter = WeldingGroups.CreateConstIterator(); GroupIter; ++GroupIter)
			{
				T WeldedValue(0);
				int32 SourceCount = 0;
				for (const TPair<int32, int32>& IndexAndWeight : GroupIter.Value())
				{
					WeldedValue += Values[IndexAndWeight.Get<0>()] * IndexAndWeight.Get<1>();
					SourceCount += IndexAndWeight.Get<1>();
				}
				check(SourceCount > 0);
				Values[GroupIter.Key()] = WeldedValue / (float)SourceCount;
			}
		}

		static void WeldNormals(const TMap<int32, FWeldingGroup>& WeldingGroups, const TArrayView<FVector3f>& Normals)
		{
			for (TMap<int32, FWeldingGroup>::TConstIterator GroupIter = WeldingGroups.CreateConstIterator(); GroupIter; ++GroupIter)
			{
				FVector3f WeldedNormal(0);
				for (const TPair<int32, int32>& IndexAndWeight : GroupIter.Value())
				{
					WeldedNormal += Normals[IndexAndWeight.Get<0>()] * IndexAndWeight.Get<1>();
				}
				Normals[GroupIter.Key()] = WeldedNormal.GetSafeNormal(UE_SMALL_NUMBER, FVector3f::XAxisVector);
			}
		}

		template<bool bNormalizeFloats, int8 MaxNumElements, typename FCompareFunc>
		static void WeldIndexAndFloatArrays(const TMap<int32, FWeldingGroup>& WeldingGroups, TArrayView<TArray<int32>> IndicesArray, TArrayView<TArray<float>> FloatsArray, FCompareFunc CompareFunc)
		{
			for (TMap<int32, FWeldingGroup>::TConstIterator GroupIter = WeldingGroups.CreateConstIterator(); GroupIter; ++GroupIter)
			{
				TMap<int32, TPair<float, int32>> WeldedData;
				for (const TPair<int32, int32>& IndexAndWeight : GroupIter.Value())
				{
					const TArray<int32>& Indices = IndicesArray[IndexAndWeight.Get<0>()];
					const TArray<float>& Floats = FloatsArray[IndexAndWeight.Get<0>()];
					check(Indices.Num() == Floats.Num());
					for (int32 Idx = 0; Idx < Indices.Num(); ++Idx)
					{
						TPair<float, int32>& WeightedFloat = WeldedData.FindOrAdd(Indices[Idx]);
						WeightedFloat.Get<0>() += Floats[Idx] * IndexAndWeight.Get<1>();
						WeightedFloat.Get<1>() += IndexAndWeight.Get<1>();
					}
				}
				TArray<int32>& IndicesToWrite = IndicesArray[GroupIter.Key()];
				TArray<float>& FloatsToWrite = FloatsArray[GroupIter.Key()];
				IndicesToWrite.Reset(WeldedData.Num());
				FloatsToWrite.Reset(WeldedData.Num());
				float FloatsSum = 0.f;
				for (TMap<int32, TPair<float, int32>>::TConstIterator WeldedDataIter = WeldedData.CreateConstIterator(); WeldedDataIter; ++WeldedDataIter)
				{
					check(WeldedDataIter.Value().Get<1>() > 0);
					IndicesToWrite.Add(WeldedDataIter.Key());
					const float FloatVal = WeldedDataIter.Value().Get<0>() / (float)WeldedDataIter.Value().Get<1>();
					FloatsToWrite.Add(FloatVal);
					FloatsSum += FloatVal;
				}
				if (IndicesToWrite.Num() > MaxNumElements)
				{
					TArray<TPair<float, int32>> SortableData;
					SortableData.Reserve(IndicesToWrite.Num());
					for (int32 Idx = 0; Idx < IndicesToWrite.Num(); ++Idx)
					{
						SortableData.Emplace(FloatsToWrite[Idx], IndicesToWrite[Idx]);
					}
					SortableData.Sort(CompareFunc);
					IndicesToWrite.SetNum(MaxNumElements);
					FloatsToWrite.SetNum(MaxNumElements);
					FloatsSum = 0.f;
					for (int32 Idx = 0; Idx < MaxNumElements; ++Idx)
					{
						IndicesToWrite[Idx] = SortableData[Idx].Get<1>();
						FloatsToWrite[Idx] = SortableData[Idx].Get<0>();
						FloatsSum += SortableData[Idx].Get<0>();
					}
				}

				if (bNormalizeFloats)
				{
					const float FloatsSumRecip = FloatsSum > UE_SMALL_NUMBER ? 1.f / FloatsSum : 0.f;
					for (float& Float : FloatsToWrite)
					{
						Float *= FloatsSumRecip;
					}
				}
			}
		}

		static void WeldTethers(const TMap<int32, int32>& WeldingMap, const TMap<int32, FWeldingGroup>& WeldingGroups, TArrayView<TArray<int32>> TetherKinematicIndices, TArrayView<TArray<float>> TetherReferenceLengths)
		{
			// Weld kinematic indices. Clean up any INDEX_NONEs out there created by removing 3d verts as well.
			// MaxNumAttachments in tether creation code is 4. Welding can introduce more than this,
			// but this is the magnitude of lengths we're talking about in these TArrays when we're 
			// doing things like linear lookups, resizes, etc.
			check(TetherKinematicIndices.Num() == TetherReferenceLengths.Num());
			const int32 NumVertices = TetherKinematicIndices.Num();
			for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
			{
				TArray<int32>& Indices = TetherKinematicIndices[VertexIndex];
				TArray<float>& Lengths = TetherReferenceLengths[VertexIndex];

				check(Indices.Num() == Lengths.Num());
				const int32 NumTethers = Indices.Num();
				// Go in reverse because we're going to remove any invalid tethers while we're here.
				for (int32 TetherIndex = NumTethers - 1; TetherIndex >= 0; --TetherIndex)
				{
					if (Indices[TetherIndex] == INDEX_NONE)
					{
						Indices.RemoveAtSwap(TetherIndex);
						Lengths.RemoveAtSwap(TetherIndex);
						continue;
					}
					const int32 MappedIndex = WeldingMappedValue(WeldingMap, Indices[TetherIndex]);
					if (MappedIndex != Indices[TetherIndex])
					{
						// Check if this mapped index is already a kinematic index
						const int32 MappedTetherIndex = Indices.Find(MappedIndex);
						if (MappedTetherIndex == INDEX_NONE)
						{
							// It doesn't. Just update the index.
							Indices[TetherIndex] = MappedIndex;
							continue;
						}
						// Merge the two tethers
						const FWeldingGroup& WeldingGroup = WeldingGroups.FindChecked(MappedIndex);
						const int32 WeightOrig = WeldingGroup.FindChecked(Indices[TetherIndex]);
						const int32 WeightMapped = WeldingGroup.FindChecked(MappedIndex);
						check(WeightOrig + WeightMapped > 0);
						Lengths[MappedTetherIndex] = (Lengths[TetherIndex] * WeightOrig + Lengths[MappedTetherIndex] * WeightMapped) / (WeightOrig + WeightMapped);
						Indices.RemoveAtSwap(TetherIndex);
						Lengths.RemoveAtSwap(TetherIndex);
					}
				}
			}

			// Now weld dynamic indices
			WeldIndexAndFloatArrays<false, FClothCollection::MaxNumTetherAttachments>(WeldingGroups, TetherKinematicIndices, TetherReferenceLengths,
				[](const TPair<float, int32>& A, const TPair<float, int32>& B) { return A < B; });
		}

		//
		// Splitting
		//
		struct FSplittingGroup
		{
			int32 NewVertex3DIndex = INDEX_NONE;
			TSet<int32> Vertex2Ds;
			TSet<int32> SeamStitches;
		};
		typedef TMap<int32, TArray<FSplittingGroup>> FSplittingGroups; // Key = Original 3D vertex

		static void UpdateSplittingGroup(TArray<FSplittingGroup>& SplittingGroup, const int32 VertexIndex3D, const TArray<int32>& RemainingStitches, const TConstArrayView<FIntVector2>& SeamStitch2DEndIndices,
			const TArray<int32>& SimVertex2DLookup)
		{				
			// Form groups of 2D vertices that should stay welded by remaining stitches.
			for (const int32 RemainingStitch : RemainingStitches)
			{
				if (RemainingStitch != INDEX_NONE)
				{
					const FIntVector2& Global2DEndPoints = SeamStitch2DEndIndices[RemainingStitch];
					int32 FirstExistingGroup = INDEX_NONE;
					for(int32 GrpIdx = 0; GrpIdx < SplittingGroup.Num(); ++GrpIdx)
					{
						FSplittingGroup& Group = SplittingGroup[GrpIdx];
						if (Group.Vertex2Ds.Contains(Global2DEndPoints[0]) || Group.Vertex2Ds.Contains(Global2DEndPoints[1]))
						{
							// Make sure both ends are in this group
							Group.Vertex2Ds.Add(Global2DEndPoints[0]);
							Group.Vertex2Ds.Add(Global2DEndPoints[1]);
							Group.SeamStitches.Add(RemainingStitch);

							if (FirstExistingGroup != INDEX_NONE)
							{
								// This stitch connects the two groups. Merge them.
								FSplittingGroup& OtherGroup = SplittingGroup[FirstExistingGroup];
								Group.SeamStitches.Append(OtherGroup.SeamStitches);
								Group.Vertex2Ds.Append(OtherGroup.Vertex2Ds);
								SplittingGroup.RemoveAtSwap(FirstExistingGroup);
								break;
							}
							else
							{
								FirstExistingGroup = GrpIdx;
							}
						}
					}
					if (FirstExistingGroup == INDEX_NONE)
					{
						// Add new group with this seam.
						FSplittingGroup& NewGroup = SplittingGroup.AddDefaulted_GetRef();
						NewGroup.Vertex2Ds.Add(Global2DEndPoints[0]);
						NewGroup.Vertex2Ds.Add(Global2DEndPoints[1]);
						NewGroup.SeamStitches.Add(RemainingStitch);
					}
				}
			}

			// Create new singleton groups for any un-stitched 2D vertices that correspond with this 3D vertex
			for (const int32 Vertex2D : SimVertex2DLookup)
			{
				if (Vertex2D != INDEX_NONE)
				{
					bool bFoundExistingGroup = false;
					for (FSplittingGroup& Group : SplittingGroup)
					{
						if (Group.Vertex2Ds.Contains(Vertex2D))
						{
							bFoundExistingGroup = true;
							break;
						}
					}
					if (!bFoundExistingGroup)
					{
						SplittingGroup.AddDefaulted_GetRef().Vertex2Ds.Add(Vertex2D);
					}
				}
			}
		}

		static void UpdateVertexLookupsAfterSplitting(const FSplittingGroups& SplittingGroups, TArrayView<int32> SimVertex3DLookup, TArrayView<TArray<int32>> SimVertex2DLookup)
		{
			for (FSplittingGroups::TConstIterator GroupIter = SplittingGroups.CreateConstIterator(); GroupIter; ++GroupIter)
			{
				for (const FSplittingGroup& Group : GroupIter.Value())
				{
					SimVertex2DLookup[Group.NewVertex3DIndex] = Group.Vertex2Ds.Array();
					for (const int32 Vertex2D : SimVertex2DLookup[Group.NewVertex3DIndex])
					{
						SimVertex3DLookup[Vertex2D] = Group.NewVertex3DIndex;
					}
				}
			}
		}

		static void UpdateSeamStitchLookupsAfterSplitting(const FSplittingGroups& SplittingGroups, TArrayView<int32> SeamStitch3DIndex, TArrayView<TArray<int32>> SeamStitchLookup)
		{
			for (FSplittingGroups::TConstIterator GroupIter = SplittingGroups.CreateConstIterator(); GroupIter; ++GroupIter)
			{
				for (const FSplittingGroup& Group : GroupIter.Value())
				{
					SeamStitchLookup[Group.NewVertex3DIndex] = Group.SeamStitches.Array();
					for (const int32 SeamStitch : SeamStitchLookup[Group.NewVertex3DIndex])
					{
						SeamStitch3DIndex[SeamStitch] = Group.NewVertex3DIndex;
					}
				}
			}
		}

		// Simple splitting by just duplicating data to newly split vertices
		template<typename T>
		static void SplitCopyVertexData(const FSplittingGroups& SplittingGroups, TArrayView<T> Values)
		{
			for (FSplittingGroups::TConstIterator GroupIter = SplittingGroups.CreateConstIterator(); GroupIter; ++GroupIter)
			{
				const T& OrigValue = Values[GroupIter.Key()];
				for (int32 GroupIdx = 1; GroupIdx < GroupIter.Value().Num(); ++GroupIdx)
				{
					Values[GroupIter.Value()[GroupIdx].NewVertex3DIndex] = OrigValue;
				}
			}
		}

		static void SplitTethers(const FSplittingGroups& SplittingGroups, TArrayView<TArray<int32>> TetherKinematicIndices, TArrayView<TArray<float>> TetherReferenceLengths)
		{
			// Copy kinematic indices. 
			check(TetherKinematicIndices.Num() == TetherReferenceLengths.Num());
			const int32 NumVertices = TetherKinematicIndices.Num();
			for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
			{
				TArray<int32>& Indices = TetherKinematicIndices[VertexIndex];
				TArray<float>& Lengths = TetherReferenceLengths[VertexIndex];

				check(Indices.Num() == Lengths.Num());
				const int32 NumTethers = Indices.Num();
				// Go in reverse because we're going to remove any invalid tethers while we're here.
				for (int32 TetherIndex = NumTethers - 1; TetherIndex >= 0; --TetherIndex)
				{
					if (Indices[TetherIndex] == INDEX_NONE)
					{
						Indices.RemoveAtSwap(TetherIndex);
						Lengths.RemoveAtSwap(TetherIndex);
						continue;
					}
					
					if (const TArray<FSplittingGroup>* SplittingGroup = SplittingGroups.Find(Indices[TetherIndex]))
					{
						check(SplittingGroup->Num() > 1);
						// Need to duplicate this tether.
						Indices.Reserve(Indices.Num() + SplittingGroup->Num() - 1);
						Lengths.Reserve(Indices.Num() + SplittingGroup->Num() - 1);
						const float Length = Lengths[TetherIndex];
						for (int32 GroupIdx = 1; GroupIdx < SplittingGroup->Num(); ++GroupIdx)
						{
							Indices.Add((*SplittingGroup)[GroupIdx].NewVertex3DIndex);
							Lengths.Add(Length);
						}
					}
				}

				if (Indices.Num() > FClothCollection::MaxNumTetherAttachments)
				{
					// Remove farthest tethers
					TArray<TPair<float, int32>> SortableData;
					SortableData.Reserve(Indices.Num());
					for (int32 Idx = 0; Idx < Indices.Num(); ++Idx)
					{
						SortableData.Emplace(Lengths[Idx], Indices[Idx]);
					}
					SortableData.Sort([](const TPair<float, int32>& A, const TPair<float, int32>& B) { return A < B; });
					Indices.SetNum(FClothCollection::MaxNumTetherAttachments);
					Lengths.SetNum(FClothCollection::MaxNumTetherAttachments);
					for (int32 Idx = 0; Idx < FClothCollection::MaxNumTetherAttachments; ++Idx)
					{
						Indices[Idx] = SortableData[Idx].Get<1>();
						Lengths[Idx] = SortableData[Idx].Get<0>();
					}
				}
			}

			// Now split dynamic indices
			SplitCopyVertexData(SplittingGroups, TetherKinematicIndices);
			SplitCopyVertexData(SplittingGroups, TetherReferenceLengths);
		}

		static void UpdateSimIndicesAfterSplitting(const TConstArrayView<FIntVector3>& SimIndices2D, const TConstArrayView<int32>& SimVertex3DLookup, TArrayView<FIntVector3> SimIndices3D)
		{
			// Since we don't have a vertex -> face lookup, it's faster to just regenerate all of the sim indices
			check(SimIndices2D.Num() == SimIndices3D.Num());
			for (int32 FaceIndex = 0; FaceIndex < SimIndices3D.Num(); ++FaceIndex)
			{
				SimIndices3D[FaceIndex][0] = SimIndices2D[FaceIndex][0] == INDEX_NONE ? INDEX_NONE : SimVertex3DLookup[SimIndices2D[FaceIndex][0]];
				SimIndices3D[FaceIndex][1] = SimIndices2D[FaceIndex][1] == INDEX_NONE ? INDEX_NONE : SimVertex3DLookup[SimIndices2D[FaceIndex][1]];
				SimIndices3D[FaceIndex][2] = SimIndices2D[FaceIndex][2] == INDEX_NONE ? INDEX_NONE : SimVertex3DLookup[SimIndices2D[FaceIndex][2]];
			}
		}

		// Cleanup methods
		TArray<TSet<int32>> BuildStitchGroupsForVertex3D(const TArray<int32>& StitchLookup, const TArray<int32>& Vertex2DLookup, const TArrayView<FIntVector2>& AllEnds, const int32 OurIndex, bool& bFoundOtherInvalidStitches)
		{
			bFoundOtherInvalidStitches = false;
			TArray<TSet<int32>> Vertex2DGroups;
			for (int32 OtherStitchIndex : StitchLookup)
			{
				if (OtherStitchIndex == OurIndex || OtherStitchIndex == INDEX_NONE)
				{
					// This is us
					continue;
				}
				const FIntVector2& OtherStitchEnds = AllEnds[OtherStitchIndex];
				if (OtherStitchEnds[0] == INDEX_NONE || OtherStitchEnds[1] == INDEX_NONE)
				{
					bFoundOtherInvalidStitches = true;
					continue;
				}
				int32 GroupIndex0 = INDEX_NONE;
				int32 GroupIndex1 = INDEX_NONE;
				for (int32 GroupIndex = 0; GroupIndex < Vertex2DGroups.Num(); ++GroupIndex)
				{
					if (Vertex2DGroups[GroupIndex].Contains(OtherStitchEnds[0]))
					{
						GroupIndex0 = GroupIndex;
					}
					if (Vertex2DGroups[GroupIndex].Contains(OtherStitchEnds[1]))
					{
						GroupIndex1 = GroupIndex;
					}
					else
					{
						continue;
					}
					if (GroupIndex0 != INDEX_NONE && GroupIndex1 != INDEX_NONE)
					{
						break;
					}
				}

				if (GroupIndex0 == INDEX_NONE)
				{
					if (GroupIndex1 == INDEX_NONE)
					{
						// Start a new group
						Vertex2DGroups.Emplace(TSet<int32>({ OtherStitchEnds[0], OtherStitchEnds[1]}));
					}
					else
					{
						// Add to Group1
						Vertex2DGroups[GroupIndex1].Add(OtherStitchEnds[0]);
					}
				}
				else if (GroupIndex1 == INDEX_NONE)
				{
					Vertex2DGroups[GroupIndex0].Add(OtherStitchEnds[1]);
				}
				else
				{
					// Both in different groups. Merge them.
					Vertex2DGroups[GroupIndex0].Append(Vertex2DGroups[GroupIndex1]);
					Vertex2DGroups.RemoveAtSwap(GroupIndex1);
				}
			}

			// Add all Vertex2D that aren't in one any groups
			for (const int32 Vertex2D : Vertex2DLookup)
			{
				if (Vertex2D == INDEX_NONE)
				{
					continue;
				}
				bool bFoundInGroup = false;
				for (const TSet<int32>& Group : Vertex2DGroups)
				{
					if (Group.Contains(Vertex2D))
					{
						bFoundInGroup = true;
						break;
					}
				}
				if (!bFoundInGroup)
				{
					Vertex2DGroups.Emplace(TSet<int32>({ Vertex2D }));
				}
			}
			return Vertex2DGroups;
		}

		TArray<FIntVector2> GenerateRemainderStitchesAfterCleanup(const TArray<TSet<int32>>& Vertex2DGroups)
		{
			TArray<FIntVector2> Stitches;
			if (Vertex2DGroups.Num() > 1)
			{
				Stitches.Reserve(Vertex2DGroups.Num() - 1);
				FIntVector2* Stitch = &Stitches.AddDefaulted_GetRef();
				(*Stitch)[0] = *Vertex2DGroups[0].CreateConstIterator();
				for (int32 GroupIndex = 1; GroupIndex < Vertex2DGroups.Num(); ++GroupIndex)
				{
					(*Stitch)[1] = *Vertex2DGroups[GroupIndex].CreateConstIterator();
					if (GroupIndex < Vertex2DGroups.Num() - 1)
					{
						const int32 NextEnd = (*Stitch)[1];
						Stitch = &Stitches.AddDefaulted_GetRef();
						(*Stitch)[0] = NextEnd;
					}
				}
			}
			return Stitches;
		}
	} // namespace Private

	int32 FCollectionClothSeamConstFacade::GetNumSeamStitches() const
	{
		return ClothCollection->GetNumElements(
			ClothCollection->GetSeamStitchStart(),
			ClothCollection->GetSeamStitchEnd(),
			GetElementIndex());
	}

	int32 FCollectionClothSeamConstFacade::GetSeamStitchesOffset() const
	{
		return ClothCollection->GetElementsOffset(
			ClothCollection->GetSeamStitchStart(),
			GetBaseElementIndex(),
			GetElementIndex());
	}

	TConstArrayView<FIntVector2> FCollectionClothSeamConstFacade::GetSeamStitch2DEndIndices() const
	{
		return ClothCollection->GetElements(
			ClothCollection->GetSeamStitch2DEndIndices(),
			ClothCollection->GetSeamStitchStart(),
			ClothCollection->GetSeamStitchEnd(),
			GetElementIndex());
	}

	TConstArrayView<int32> FCollectionClothSeamConstFacade::GetSeamStitch3DIndex() const
	{
		return ClothCollection->GetElements(
			ClothCollection->GetSeamStitch3DIndex(),
			ClothCollection->GetSeamStitchStart(),
			ClothCollection->GetSeamStitchEnd(),
			GetElementIndex());
	}
	
	void FCollectionClothSeamConstFacade::ValidateSeam() const
	{
#if DO_ENSURE
		if (bEnableSeamChecks)
		{
			FCollectionClothConstFacade Cloth(ClothCollection);
			TConstArrayView<FIntVector2> Ends = GetSeamStitch2DEndIndices();
			TConstArrayView<int32> Index3D = GetSeamStitch3DIndex();
			TConstArrayView<TArray<int32>> Lookup2D = Cloth.GetSimVertex2DLookup();
			TConstArrayView<TArray<int32>> StitchLookup = Cloth.GetSeamStitchLookup();
			TConstArrayView<int32> Lookup3D = Cloth.GetSimVertex3DLookup();

			const int32 StitchOffset = GetSeamStitchesOffset();
			for (int32 StitchIndex = 0; StitchIndex < GetNumSeamStitches(); ++StitchIndex)
			{
				if (Ends[StitchIndex][0] != INDEX_NONE && Ends[StitchIndex][1] != INDEX_NONE)
				{
					if (Index3D[StitchIndex] != INDEX_NONE)
					{
						ensureAlways(StitchLookup[Index3D[StitchIndex]].Contains(StitchOffset + StitchIndex));
						ensureAlways(Lookup2D[Index3D[StitchIndex]].Contains(Ends[StitchIndex][0]));
						ensureAlways(Lookup2D[Index3D[StitchIndex]].Contains(Ends[StitchIndex][1]));
						ensureAlways(Lookup3D[Ends[StitchIndex][0]] == Index3D[StitchIndex]);
						ensureAlways(Lookup3D[Ends[StitchIndex][1]] == Index3D[StitchIndex]);
					}
				}
			}
		}
#endif
	}

	FCollectionClothSeamConstFacade::FCollectionClothSeamConstFacade(const TSharedRef<const FConstClothCollection>& ClothCollection, int32 SeamIndex)
		: ClothCollection(ClothCollection)
		, SeamIndex(SeamIndex)
	{
		check(ClothCollection->IsValid());
		check(SeamIndex >= 0 && SeamIndex < ClothCollection->GetNumElements(ClothCollectionGroup::Seams));
	}

	void FCollectionClothSeamFacade::Reset()
	{
		using namespace Private;

		const int32 OrigNumSeamStitches = GetNumSeamStitches();

		// Split all seams by duplicating points.
		if (OrigNumSeamStitches > 0)
		{
			FCollectionClothFacade Cloth(GetClothCollection());

			const int32 GlobalSeamStitchesOffset = GetSeamStitchesOffset();
			const TArrayView<int32> SeamStitch3DIndex = GetSeamStitch3DIndex();

			const TArrayView<TArray<int32>> SeamStitchLookup = Cloth.GetSeamStitchLookupPrivate();
			const TConstArrayView<FIntVector2> GlobalSeamStitch2DEndIndices = ClothCollection->GetElements(ClothCollection->GetSeamStitch2DEndIndices());
			const TConstArrayView<TArray<int32>> SimVertex2DLookup = Cloth.GetSimVertex2DLookup();

			const int32 OrigNumSimVertices3D = Cloth.GetNumSimVertices3D();
			int32 NumNewSimVertices3D = 0;

			FSplittingGroups SplittingGroups;

			// Gather all 3D vertices that might need to be split
			for (int32 StitchIndex = 0; StitchIndex < OrigNumSeamStitches; ++StitchIndex)
			{
				const int32 VertexIndex3D = SeamStitch3DIndex[StitchIndex];
				if (VertexIndex3D != INDEX_NONE)
				{
					// Break this stitch
					verify(SeamStitchLookup[VertexIndex3D].RemoveSwap(StitchIndex + GlobalSeamStitchesOffset) == 1);

					// Add this to the splitting groups to be processed
					SplittingGroups.FindOrAdd(VertexIndex3D);
				}
			}

			// Process splitting groups
			for (FSplittingGroups::TIterator GroupIter = SplittingGroups.CreateIterator(); GroupIter; ++GroupIter)
			{
				const int32 VertexIndex3D = GroupIter.Key();

				UpdateSplittingGroup(GroupIter.Value(), GroupIter.Key(), SeamStitchLookup[VertexIndex3D], GlobalSeamStitch2DEndIndices, SimVertex2DLookup[VertexIndex3D]);

				check(GroupIter.Value().Num() > 0);

				if (GroupIter.Value().Num() == 1)
				{
					// This vertex actually doesn't need to split.
					GroupIter.RemoveCurrent();
					continue;
				}

				GroupIter.Value()[0].NewVertex3DIndex = VertexIndex3D; // First group will stay at the original 3D index.
				for (int32 GroupIdx = 1; GroupIdx < GroupIter.Value().Num(); ++GroupIdx)
				{
					GroupIter.Value()[GroupIdx].NewVertex3DIndex = OrigNumSimVertices3D + NumNewSimVertices3D++;
				}
			}

			if (NumNewSimVertices3D > 0)
			{	
				//
				// Add the 3D vertices and split all of their data.
				//

				// This resize will invalidate any existing SimVertex3DGroup ArrayViews!
				GetClothCollection()->SetNumElements(OrigNumSimVertices3D + NumNewSimVertices3D, ClothCollectionGroup::SimVertices3D);

				// Update 2D <--> 3D lookups
				UpdateVertexLookupsAfterSplitting(SplittingGroups, Cloth.GetSimVertex3DLookupPrivate(), Cloth.GetSimVertex2DLookupPrivate());

				// Update Stitch <-> 3D vertex lookups for stitches in other seams.
				UpdateSeamStitchLookupsAfterSplitting(SplittingGroups, GetClothCollection()->GetElements(GetClothCollection()->GetSeamStitch3DIndex()), Cloth.GetSeamStitchLookupPrivate());

				// Split 3D positions
				SplitCopyVertexData(SplittingGroups, Cloth.GetSimPosition3D());

				// Split normals
				SplitCopyVertexData(SplittingGroups, Cloth.GetSimNormal());

				// Split BoneIndices
				SplitCopyVertexData(SplittingGroups, Cloth.GetSimBoneIndices());

				// Split BoneWeights
				SplitCopyVertexData(SplittingGroups, Cloth.GetSimBoneWeights());

				// Split Tethers
				SplitTethers(SplittingGroups, Cloth.GetTetherKinematicIndex(), Cloth.GetTetherReferenceLength());

				// Split accessory mesh data
				for (int32 AccessoryMeshIndex = 0; AccessoryMeshIndex < Cloth.GetNumSimAccessoryMeshes(); ++AccessoryMeshIndex)
				{
					FCollectionClothSimAccessoryMeshFacade AccessoryMesh = Cloth.GetSimAccessoryMesh(AccessoryMeshIndex);

					// Split 3D positions
					SplitCopyVertexData(SplittingGroups, AccessoryMesh.GetSimAccessoryMeshPosition3D());

					// Split normals
					SplitCopyVertexData(SplittingGroups, AccessoryMesh.GetSimAccessoryMeshNormal());

					// Split BoneIndices
					SplitCopyVertexData(SplittingGroups, AccessoryMesh.GetSimAccessoryMeshBoneIndices());

					// Split BoneWeights
					SplitCopyVertexData(SplittingGroups, AccessoryMesh.GetSimAccessoryMeshBoneWeights());
				}

				// Split Faces
				// The easiest thing to do here is actually to regenerate the 3D faces from the 2D faces.
				UpdateSimIndicesAfterSplitting(Cloth.GetSimIndices2D(), Cloth.GetSimVertex3DLookup(), Cloth.GetSimIndices3D());

				// Split Maps
				const TArray<FName> WeightMapNames = Cloth.GetWeightMapNames();
				for (const FName& WeightMapName : WeightMapNames)
				{
					SplitCopyVertexData(SplittingGroups, Cloth.GetWeightMap(WeightMapName));
				}
			}
		}
		
		SetNumSeamStitches(0);
		SetDefaults();
	}

	void FCollectionClothSeamFacade::Initialize(TConstArrayView<FIntVector2> InStitches)
	{
		using namespace Private;

		Reset();

		FCollectionClothFacade Cloth(GetClothCollection());
		const int32 NumSimVertices2D = Cloth.GetNumSimVertices2D();
		// Do not add stitches between the same vertices. These just make bookkeeping hard and do nothing.
		TArray<FIntVector2> Stitches;
		Stitches.Reserve(InStitches.Num());
		for (FIntVector2 Stitch : InStitches)
		{
			if (Stitch[0] >= 0 && Stitch[1] >= 0 && 
				Stitch[0] < NumSimVertices2D && Stitch[1] < NumSimVertices2D && 
				Stitch[0] != Stitch[1])
			{
				Stitches.Add(Stitch);
			}
		}

		const int32 NumStitches = Stitches.Num();

		SetNumSeamStitches(NumStitches);

		const TArrayView<FIntVector2> SeamStitch2DEndIndices = GetSeamStitch2DEndIndices();
		const TArrayView<int32> SeamStitch3DIndex = GetSeamStitch3DIndex();

		const TConstArrayView<int32> SimVertex3DLookup = Cloth.GetSimVertex3DLookup();
		const TConstArrayView<TArray<int32>> SimVertex2DLookup = Cloth.GetSimVertex2DLookup();

		// The welding map redirects to an existing vertex index if these two are part of the same welding group.
		// The redirected index must be the smallest index in the group. If a key is not in the WeldingMap, it redirects to itself.
		TMap<int32, int32> WeldingMap;
		WeldingMap.Reserve(NumStitches);

		// Define welding groups
		// Welding groups contain all stitched pair of indices to be welded together that are required to build the welding map.
		// Key is the smallest redirected index in the group, and will be the one index used in the welding map redirects.
		TMap<int32, FWeldingGroup> WeldingGroups;
		for (int32 StitchIndex = 0; StitchIndex < NumStitches; ++StitchIndex)
		{
			const FIntVector2& Stitch = Stitches[StitchIndex];
			const FIntVector2 Curr3DIndices(SimVertex3DLookup[Stitch[0]], SimVertex3DLookup[Stitch[1]]);

			// Copy stitch into our data
			SeamStitch2DEndIndices[StitchIndex] = Stitch;
			SeamStitch3DIndex[StitchIndex] = Curr3DIndices[0];

			UpdateWeldingMap(WeldingMap, WeldingGroups, Curr3DIndices[0], Curr3DIndices[1], SimVertex2DLookup);
		}
		
		// Update SeamStitch3DIndex with redirected values
		// Add SeamStitch to SeamStitchLookup (reverse lookup to SeamStitch3DIndex)
		const int32 StitchOffset = GetSeamStitchesOffset();
		TArrayView<TArray<int32>> SeamStitchLookup = Cloth.GetSeamStitchLookupPrivate();
		for (int32 StitchIndex = 0; StitchIndex < NumStitches; ++StitchIndex)
		{
			SeamStitch3DIndex[StitchIndex] = WeldingMappedValue(WeldingMap, SeamStitch3DIndex[StitchIndex]);
			SeamStitchLookup[SeamStitch3DIndex[StitchIndex]].Add(StitchOffset + StitchIndex);
		}

		if (WeldingMap.IsEmpty())
		{
			// Nothing actually got welded, so we're done.
			return;
		}

		// Update 2D vs 3D lookups
		UpdateWeldingLookups(WeldingGroups, Cloth.GetSimVertex3DLookupPrivate(), Cloth.GetSimVertex2DLookupPrivate());

		// Weld Stitch <-> 3D vertex lookups for stitches in other seams.
		UpdateWeldingLookups(WeldingGroups, GetClothCollection()->GetElements(GetClothCollection()->GetSeamStitch3DIndex()), Cloth.GetSeamStitchLookupPrivate());

		// Weld 3D positions
		WeldByWeightedAverage(WeldingGroups, Cloth.GetSimPosition3D());

		// Weld normals
		WeldNormals(WeldingGroups, Cloth.GetSimNormal());

		// Weld BoneIndices and Weights
		WeldIndexAndFloatArrays<true, FClothCollection::MaxNumBoneInfluences>(WeldingGroups, Cloth.GetSimBoneIndices(), Cloth.GetSimBoneWeights(),
			[](const TPair<float, int32>& A, const TPair<float, int32>& B) { return A > B; });

		// Weld Tethers
		WeldTethers(WeldingMap, WeldingGroups, Cloth.GetTetherKinematicIndex(), Cloth.GetTetherReferenceLength());

		// Weld accessory mesh data
		for (int32 AccessoryMeshIndex = 0; AccessoryMeshIndex < Cloth.GetNumSimAccessoryMeshes(); ++AccessoryMeshIndex)
		{
			FCollectionClothSimAccessoryMeshFacade AccessoryMesh = Cloth.GetSimAccessoryMesh(AccessoryMeshIndex);

			// Weld 3D positions
			WeldByWeightedAverage(WeldingGroups, AccessoryMesh.GetSimAccessoryMeshPosition3D());

			// Weld normals
			WeldNormals(WeldingGroups, AccessoryMesh.GetSimAccessoryMeshNormal());

			// Weld BoneIndices and Weights
			WeldIndexAndFloatArrays<true, FClothCollection::MaxNumBoneInfluences>(WeldingGroups, AccessoryMesh.GetSimAccessoryMeshBoneIndices(), AccessoryMesh.GetSimAccessoryMeshBoneWeights(),
				[](const TPair<float, int32>& A, const TPair<float, int32>& B) { return A > B; });
		}

		// Weld Faces
		// Just go through all faces and fix up. We could store vertex -> face lookups, but we'd have to ensure they stay in sync
		for (FIntVector3& Index3D : Cloth.GetSimIndices3D())
		{
			Index3D[0] = WeldingMappedValue(WeldingMap, Index3D[0]);
			Index3D[1] = WeldingMappedValue(WeldingMap, Index3D[1]);
			Index3D[2] = WeldingMappedValue(WeldingMap, Index3D[2]);
		}

		// Weld the deformer's SimIndices3D
		for (TArray<FIntVector3>& SimIndices3D : Cloth.GetRenderDeformerSimIndices3D())
		{
			for (FIntVector3& Index3D : SimIndices3D)
			{
				Index3D[0] = WeldingMappedValue(WeldingMap, Index3D[0]);
				Index3D[1] = WeldingMappedValue(WeldingMap, Index3D[1]);
				Index3D[2] = WeldingMappedValue(WeldingMap, Index3D[2]);
			}
		}

		// Weld maps
		const TArray<FName> WeightMapNames = Cloth.GetWeightMapNames();
		for (const FName& WeightMapName : WeightMapNames)
		{
			WeldByWeightedAverage(WeldingGroups, Cloth.GetWeightMap(WeightMapName));
		}

		// Gather list of vertices to remove
		TArray<int32> VerticesToRemove;
		for (TMap<int32, int32>::TConstIterator WeldingMapIter = WeldingMap.CreateConstIterator(); WeldingMapIter; ++WeldingMapIter)
		{
			if (WeldingMapIter.Key() != WeldingMapIter.Value())
			{
				VerticesToRemove.Add(WeldingMapIter.Key());
			}
		}
		VerticesToRemove.Sort();
		GetClothCollection()->RemoveElements(ClothCollectionGroup::SimVertices3D, VerticesToRemove);

		ValidateSeam();
	}

	void FCollectionClothSeamFacade::Initialize(const FCollectionClothSeamConstFacade& Other, const int32 SimVertex2DOffset, const int32 SimVertex3DOffset)
	{
		SetNumSeamStitches(Other.GetNumSeamStitches());
		FClothCollection::CopyArrayViewDataAndApplyOffset(GetSeamStitch2DEndIndices(), Other.GetSeamStitch2DEndIndices(), FIntVector2(SimVertex2DOffset));
		FClothCollection::CopyArrayViewDataAndApplyOffset(GetSeamStitch3DIndex(), Other.GetSeamStitch3DIndex(), SimVertex3DOffset);
	}

	void FCollectionClothSeamFacade::CleanupAndCompact()
	{
		FCollectionClothFacade Cloth(GetClothCollection());
		TArrayView<FIntVector2> AllEnds = GetClothCollection()->GetElements(GetClothCollection()->GetSeamStitch2DEndIndices());
		TArrayView<int32> Index3D = GetSeamStitch3DIndex();
		TConstArrayView<TArray<int32>> SimVertex2DLookup = Cloth.GetSimVertex2DLookup();
		TConstArrayView<int32> Lookup3D = Cloth.GetSimVertex3DLookup();
		TArrayView<TArray<int32>> SeamStitchLookup = Cloth.GetSeamStitchLookupPrivate();

		const int32 StitchOffset = GetSeamStitchesOffset();
		TArray<int32> StitchesToRemoveGlobal;
		struct FNewStitches
		{
			int32 Vertex3D;
			TArray<FIntVector2> StitchEnds;
			FNewStitches(int32 InVertex3D, TArray<FIntVector2>&& InStitchEnds)
				: Vertex3D(InVertex3D), StitchEnds(MoveTemp(InStitchEnds))
			{}
		};
		TArray<FNewStitches> NewStitchesToAdd;
		for (int32 StitchIndex = 0; StitchIndex < GetNumSeamStitches(); ++StitchIndex)
		{
			if (Index3D[StitchIndex] == INDEX_NONE)
			{
				// This stitch is no longer stitching anything together. It can be removed.
				StitchesToRemoveGlobal.Add(StitchIndex + StitchOffset);
				continue;
			}

			FIntVector2& StitchEnds = AllEnds[StitchIndex + StitchOffset];
			if (StitchEnds[0] == INDEX_NONE || StitchEnds[1] == INDEX_NONE)
			{
				// Try to fix up the stitches that go to this 3D vertex 
				bool bFoundOtherInvalidStitches = false;
				TArray<TSet<int32>> Vertex2DGroups = Private::BuildStitchGroupsForVertex3D(SeamStitchLookup[Index3D[StitchIndex]], SimVertex2DLookup[Index3D[StitchIndex]], AllEnds, StitchIndex + StitchOffset, bFoundOtherInvalidStitches);

				if (Vertex2DGroups.Num() < 2)
				{
					// All 2D vertices going to this 3D vertex are stitched together. Can discard this stitch.
					StitchesToRemoveGlobal.Add(StitchIndex + StitchOffset);
					continue;
				}

				if (StitchEnds[1] != INDEX_NONE)
				{
					// Swap ends just to make next bits a little easier.
					StitchEnds[0] = StitchEnds[1];
					StitchEnds[1] = INDEX_NONE;
				}

				int32 GroupIndex0 = INDEX_NONE;
				if (StitchEnds[0] != INDEX_NONE)
				{
					// Find which group our first end is in.
					for (int32 GroupIndex = 0; GroupIndex < Vertex2DGroups.Num(); ++GroupIndex)
					{
						if (Vertex2DGroups[GroupIndex].Contains(StitchEnds[0]))
						{
							GroupIndex0 = GroupIndex;
							break;
						}
					}
				}
				else
				{
					GroupIndex0 = 0;
					StitchEnds[0] = *Vertex2DGroups[GroupIndex0].CreateConstIterator();
				}
				check(GroupIndex0 != INDEX_NONE);
				const int32 GroupIndex1 = GroupIndex0 == 0 ? 1 : 0;
				StitchEnds[1] = *Vertex2DGroups[GroupIndex1].CreateConstIterator();

				if (Vertex2DGroups.Num() > 2 && !bFoundOtherInvalidStitches)
				{
					Vertex2DGroups[GroupIndex0].Append(Vertex2DGroups[GroupIndex1]);
					Vertex2DGroups.RemoveAtSwap(GroupIndex1);
					// There are more groups to stitch together and there aren't any other invalid stitches that correspond with this 3D index that 
					// can be used to fix this up.
					NewStitchesToAdd.Emplace(Index3D[StitchIndex], Private::GenerateRemainderStitchesAfterCleanup(Vertex2DGroups));
				}
			}
		}

		// Determine how many remainder stitches we need to add.
		int32 NumStitchesToAdd = 0;
		for (const FNewStitches& NewStitches : NewStitchesToAdd)
		{
			NumStitchesToAdd += NewStitches.StitchEnds.Num();
		}

		// Reuse indices for stitches marked to remove to create new stitches.
		const int32 OrigNumStitches = GetNumSeamStitches();
		if (NumStitchesToAdd > StitchesToRemoveGlobal.Num())
		{
			SetNumSeamStitches(OrigNumStitches + NumStitchesToAdd - StitchesToRemoveGlobal.Num());
			// Allocating new stitches can cause the SeamStitches group pointers to change.
			AllEnds = GetClothCollection()->GetElements(GetClothCollection()->GetSeamStitch2DEndIndices());
			Index3D = GetSeamStitch3DIndex();
		}

		int32 NewStitchIndex = OrigNumStitches;
		for (const FNewStitches& NewStitches : NewStitchesToAdd)
		{
			for (const FIntVector2& StitchEnds : NewStitches.StitchEnds)
			{
				if (StitchesToRemoveGlobal.Num() > 0)
				{
					const int32 RecycledStitchIndex = StitchesToRemoveGlobal.Pop() - StitchOffset;
					if (Index3D[RecycledStitchIndex] != INDEX_NONE)
					{
						// Remove RecycledStitchIndex from this SeamStitchLookup
						SeamStitchLookup[Index3D[RecycledStitchIndex]].Remove(RecycledStitchIndex);
					}
					Index3D[RecycledStitchIndex] = NewStitches.Vertex3D;
					AllEnds[RecycledStitchIndex + StitchOffset] = StitchEnds;
					SeamStitchLookup[NewStitches.Vertex3D].Add(RecycledStitchIndex + StitchOffset);
				}
				else
				{
					Index3D[NewStitchIndex] = NewStitches.Vertex3D;
					AllEnds[NewStitchIndex + StitchOffset] = StitchEnds;
					SeamStitchLookup[NewStitches.Vertex3D].Add(NewStitchIndex + StitchOffset);
					++NewStitchIndex;
				}
			}
		}
		check(NewStitchIndex == GetNumSeamStitches());

		if (!StitchesToRemoveGlobal.IsEmpty())
		{
			GetClothCollection()->RemoveElements(
				ClothCollectionGroup::SeamStitches,
				StitchesToRemoveGlobal,
				GetClothCollection()->GetSeamStitchStart(),
				GetClothCollection()->GetSeamStitchEnd(),
				GetElementIndex());
		}
	}

	void FCollectionClothSeamFacade::SetNumSeamStitches(int32 NumStitches)
	{
		GetClothCollection()->SetNumElements(
			NumStitches,
			ClothCollectionGroup::SeamStitches,
			GetClothCollection()->GetSeamStitchStart(),
			GetClothCollection()->GetSeamStitchEnd(),
			GetElementIndex());
	}

	TArrayView<FIntVector2> FCollectionClothSeamFacade::GetSeamStitch2DEndIndices()
	{
		return GetClothCollection()->GetElements(
			GetClothCollection()->GetSeamStitch2DEndIndices(),
			GetClothCollection()->GetSeamStitchStart(),
			GetClothCollection()->GetSeamStitchEnd(),
			GetElementIndex());
	}

	TArrayView<int32> FCollectionClothSeamFacade::GetSeamStitch3DIndex()
	{
		return GetClothCollection()->GetElements(
			GetClothCollection()->GetSeamStitch3DIndex(),
			GetClothCollection()->GetSeamStitchStart(),
			GetClothCollection()->GetSeamStitchEnd(),
			GetElementIndex());
	}

	FCollectionClothSeamFacade::FCollectionClothSeamFacade(const TSharedRef<FClothCollection>& ClothCollection, int32 InSeamIndex)
		: FCollectionClothSeamConstFacade(ClothCollection, InSeamIndex)
	{
	}

	void FCollectionClothSeamFacade::SetDefaults()
	{
		const int32 ElementIndex = GetElementIndex();

		(*GetClothCollection()->GetSeamStitchStart())[ElementIndex] = INDEX_NONE;
		(*GetClothCollection()->GetSeamStitchEnd())[ElementIndex] = INDEX_NONE;
	}

	TSharedRef<class FClothCollection> FCollectionClothSeamFacade::GetClothCollection()
	{
		return StaticCastSharedRef<class FClothCollection>(ConstCastSharedRef<class FConstClothCollection>(ClothCollection));
	}
}  // End namespace UE::Chaos::ClothAsset
