// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/CollectionClothSelectionFacade.h"
#include "GeometryCollection/ManagedArrayCollection.h"

#define LOCTEXT_NAMESPACE "FCollectionClothSelectionFacade"

namespace UE::Chaos::ClothAsset
{

	namespace Private
	{
		static const FName SelectionGroup(TEXT("Selection"));
		static const FName SelectionSecondaryGroup(TEXT("SelectionSecondary"));
	}

	// --------------- FCollectionClothSelectionConstFacade -------------------------------

	FCollectionClothSelectionConstFacade::FCollectionClothSelectionConstFacade(const TSharedRef<const FManagedArrayCollection>& InManagedArrayCollection) :
		ManagedArrayCollection(ConstCastSharedRef<FManagedArrayCollection>(InManagedArrayCollection))
	{
	}

	bool FCollectionClothSelectionConstFacade::IsValid() const
	{
		return ManagedArrayCollection->HasGroup(Private::SelectionGroup) &&
			ManagedArrayCollection->NumElements(Private::SelectionGroup) == 1;
	}

	int32 FCollectionClothSelectionConstFacade::GetNumSelections() const
	{
		return ManagedArrayCollection->NumAttributes(Private::SelectionGroup);
	}

	TArray<FName> FCollectionClothSelectionConstFacade::GetNames() const
	{
		return ManagedArrayCollection->AttributeNames(Private::SelectionGroup);
	}

	bool FCollectionClothSelectionConstFacade::HasSelection(const FName& Name) const
	{
		return IsValid() && ManagedArrayCollection->HasAttribute(Name, Private::SelectionGroup);
	}

	FName FCollectionClothSelectionConstFacade::GetSelectionGroup(const FName& Name) const
	{
		check(IsValid());
		check(HasSelection(Name));
		return ManagedArrayCollection->GetDependency(Name, Private::SelectionGroup);
	}

	const TSet<int32>& FCollectionClothSelectionConstFacade::GetSelectionSet(const FName& Name) const
	{
		check(IsValid());
		const TManagedArray<TSet<int32>>* const Selection = ManagedArrayCollection->FindAttributeTyped<TSet<int32>>(Name, Private::SelectionGroup);
		check(Selection);
		return *Selection->GetData();
	}

	const TSet<int32>* FCollectionClothSelectionConstFacade::FindSelectionSet(const FName& Name) const
	{
		check(IsValid());
		const TManagedArray<TSet<int32>>* const Selection = ManagedArrayCollection->FindAttributeTyped<TSet<int32>>(Name, Private::SelectionGroup);
		return Selection ? Selection->GetData() : nullptr;
	}

	bool FCollectionClothSelectionConstFacade::HasSelectionSecondarySet(const FName& Name) const
	{
		if (!IsValid())
		{
			return false;
		}
		const bool bPrimaryAttributeExists = ManagedArrayCollection->HasAttribute(Name, Private::SelectionGroup);
		const bool bSecondaryAttributeExists = ManagedArrayCollection->HasAttribute(Name, Private::SelectionSecondaryGroup);
		checkf(bPrimaryAttributeExists || !bSecondaryAttributeExists, TEXT("FCollectionClothSelectionConstFacade: Secondary Selection found with no matching Primary attribute"));
		return bPrimaryAttributeExists && bSecondaryAttributeExists;
	}

	FName FCollectionClothSelectionConstFacade::GetSelectionSecondaryGroup(const FName& Name) const
	{
		check(IsValid());
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		check(HasSelectionSecondarySet(Name));	// Also checks the primary set exists
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		return ManagedArrayCollection->GetDependency(Name, Private::SelectionSecondaryGroup);
	}

	const TSet<int32>& FCollectionClothSelectionConstFacade::GetSelectionSecondarySet(const FName& Name) const
	{
		check(IsValid());
		check(ManagedArrayCollection->HasAttribute(Name, Private::SelectionGroup));

		const TManagedArray<TSet<int32>>* const Selection = ManagedArrayCollection->FindAttributeTyped<TSet<int32>>(Name, Private::SelectionSecondaryGroup);
		check(Selection);
		return *Selection->GetData();
	}

	const TSet<int32>* FCollectionClothSelectionConstFacade::FindSelectionSecondarySet(const FName& Name) const
	{
		check(IsValid());
		const bool bPrimaryAttributeExists = ManagedArrayCollection->HasAttribute(Name, Private::SelectionGroup);
		const TManagedArray<TSet<int32>>* const SecondarySelection = ManagedArrayCollection->FindAttributeTyped<TSet<int32>>(Name, Private::SelectionSecondaryGroup);

		checkf(bPrimaryAttributeExists || !SecondarySelection, TEXT("FCollectionClothSelectionConstFacade: Secondary Selection found with no matching Primary attribute"));

		return SecondarySelection ? SecondarySelection->GetData() : nullptr;
	}


	// --------------- FCollectionClothSelectionFacade -------------------------------

	FCollectionClothSelectionFacade::FCollectionClothSelectionFacade(const TSharedRef<FManagedArrayCollection>& ManagedArrayCollection) :
		FCollectionClothSelectionConstFacade(ManagedArrayCollection)
	{}

	// UE_DEPRECATED(5.6, "Use the constructor that takes a non const ManagedArrayCollection instead.")
	FCollectionClothSelectionFacade::FCollectionClothSelectionFacade(const TSharedRef<const FManagedArrayCollection>& ManagedArrayCollection) :
		FCollectionClothSelectionConstFacade(ManagedArrayCollection)
	{}

	void FCollectionClothSelectionFacade::DefineSchema()
	{
		auto InitSelectionGroup = [this](const FName& SelectionGroupName)
		{
			if (!ManagedArrayCollection->HasGroup(SelectionGroupName))
			{
				ManagedArrayCollection->AddGroup(SelectionGroupName);
			}
			const int32 NumElements = ManagedArrayCollection->NumElements(SelectionGroupName);
			if (NumElements > 1)
			{
				ManagedArrayCollection->RemoveElements(SelectionGroupName, NumElements - 1, 1);
			}
			else if (NumElements == 0)
			{
				ManagedArrayCollection->AddElements(1, SelectionGroupName);
			}
		};

		InitSelectionGroup(Private::SelectionGroup);
		InitSelectionGroup(Private::SelectionSecondaryGroup);
	}

	void FCollectionClothSelectionFacade::AppendWithOffsets(const FCollectionClothSelectionConstFacade& Other, bool bOverwriteExistingIfMismatched, const TMap<FName, int32>& GroupOffsets)
	{
		if (Other.IsValid())
		{
			const int32 NumInSelections = Other.GetNumSelections();
			const TArray<FName> InSelectionNames = Other.GetNames();
			for (int32 InSelectionIndex = 0; InSelectionIndex < NumInSelections; ++InSelectionIndex)
			{
				const FName& SelectionName = InSelectionNames[InSelectionIndex];
				const FName OtherGroupName = Other.GetSelectionGroup(SelectionName);
				const TSet<int32>& OtherSet = Other.GetSelectionSet(SelectionName);
				TSet<int32> OffsetSetIfNeeded;
				const TSet<int32>* OtherSetWithOffset = &OtherSet;
				if (const int32* const Offset = GroupOffsets.Find(OtherGroupName))
				{
					if (*Offset != 0)
					{
						OffsetSetIfNeeded.Reserve(OtherSet.Num());
						for (const int32 OrigIndex : OtherSet)
						{
							OffsetSetIfNeeded.Emplace(OrigIndex + *Offset);
						}
						OtherSetWithOffset = &OffsetSetIfNeeded;
					}
				}	
				
				check(OtherSetWithOffset);
				if (HasSelection(SelectionName))
				{
					if (GetSelectionGroup(SelectionName) == OtherGroupName)
					{
						TSet<int32>& UnionedSet = GetSelectionSet(SelectionName);

						UnionedSet.Append(*OtherSetWithOffset);
						continue;
					}
					if (!bOverwriteExistingIfMismatched)
					{
						continue;
					}
				}

				if (!IsValid())
				{
					DefineSchema();
				}

				FindOrAddSelectionSet(SelectionName, OtherGroupName) = *OtherSetWithOffset;
			}
		}
	}

	TSet<int32>& FCollectionClothSelectionFacade::GetSelectionSet(const FName& Name)
	{
		check(IsValid());
		TManagedArray<TSet<int32>>* const Selection = ManagedArrayCollection->FindAttributeTyped<TSet<int32>>(Name, Private::SelectionGroup);
		check(Selection);
		return *Selection->GetData();
	}

	TSet<int32>* FCollectionClothSelectionFacade::FindSelectionSet(const FName& Name)
	{
		check(IsValid());
		TManagedArray<TSet<int32>>* const Selection = ManagedArrayCollection->FindAttributeTyped<TSet<int32>>(Name, Private::SelectionGroup);
		return Selection ? Selection->GetData() : nullptr;
	}

	void FCollectionClothSelectionFacade::RemoveSelectionSet(const FName& Name)
	{
		check(IsValid());
		ManagedArrayCollection->RemoveAttribute(Name, Private::SelectionGroup);
	}


	TSet<int32>& FCollectionClothSelectionFacade::GetSelectionSecondarySet(const FName& Name)
	{
		check(IsValid());
		check(ManagedArrayCollection->FindAttributeTyped<TSet<int32>>(Name, Private::SelectionGroup) != nullptr);

		TManagedArray<TSet<int32>>* const Selection = ManagedArrayCollection->FindAttributeTyped<TSet<int32>>(Name, Private::SelectionSecondaryGroup);
		check(Selection);
		return *Selection->GetData();
	}

	TSet<int32>* FCollectionClothSelectionFacade::FindSelectionSecondarySet(const FName& Name)
	{
		check(IsValid());
		const bool bPrimaryAttributeExists = ManagedArrayCollection->HasAttribute(Name, Private::SelectionGroup);

		TManagedArray<TSet<int32>>* const SecondarySelection = ManagedArrayCollection->FindAttributeTyped<TSet<int32>>(Name, Private::SelectionSecondaryGroup);	
		
		checkf(bPrimaryAttributeExists || !SecondarySelection, TEXT("FCollectionClothSelectionFacade: Secondary Selection found with no Primary attribute"));

		return SecondarySelection ? SecondarySelection->GetData() : nullptr;
	}

	void FCollectionClothSelectionFacade::RemoveSelectionSecondarySet(const FName& Name)
	{
		check(IsValid());
		ManagedArrayCollection->RemoveAttribute(Name, Private::SelectionSecondaryGroup);
	}


	TSet<int32>& FCollectionClothSelectionFacade::FindOrAddSelectionSetInternal(const FName& Name, const FName& GroupName, const FName& SelectionGroupName)
	{
		check(IsValid());
		ensure(GroupName != NAME_None && Name != NAME_None);
		constexpr bool bAllowCircularDependency = false;

		TManagedArray<TSet<int32>>* Selection = ManagedArrayCollection->FindAttributeTyped<TSet<int32>>(Name, SelectionGroupName);
		if (Selection)
		{
			check(Selection->Num() == 1);  // This should always be the case if the facade is valid

			// Recycle the existing selection, with the new group if needed
			if (ManagedArrayCollection->GetDependency(Name, SelectionGroupName) != GroupName)
			{
				ManagedArrayCollection->SetDependency(Name, SelectionGroupName, GroupName, bAllowCircularDependency);
				(*Selection)[0].Reset();  // No point in keeping unrelated selection indices since the group has changed better clear everything
			}
		}
		else
		{
			// Create a new selection
			constexpr bool bSaved = true;
			const FManagedArrayCollection::FConstructionParameters GroupDependency(GroupName, bSaved, bAllowCircularDependency);
			Selection = &ManagedArrayCollection->AddAttribute<TSet<int32>>(Name, SelectionGroupName, GroupDependency);

			check(Selection->Num() == 1);  // This should always be the case if the facade is valid
		}

		return (*Selection)[0];
	}


	TSet<int32>& FCollectionClothSelectionFacade::FindOrAddSelectionSet(const FName& Name, const FName& GroupName)
	{
		return FindOrAddSelectionSetInternal(Name, GroupName, Private::SelectionGroup);
	}

	TSet<int32>& FCollectionClothSelectionFacade::FindOrAddSelectionSecondarySet(const FName& Name, const FName& GroupName)
	{
		checkf(HasSelection(Name), TEXT("FCollectionClothSelectionFacade: Can't add a Secondary Selection with no matching Primary attribute"));
		return FindOrAddSelectionSetInternal(Name, GroupName, Private::SelectionSecondaryGroup);
	}

}	// namespace  UE::Chaos::ClothAsset

#undef LOCTEXT_NAMESPACE
