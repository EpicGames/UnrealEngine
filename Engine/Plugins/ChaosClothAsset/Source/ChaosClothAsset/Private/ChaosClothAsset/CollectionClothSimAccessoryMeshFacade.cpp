// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/CollectionClothSimAccessoryMeshFacade.h"
#include "ChaosClothAsset/ClothCollection.h"
#include "ChaosClothAsset/ClothCollectionGroup.h"

namespace UE::Chaos::ClothAsset
{
	FName FCollectionClothSimAccessoryMeshConstFacade::GetSimAccessoryMeshName() const
	{
		return ClothCollection->GetSimAccessoryMeshName() && ClothCollection->GetNumElements(ClothCollectionGroup::SimAccessoryMeshes) > GetElementIndex() ?
			(*ClothCollection->GetSimAccessoryMeshName())[GetElementIndex()] : NAME_None;
	}

	FName FCollectionClothSimAccessoryMeshConstFacade::GetSimAccessoryMeshPosition3DAttribute() const
	{
		return ClothCollection->GetSimAccessoryMeshPosition3DAttribute() && ClothCollection->GetNumElements(ClothCollectionGroup::SimAccessoryMeshes) > GetElementIndex() ?
			(*ClothCollection->GetSimAccessoryMeshPosition3DAttribute())[GetElementIndex()] : NAME_None;
	}

	FName FCollectionClothSimAccessoryMeshConstFacade::GetSimAccessoryMeshNormalAttribute() const
	{
		return ClothCollection->GetSimAccessoryMeshNormalAttribute() && ClothCollection->GetNumElements(ClothCollectionGroup::SimAccessoryMeshes) > GetElementIndex() ?
			(*ClothCollection->GetSimAccessoryMeshNormalAttribute())[GetElementIndex()] : NAME_None;
	}

	FName FCollectionClothSimAccessoryMeshConstFacade::GetSimAccessoryMeshBoneIndicesAttribute() const
	{
		return ClothCollection->GetSimAccessoryMeshBoneIndicesAttribute() && ClothCollection->GetNumElements(ClothCollectionGroup::SimAccessoryMeshes) > GetElementIndex() ?
			(*ClothCollection->GetSimAccessoryMeshBoneIndicesAttribute())[GetElementIndex()] : NAME_None;
	}

	FName FCollectionClothSimAccessoryMeshConstFacade::GetSimAccessoryMeshBoneWeightsAttribute() const
	{
		return ClothCollection->GetSimAccessoryMeshBoneWeightsAttribute() && ClothCollection->GetNumElements(ClothCollectionGroup::SimAccessoryMeshes) > GetElementIndex() ?
			(*ClothCollection->GetSimAccessoryMeshBoneWeightsAttribute())[GetElementIndex()] : NAME_None;
	}

	TConstArrayView<FVector3f> FCollectionClothSimAccessoryMeshConstFacade::GetSimAccessoryMeshPosition3D() const
	{
		return ClothCollection->GetElements(ClothCollection->GetSimAccessoryMeshPosition3D(GetSimAccessoryMeshPosition3DAttribute()));
	}

	TConstArrayView<FVector3f> FCollectionClothSimAccessoryMeshConstFacade::GetSimAccessoryMeshNormal() const
	{
		return ClothCollection->GetElements(ClothCollection->GetSimAccessoryMeshNormal(GetSimAccessoryMeshNormalAttribute()));
	}

	TConstArrayView<TArray<int32>> FCollectionClothSimAccessoryMeshConstFacade::GetSimAccessoryMeshBoneIndices() const
	{
		return ClothCollection->GetElements(ClothCollection->GetSimAccessoryMeshBoneIndices(GetSimAccessoryMeshBoneIndicesAttribute()));
	}

	TConstArrayView<TArray<float>> FCollectionClothSimAccessoryMeshConstFacade::GetSimAccessoryMeshBoneWeights() const
	{
		return ClothCollection->GetElements(ClothCollection->GetSimAccessoryMeshBoneWeights(GetSimAccessoryMeshBoneWeightsAttribute()));
	}

	FCollectionClothSimAccessoryMeshConstFacade::FCollectionClothSimAccessoryMeshConstFacade(const TSharedRef<const FConstClothCollection>& ClothCollection, int32 InMeshIndex)
		: ClothCollection(ClothCollection)
		, MeshIndex(InMeshIndex)
	{
		check(ClothCollection->IsValid(EClothCollectionExtendedSchemas::CookedOnly) || ClothCollection->IsValid(EClothCollectionExtendedSchemas::SimAccessoryMeshes));
		check(MeshIndex >= 0 && MeshIndex < ClothCollection->GetNumElements(ClothCollectionGroup::SimAccessoryMeshes));
	}

	void FCollectionClothSimAccessoryMeshFacade::Reset()
	{
		SetSimAccessoryMeshName(NAME_None);
		ClearDynamicAttributes();
	}

	void FCollectionClothSimAccessoryMeshFacade::Initialize(const FCollectionClothSimAccessoryMeshConstFacade& Other)
	{
		GenerateDynamicAttributesIfNecessary();

		//~ Sim Accessory Meshes Group
		SetSimAccessoryMeshName(Other.GetSimAccessoryMeshName());

		//~ Sim Vertices 3D Group
		FClothCollection::CopyArrayViewData(GetSimAccessoryMeshPosition3D(), Other.GetSimAccessoryMeshPosition3D());
		FClothCollection::CopyArrayViewData(GetSimAccessoryMeshNormal(), Other.GetSimAccessoryMeshNormal());
		FClothCollection::CopyArrayViewData(GetSimAccessoryMeshBoneIndices(), Other.GetSimAccessoryMeshBoneIndices());
		FClothCollection::CopyArrayViewData(GetSimAccessoryMeshBoneWeights(), Other.GetSimAccessoryMeshBoneWeights());
	}

	void FCollectionClothSimAccessoryMeshFacade::Initialize(const FName& Name, const TConstArrayView<FVector3f>& Positions, const TConstArrayView<FVector3f>& Normals, const TConstArrayView<TArray<int32>>& BoneIndices, const TConstArrayView<TArray<float>>& BoneWeights)
	{
		const int32 NumExpectedVertices = ClothCollection->GetNumElements(ClothCollectionGroup::SimVertices3D);
		check(Positions.Num() == NumExpectedVertices);
		check(Normals.Num() == NumExpectedVertices);
		check(BoneIndices.Num() == NumExpectedVertices);
		check(BoneWeights.Num() == NumExpectedVertices);

		GenerateDynamicAttributesIfNecessary();

		//~ Sim Accessory Meshes Group
		SetSimAccessoryMeshName(Name);

		//~ Sim Vertices 3D Group
		FClothCollection::CopyArrayViewData(GetSimAccessoryMeshPosition3D(), Positions);
		FClothCollection::CopyArrayViewData(GetSimAccessoryMeshNormal(), Normals);
		FClothCollection::CopyArrayViewData(GetSimAccessoryMeshBoneIndices(), BoneIndices);
		FClothCollection::CopyArrayViewData(GetSimAccessoryMeshBoneWeights(), BoneWeights);
	}

	void FCollectionClothSimAccessoryMeshFacade::SetSimAccessoryMeshName(const FName& MeshName)
	{
		(*GetClothCollection()->GetSimAccessoryMeshName())[GetElementIndex()] = MeshName;
	}

	TArrayView<FVector3f> FCollectionClothSimAccessoryMeshFacade::GetSimAccessoryMeshPosition3D()
	{
		return GetClothCollection()->GetElements(
			GetClothCollection()->GetSimAccessoryMeshPosition3D(GetSimAccessoryMeshPosition3DAttribute()));
	}

	TArrayView<FVector3f> FCollectionClothSimAccessoryMeshFacade::GetSimAccessoryMeshNormal()
	{
		return GetClothCollection()->GetElements(
			GetClothCollection()->GetSimAccessoryMeshNormal(GetSimAccessoryMeshNormalAttribute()));
	}

	TArrayView<TArray<int32>> FCollectionClothSimAccessoryMeshFacade::GetSimAccessoryMeshBoneIndices()
	{
		return GetClothCollection()->GetElements(
			GetClothCollection()->GetSimAccessoryMeshBoneIndices(GetSimAccessoryMeshBoneIndicesAttribute()));
	}

	TArrayView<TArray<float>> FCollectionClothSimAccessoryMeshFacade::GetSimAccessoryMeshBoneWeights()
	{
		return GetClothCollection()->GetElements(
			GetClothCollection()->GetSimAccessoryMeshBoneWeights(GetSimAccessoryMeshBoneWeightsAttribute()));
	}

	FCollectionClothSimAccessoryMeshFacade::FCollectionClothSimAccessoryMeshFacade(const TSharedRef<FClothCollection>& ClothCollection, int32 InMeshIndex)
		: FCollectionClothSimAccessoryMeshConstFacade(ClothCollection, InMeshIndex)
	{
	}

	void FCollectionClothSimAccessoryMeshFacade::ClearDynamicAttributes()
	{
		//~ Sim Vertices 3D Group
		GetClothCollection()->RemoveSimAccessoryMeshPosition3D(GetSimAccessoryMeshPosition3DAttribute());
		GetClothCollection()->RemoveSimAccessoryMeshNormal(GetSimAccessoryMeshNormalAttribute());
		GetClothCollection()->RemoveSimAccessoryMeshBoneIndices(GetSimAccessoryMeshBoneIndicesAttribute());
		GetClothCollection()->RemoveSimAccessoryMeshBoneWeights(GetSimAccessoryMeshBoneWeightsAttribute());
		(*GetClothCollection()->GetSimAccessoryMeshPosition3DAttribute())[GetElementIndex()] = NAME_None;
		(*GetClothCollection()->GetSimAccessoryMeshNormalAttribute())[GetElementIndex()] = NAME_None;
		(*GetClothCollection()->GetSimAccessoryMeshBoneIndicesAttribute())[GetElementIndex()] = NAME_None;
		(*GetClothCollection()->GetSimAccessoryMeshBoneWeightsAttribute())[GetElementIndex()] = NAME_None;
	}

	void FCollectionClothSimAccessoryMeshFacade::GenerateDynamicAttributesIfNecessary()
	{
		//~ Sim Vertices 3D Group
		if (ClothCollection->GetSimAccessoryMeshPosition3D(GetSimAccessoryMeshPosition3DAttribute()) == nullptr)
		{
			(*GetClothCollection()->GetSimAccessoryMeshPosition3DAttribute())[GetElementIndex()] = GetClothCollection()->AddSimAccessoryMeshPosition3D();
		}
		if (ClothCollection->GetSimAccessoryMeshNormal(GetSimAccessoryMeshNormalAttribute()) == nullptr)
		{
			(*GetClothCollection()->GetSimAccessoryMeshNormalAttribute())[GetElementIndex()] = GetClothCollection()->AddSimAccessoryMeshNormal();
		}
		if (ClothCollection->GetSimAccessoryMeshBoneIndices(GetSimAccessoryMeshBoneIndicesAttribute()) == nullptr)
		{
			(*GetClothCollection()->GetSimAccessoryMeshBoneIndicesAttribute())[GetElementIndex()] = GetClothCollection()->AddSimAccessoryMeshBoneIndices();
		}
		if (ClothCollection->GetSimAccessoryMeshBoneWeights(GetSimAccessoryMeshBoneWeightsAttribute()) == nullptr)
		{
			(*GetClothCollection()->GetSimAccessoryMeshBoneWeightsAttribute())[GetElementIndex()] = GetClothCollection()->AddSimAccessoryMeshBoneWeights();
		}
	}

	TSharedRef<class FClothCollection> FCollectionClothSimAccessoryMeshFacade::GetClothCollection()
	{
		return StaticCastSharedRef<class FClothCollection>(ConstCastSharedRef<class FConstClothCollection>(ClothCollection));
	}
}  // End namespace UE::Chaos::ClothAsset
