// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/CollectionClothSimMorphTargetFacade.h"
#include "ChaosClothAsset/ClothCollection.h"
#include "ChaosClothAsset/ClothCollectionGroup.h"

namespace UE::Chaos::ClothAsset
{
	const FString& FCollectionClothSimMorphTargetConstFacade::GetSimMorphTargetName() const
	{
		static const FString EmptyString;
		return ClothCollection->GetSimMorphTargetName() && ClothCollection->GetNumElements(ClothCollectionGroup::SimMorphTargets) > GetElementIndex() ?
			(*ClothCollection->GetSimMorphTargetName())[GetElementIndex()] : EmptyString;
	}

	int32 FCollectionClothSimMorphTargetConstFacade::GetNumSimMorphTargetVertices() const
	{
		return ClothCollection->GetNumElements(
			ClothCollection->GetSimMorphTargetVerticesStart(),
			ClothCollection->GetSimMorphTargetVerticesEnd(),
			GetElementIndex());
	}

	int32 FCollectionClothSimMorphTargetConstFacade::GetSimMorphTargetVerticesOffset() const
	{
		return ClothCollection->GetElementsOffset(
			ClothCollection->GetSimMorphTargetVerticesStart(),
			GetBaseElementIndex(),
			GetElementIndex());
	}

	TConstArrayView<FVector3f> FCollectionClothSimMorphTargetConstFacade::GetSimMorphTargetPositionDelta() const
	{
		return ClothCollection->GetElements(
			ClothCollection->GetSimMorphTargetPositionDelta(),
			ClothCollection->GetSimMorphTargetVerticesStart(),
			ClothCollection->GetSimMorphTargetVerticesEnd(),
			GetElementIndex());
	}

	TConstArrayView<FVector3f> FCollectionClothSimMorphTargetConstFacade::GetSimMorphTargetTangentZDelta() const
	{
		return ClothCollection->GetElements(
			ClothCollection->GetSimMorphTargetTangentZDelta(),
			ClothCollection->GetSimMorphTargetVerticesStart(),
			ClothCollection->GetSimMorphTargetVerticesEnd(),
			GetElementIndex());
	}

	TConstArrayView<int32> FCollectionClothSimMorphTargetConstFacade::GetSimMorphTargetSimVertex3DIndex() const
	{
		return ClothCollection->GetElements(
			ClothCollection->GetSimMorphTargetSimVertex3DIndex(),
			ClothCollection->GetSimMorphTargetVerticesStart(),
			ClothCollection->GetSimMorphTargetVerticesEnd(),
			GetElementIndex());
	}

	bool FCollectionClothSimMorphTargetConstFacade::IsEmpty() const
	{
		return GetNumSimMorphTargetVertices() == 0;
	}

	FCollectionClothSimMorphTargetConstFacade::FCollectionClothSimMorphTargetConstFacade(const TSharedRef<const FConstClothCollection>& ClothCollection, int32 InMorphTargetIndex)
		: ClothCollection(ClothCollection)
		, MorphTargetIndex(InMorphTargetIndex)
	{
		check(ClothCollection->IsValid(EClothCollectionExtendedSchemas::CookedOnly) || ClothCollection->IsValid(EClothCollectionExtendedSchemas::SimMorphTargets));
		check(MorphTargetIndex >= 0 && MorphTargetIndex < ClothCollection->GetNumElements(ClothCollectionGroup::SimMorphTargets));
	}

	void FCollectionClothSimMorphTargetFacade::Reset()
	{
		SetNumSimMorphTargetVertices(0);
		SetDefaults();
	}

	void FCollectionClothSimMorphTargetFacade::Initialize(const FCollectionClothSimMorphTargetConstFacade& Other, int32 SimVertex3DOffset)
	{
		Reset();

		//~ Sim Morph Targets Group
		SetSimMorphTargetName(Other.GetSimMorphTargetName());

		//~ Sim Morph Target Vertices Group
		SetNumSimMorphTargetVertices(Other.GetNumSimMorphTargetVertices());
		FClothCollection::CopyArrayViewData(GetSimMorphTargetPositionDelta(), Other.GetSimMorphTargetPositionDelta());
		FClothCollection::CopyArrayViewData(GetSimMorphTargetTangentZDelta(), Other.GetSimMorphTargetTangentZDelta());
		FClothCollection::CopyArrayViewDataAndApplyOffset(GetSimMorphTargetSimVertex3DIndex(), Other.GetSimMorphTargetSimVertex3DIndex(), SimVertex3DOffset);
	}

	void FCollectionClothSimMorphTargetFacade::Initialize(const FString& Name, const TConstArrayView<FVector3f>& PositionDeltas, const TConstArrayView<FVector3f>& TangentZDeltas, const TConstArrayView<int32>& SimVertex3DIndices)
	{
		check(PositionDeltas.Num() == TangentZDeltas.Num());
		check(PositionDeltas.Num() == SimVertex3DIndices.Num());
		Reset();

		//~ Sim Morph Targets Group
		SetSimMorphTargetName(Name);

		//~ Sim Morph Target Vertices Group
		SetNumSimMorphTargetVertices(PositionDeltas.Num());
		FClothCollection::CopyArrayViewData(GetSimMorphTargetPositionDelta(), PositionDeltas);
		FClothCollection::CopyArrayViewData(GetSimMorphTargetTangentZDelta(), TangentZDeltas);
		FClothCollection::CopyArrayViewData(GetSimMorphTargetSimVertex3DIndex(), SimVertex3DIndices);
	}

	void FCollectionClothSimMorphTargetFacade::SetSimMorphTargetName(const FString& MorphTargetName)
	{
		(*GetClothCollection()->GetSimMorphTargetName())[GetElementIndex()] = MorphTargetName;
	}

	void FCollectionClothSimMorphTargetFacade::SetNumSimMorphTargetVertices(int32 NumMorphTargetVertices)
	{
		GetClothCollection()->SetNumElements(
			NumMorphTargetVertices,
			ClothCollectionGroup::SimMorphTargetVertices,
			GetClothCollection()->GetSimMorphTargetVerticesStart(),
			GetClothCollection()->GetSimMorphTargetVerticesEnd(),
			GetElementIndex());
	}

	void FCollectionClothSimMorphTargetFacade::RemoveSimMorphTargetVertices(const TArray<int32>& SortedDeletionList)
	{
		TArray<int32> GlobalIndexSortedDeletionList;
		const int32 Offset = GetSimMorphTargetVerticesOffset();
		GlobalIndexSortedDeletionList.SetNumUninitialized(SortedDeletionList.Num());
		for (int32 Idx = 0; Idx < SortedDeletionList.Num(); ++Idx)
		{
			GlobalIndexSortedDeletionList[Idx] = SortedDeletionList[Idx] + Offset;
		}

		GetClothCollection()->RemoveElements(
			ClothCollectionGroup::SimMorphTargetVertices,
			GlobalIndexSortedDeletionList,
			GetClothCollection()->GetSimMorphTargetVerticesStart(),
			GetClothCollection()->GetSimMorphTargetVerticesEnd(),
			GetElementIndex());
	}

	TArrayView<FVector3f> FCollectionClothSimMorphTargetFacade::GetSimMorphTargetPositionDelta()
	{
		return GetClothCollection()->GetElements(
			GetClothCollection()->GetSimMorphTargetPositionDelta(),
			GetClothCollection()->GetSimMorphTargetVerticesStart(),
			GetClothCollection()->GetSimMorphTargetVerticesEnd(),
			GetElementIndex());
	}

	TArrayView<FVector3f> FCollectionClothSimMorphTargetFacade::GetSimMorphTargetTangentZDelta()
	{
		return GetClothCollection()->GetElements(
			GetClothCollection()->GetSimMorphTargetTangentZDelta(),
			GetClothCollection()->GetSimMorphTargetVerticesStart(),
			GetClothCollection()->GetSimMorphTargetVerticesEnd(),
			GetElementIndex());
	}

	TArrayView<int32> FCollectionClothSimMorphTargetFacade::GetSimMorphTargetSimVertex3DIndex()
	{
		return GetClothCollection()->GetElements(
			GetClothCollection()->GetSimMorphTargetSimVertex3DIndex(),
			GetClothCollection()->GetSimMorphTargetVerticesStart(),
			GetClothCollection()->GetSimMorphTargetVerticesEnd(),
			GetElementIndex());
	}

	void FCollectionClothSimMorphTargetFacade::Compact()
	{
		TArray<int32> SortedDeletionList;
		const TConstArrayView<int32> SimVertexIndices = GetSimMorphTargetSimVertex3DIndex();
		for(int32 Index = 0; Index <SimVertexIndices.Num(); ++Index)
		{
			if (SimVertexIndices[Index] == INDEX_NONE)
			{
				SortedDeletionList.Add(Index);
			}
		}
		RemoveSimMorphTargetVertices(SortedDeletionList);
	}

	FCollectionClothSimMorphTargetFacade::FCollectionClothSimMorphTargetFacade(const TSharedRef<FClothCollection>& ClothCollection, int32 InMorphTargetIndex)
		: FCollectionClothSimMorphTargetConstFacade(ClothCollection, InMorphTargetIndex)
	{
	}

	void FCollectionClothSimMorphTargetFacade::SetDefaults()
	{
		const int32 ElementIndex = GetElementIndex();
		SetSimMorphTargetName(FString());
		(*GetClothCollection()->GetSimMorphTargetVerticesStart())[ElementIndex] = INDEX_NONE;
		(*GetClothCollection()->GetSimMorphTargetVerticesEnd())[ElementIndex] = INDEX_NONE;
	}

	TSharedRef<class FClothCollection> FCollectionClothSimMorphTargetFacade::GetClothCollection()
	{
		return StaticCastSharedRef<class FClothCollection>(ConstCastSharedRef<class FConstClothCollection>(ClothCollection));
	}
}  // End namespace UE::Chaos::ClothAsset
