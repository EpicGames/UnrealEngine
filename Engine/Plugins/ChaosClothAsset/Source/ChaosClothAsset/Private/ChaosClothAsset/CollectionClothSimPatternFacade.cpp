// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/CollectionClothSimPatternFacade.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "ChaosClothAsset/ClothCollection.h"
#include "ChaosClothAsset/ClothCollectionGroup.h"
#include "ChaosClothAsset/ClothGeometryTools.h"

namespace UE::Chaos::ClothAsset
{
	namespace Private
	{
		template<typename IndexType> static int32 GetNumFaces(const TArray<IndexType>& Indices) { check(Indices.Num() % 3 == 0); return Indices.Num() / 3; }
		template<> int32 GetNumFaces<FIntVector3>(const TArray<FIntVector3>& Indices) { return Indices.Num(); }

		template<int32 Component, typename IndexType> static int32 GetIndex(const TArray<IndexType>& Indices, int32 Index) { return (int32)Indices[Index * 3 + Component]; }
		template<> int32 GetIndex<0, FIntVector3>(const TArray<FIntVector3>& Indices, int32 Index) { return Indices[Index][0]; }
		template<> int32 GetIndex<1, FIntVector3>(const TArray<FIntVector3>& Indices, int32 Index) { return Indices[Index][1]; }
		template<> int32 GetIndex<2, FIntVector3>(const TArray<FIntVector3>& Indices, int32 Index) { return Indices[Index][2]; }
	}

	int32 FCollectionClothSimPatternConstFacade::GetNumSimVertices2D() const
	{
		return ClothCollection->GetNumElements(
			ClothCollection->GetSimVertices2DStart(),
			ClothCollection->GetSimVertices2DEnd(),
			GetElementIndex());
	}

	int32 FCollectionClothSimPatternConstFacade::GetSimVertices2DOffset() const
	{
		return ClothCollection->GetElementsOffset(
			ClothCollection->GetSimVertices2DStart(),
			GetBaseElementIndex(),
			GetElementIndex());
	}

	TConstArrayView<FVector2f> FCollectionClothSimPatternConstFacade::GetSimPosition2D() const
	{
		return ClothCollection->GetElements(
			ClothCollection->GetSimPosition2D(),
			ClothCollection->GetSimVertices2DStart(),
			ClothCollection->GetSimVertices2DEnd(),
			GetElementIndex());
	}

	TConstArrayView<int32> FCollectionClothSimPatternConstFacade::GetSimVertex3DLookup() const
	{
		return ClothCollection->GetElements(
			ClothCollection->GetSimVertex3DLookup(),
			ClothCollection->GetSimVertices2DStart(),
			ClothCollection->GetSimVertices2DEnd(),
			GetElementIndex());
	}

	TConstArrayView<int32> FCollectionClothSimPatternConstFacade::GetSimImportVertexID() const
	{
		return ClothCollection->GetElements(
			ClothCollection->GetSimImportVertexID(),
			ClothCollection->GetSimVertices2DStart(),
			ClothCollection->GetSimVertices2DEnd(),
			GetElementIndex());
	}

	int32 FCollectionClothSimPatternConstFacade::GetNumSimFaces() const
	{
		return ClothCollection->GetNumElements(
			ClothCollection->GetSimFacesStart(),
			ClothCollection->GetSimFacesEnd(),
			GetElementIndex());
	}
	
	int32 FCollectionClothSimPatternConstFacade::GetFabricIndex() const
	{
		return ClothCollection->GetElements(ClothCollection->GetSimPatternFabric())[GetElementIndex()];
	}

	int32 FCollectionClothSimPatternConstFacade::GetSimFacesOffset() const
	{
		return ClothCollection->GetElementsOffset(
			ClothCollection->GetSimFacesStart(),
			GetBaseElementIndex(),
			GetElementIndex());
	}

	TConstArrayView<FIntVector3> FCollectionClothSimPatternConstFacade::GetSimIndices2D() const
	{
		return ClothCollection->GetElements(
			ClothCollection->GetSimIndices2D(),
			ClothCollection->GetSimFacesStart(),
			ClothCollection->GetSimFacesEnd(),
			GetElementIndex());
	}

	TConstArrayView<FIntVector3> FCollectionClothSimPatternConstFacade::GetSimIndices3D() const
	{
		return ClothCollection->GetElements(
			ClothCollection->GetSimIndices3D(),
			ClothCollection->GetSimFacesStart(),
			ClothCollection->GetSimFacesEnd(),
			GetElementIndex());
	}

	bool FCollectionClothSimPatternConstFacade::IsEmpty() const
	{
		return GetNumSimFaces() == 0 && GetNumSimVertices2D() == 0;
	}

	FCollectionClothSimPatternConstFacade::FCollectionClothSimPatternConstFacade(const TSharedRef<const FConstClothCollection>& ClothCollection, int32 InPatternIndex)
		: ClothCollection(ClothCollection)
		, PatternIndex(InPatternIndex)
	{
		check(ClothCollection->IsValid());
		check(PatternIndex >= 0 && PatternIndex < ClothCollection->GetNumElements(ClothCollectionGroup::SimPatterns));
	}

	void FCollectionClothSimPatternFacade::Reset()
	{
		SetNumSimFaces(0);
		RemoveAllSimVertices2D();
		SetDefaults();
	}

	template<typename IndexType UE_REQUIRES_DEFINITION(TIsIndexType<IndexType>::Value)>
	void FCollectionClothSimPatternFacade::Initialize(const TArray<FVector2f>& Positions2D, const TArray<FVector3f>& Positions3D, const TArray<IndexType>& Indices, const int32 FabricIndex, const TArray<FVector3f>& Normals)
	{
		Reset();

		SetFabricIndex(FabricIndex);

		const int32 NumSimVertices = Positions2D.Num();
		check(NumSimVertices == Positions3D.Num());

		FCollectionClothFacade Cloth(GetClothCollection());
		const FIntVector2 StartVertexIndex = AppendSimVertices(Positions2D.Num());
		check(StartVertexIndex[0] == 0); // We should have an empty pattern

		const TArrayView<FVector2f> SimPosition2D = GetSimPosition2D();
		const TArrayView<FVector3f> SimPosition3D = Cloth.GetSimPosition3D();
		const TArrayView<FVector3f> SimNormal = Cloth.GetSimNormal();

		for (int32 SimVertexIndex = 0; SimVertexIndex < NumSimVertices; ++SimVertexIndex)
		{
			SimPosition2D[SimVertexIndex] = Positions2D[SimVertexIndex];
			SimPosition3D[SimVertexIndex + StartVertexIndex[1]] = Positions3D[SimVertexIndex];
			SimNormal[SimVertexIndex + StartVertexIndex[1]] = Normals.IsValidIndex(SimVertexIndex) ? Normals[SimVertexIndex] : FVector3f::ZeroVector;
		}

		const int32 NumSimFaces = Private::GetNumFaces(Indices);

		SetNumSimFaces(NumSimFaces);
		const TArrayView<FIntVector3> SimIndices2D = GetSimIndices2D();
		const TArrayView<FIntVector3> SimIndices3D = GetSimIndices3D();
		const int32 BaseVertex2D = GetSimVertices2DOffset();
		for (int32 SimFaceIndex = 0; SimFaceIndex < NumSimFaces; ++SimFaceIndex)
		{
			// Indices from the start of the pattern
			const int32 PatternVertexIndex0 = Private::GetIndex<0>(Indices, SimFaceIndex);
			const int32 PatternVertexIndex1 = Private::GetIndex<1>(Indices, SimFaceIndex);
			const int32 PatternVertexIndex2 = Private::GetIndex<2>(Indices, SimFaceIndex);

			// 2D Indices from the start of the cloth
			const int32 Vertex2DIndex0 = PatternVertexIndex0 + BaseVertex2D;
			const int32 Vertex2DIndex1 = PatternVertexIndex1 + BaseVertex2D;
			const int32 Vertex2DIndex2 = PatternVertexIndex2 + BaseVertex2D;

			// 3D Indices from the start of the cloth
			const int32 Vertex3DIndex0 = PatternVertexIndex0 + StartVertexIndex[1];
			const int32 Vertex3DIndex1 = PatternVertexIndex1 + StartVertexIndex[1];
			const int32 Vertex3DIndex2 = PatternVertexIndex2 + StartVertexIndex[1];

			// Set indices in cloth index space
			SimIndices2D[SimFaceIndex] = FIntVector3(Vertex2DIndex0, Vertex2DIndex1, Vertex2DIndex2);
			SimIndices3D[SimFaceIndex] = FIntVector3(Vertex3DIndex0, Vertex3DIndex1, Vertex3DIndex2);

			// Calculate face normal contribution
			const FVector3f& Pos0 = SimPosition3D[Vertex3DIndex0];
			const FVector3f& Pos1 = SimPosition3D[Vertex3DIndex1];
			const FVector3f& Pos2 = SimPosition3D[Vertex3DIndex2];
			const FVector3f Normal = (Pos2 - Pos0).Cross(Pos1 - Pos0).GetSafeNormal();
			if (!Normals.IsValidIndex(PatternVertexIndex0))
			{
				SimNormal[Vertex3DIndex0] += Normal;
			}
			if (!Normals.IsValidIndex(PatternVertexIndex1))
			{
				SimNormal[Vertex3DIndex1] += Normal;
			}
			if (!Normals.IsValidIndex(PatternVertexIndex2))
			{
				SimNormal[Vertex3DIndex2] += Normal;
			}
		}

		// Normalize normals
		for (int32 SimVertexIndex = 0; SimVertexIndex < NumSimVertices; ++SimVertexIndex)
		{
			SimNormal[SimVertexIndex + StartVertexIndex[1]] = SimNormal[SimVertexIndex + StartVertexIndex[1]].GetSafeNormal(UE_SMALL_NUMBER, FVector3f::XAxisVector);
		}
	}
	template CHAOSCLOTHASSET_API void FCollectionClothSimPatternFacade::Initialize(const TArray<FVector2f>& Positions2D, const TArray<FVector3f>& Positions3D, const TArray<int32>& Indices, const int32 FabricIndex, const TArray<FVector3f>& Normals);
	template CHAOSCLOTHASSET_API void FCollectionClothSimPatternFacade::Initialize(const TArray<FVector2f>& Positions2D, const TArray<FVector3f>& Positions3D, const TArray<uint32>& Indices, const int32 FabricIndex, const TArray<FVector3f>& Normals);
	template CHAOSCLOTHASSET_API void FCollectionClothSimPatternFacade::Initialize(const TArray<FVector2f>& Positions2D, const TArray<FVector3f>& Positions3D, const TArray<FIntVector3>& Indices, const int32 FabricIndex, const TArray<FVector3f>& Normals);

	void FCollectionClothSimPatternFacade::Initialize(const FCollectionClothSimPatternConstFacade& Other, const int32 SimVertex3DOffset, const int32 FabricsOffset)
	{
		Reset();

		SetFabricIndex(FabricsOffset+Other.GetFabricIndex());

		// Sim Vertices 2D Group
		SetNumSimVertices2D(Other.GetNumSimVertices2D());
		FClothCollection::CopyArrayViewData(GetSimPosition2D(), Other.GetSimPosition2D());
		FClothCollection::CopyArrayViewDataAndApplyOffset(GetSimVertex3DLookup(), Other.GetSimVertex3DLookup(), SimVertex3DOffset);

		// Sim Faces Group
		SetNumSimFaces(Other.GetNumSimFaces());
		const int32 SimVertex2DOffset = GetSimVertices2DOffset() - Other.GetSimVertices2DOffset();
		FClothCollection::CopyArrayViewDataAndApplyOffset(GetSimIndices2D(), Other.GetSimIndices2D(), FIntVector(SimVertex2DOffset));
		FClothCollection::CopyArrayViewDataAndApplyOffset(GetSimIndices3D(), Other.GetSimIndices3D(), FIntVector(SimVertex3DOffset));
		if (Other.ClothCollection->IsValid(EClothCollectionExtendedSchemas::Import))
		{
			FClothCollection::CopyArrayViewData(GetSimImportVertexID(), Other.GetSimImportVertexID());
		}
		else if (ClothCollection->IsValid(EClothCollectionExtendedSchemas::Import))
		{
			// Initialize to INDEX_NONE
			TArrayView<int32> SimImportVertexID = GetSimImportVertexID();
			for (int32 Index = 0; Index < SimImportVertexID.Num(); ++Index)
			{
				SimImportVertexID[Index] = INDEX_NONE;
			}
		}
	}

	TArrayView<FVector2f> FCollectionClothSimPatternFacade::GetSimPosition2D()
	{
		return GetClothCollection()->GetElements(
			GetClothCollection()->GetSimPosition2D(),
			GetClothCollection()->GetSimVertices2DStart(),
			GetClothCollection()->GetSimVertices2DEnd(),
			GetElementIndex());
	}

	TArrayView<int32> FCollectionClothSimPatternFacade::GetSimImportVertexID()
	{
		if (!GetClothCollection()->IsValid(EClothCollectionExtendedSchemas::Import))
		{
			GetClothCollection()->DefineSchema(EClothCollectionExtendedSchemas::Import);
		}

		return GetClothCollection()->GetElements(
			GetClothCollection()->GetSimImportVertexID(),
			GetClothCollection()->GetSimVertices2DStart(),
			GetClothCollection()->GetSimVertices2DEnd(),
			GetElementIndex());
	}

	FIntVector2 FCollectionClothSimPatternFacade::AppendSimVertices(int32 NumSimVertices)
	{
		const FIntVector2 OrigNumVertices(GetNumSimVertices2D(), ClothCollection->GetNumElements(ClothCollectionGroup::SimVertices3D));

		// Resize arrays
		GetClothCollection()->SetNumElements(OrigNumVertices[1] + NumSimVertices, ClothCollectionGroup::SimVertices3D);
		SetNumSimVertices2D(OrigNumVertices[0] + NumSimVertices);

		// Set lookups to each other
		const int32 BaseVertex2DIndex = GetSimVertices2DOffset();
		TArrayView<int32> SimVertex3DLookup = GetSimVertex3DLookup();
		TArrayView<TArray<int32>> SimVertex2DLookup = GetClothCollection()->GetElements(
			GetClothCollection()->GetSimVertex2DLookup());

		const bool bSetImportVertexID = ClothCollection->IsValid(EClothCollectionExtendedSchemas::Import);
		TArrayView<int32> SimImportVertexID = bSetImportVertexID ? GetSimImportVertexID() : TArrayView<int32>();
		for (int32 NewVertexIndex = 0; NewVertexIndex < NumSimVertices; ++NewVertexIndex)
		{
			const int32 LocalVertex2D = OrigNumVertices[0] + NewVertexIndex;
			const int32 GlobalVertex2D = LocalVertex2D + BaseVertex2DIndex;
			const int32 Vertex3D = OrigNumVertices[1] + NewVertexIndex;
			SimVertex3DLookup[LocalVertex2D] = Vertex3D;
			SimVertex2DLookup[Vertex3D].Init(GlobalVertex2D, 1);
			if (bSetImportVertexID)
			{
				SimImportVertexID[LocalVertex2D] = INDEX_NONE;
			}
		}

		return OrigNumVertices;
	}

	void FCollectionClothSimPatternFacade::RemoveSimVertices2D(int32 InNumSimVertices)
	{
		const int32 NumSimVertices = GetNumSimVertices2D();
		check(InNumSimVertices >= NumSimVertices);
		SetNumSimVertices2D(NumSimVertices - InNumSimVertices);
	}

	void FCollectionClothSimPatternFacade::RemoveSimVertices2D(const TArray<int32>& SortedDeletionList)
	{
		TArray<int32> GlobalIndexSortedDeletionList;
		const int32 Offset = GetSimVertices2DOffset();
		GlobalIndexSortedDeletionList.SetNumUninitialized(SortedDeletionList.Num());
		for (int32 Idx = 0; Idx < SortedDeletionList.Num(); ++Idx)
		{
			GlobalIndexSortedDeletionList[Idx] = SortedDeletionList[Idx] + Offset;
		}

		GetClothCollection()->RemoveElements(
			ClothCollectionGroup::SimVertices2D,
			GlobalIndexSortedDeletionList,
			GetClothCollection()->GetSimVertices2DStart(),
			GetClothCollection()->GetSimVertices2DEnd(),
			GetElementIndex());
	}

	void FCollectionClothSimPatternFacade::SetNumSimFaces(int32 NumSimFaces)
	{
		GetClothCollection()->SetNumElements(
			NumSimFaces,
			ClothCollectionGroup::SimFaces,
			GetClothCollection()->GetSimFacesStart(),
			GetClothCollection()->GetSimFacesEnd(),
			GetElementIndex());
	}

	void FCollectionClothSimPatternFacade::SetFabricIndex(const int32 FabricIndex)
	{
		GetClothCollection()->GetElements(GetClothCollection()->GetSimPatternFabric())[GetElementIndex()] = FabricIndex;
	}

	TArrayView<FIntVector3> FCollectionClothSimPatternFacade::GetSimIndices2D()
	{
		return GetClothCollection()->GetElements(
			GetClothCollection()->GetSimIndices2D(),
			GetClothCollection()->GetSimFacesStart(),
			GetClothCollection()->GetSimFacesEnd(),
			GetElementIndex());
	}

	TArrayView<FIntVector3> FCollectionClothSimPatternFacade::GetSimIndices3D()
	{
		return GetClothCollection()->GetElements(
			GetClothCollection()->GetSimIndices3D(),
			GetClothCollection()->GetSimFacesStart(),
			GetClothCollection()->GetSimFacesEnd(),
			GetElementIndex());
	}

	void FCollectionClothSimPatternFacade::RemoveSimFaces(const TArray<int32>& SortedDeletionList)
	{
		TArray<int32> GlobalIndexSortedDeletionList;
		const int32 Offset = GetSimFacesOffset();
		GlobalIndexSortedDeletionList.SetNumUninitialized(SortedDeletionList.Num());
		for (int32 Idx = 0; Idx < SortedDeletionList.Num(); ++Idx)
		{
			GlobalIndexSortedDeletionList[Idx] = SortedDeletionList[Idx] + Offset;
		}

		GetClothCollection()->RemoveElements(
			ClothCollectionGroup::SimFaces,
			GlobalIndexSortedDeletionList,
			GetClothCollection()->GetSimFacesStart(),
			GetClothCollection()->GetSimFacesEnd(),
			GetElementIndex());
	}

	FCollectionClothSimPatternFacade::FCollectionClothSimPatternFacade(const TSharedRef<FClothCollection>& ClothCollection, int32 InPatternIndex)
		: FCollectionClothSimPatternConstFacade(ClothCollection, InPatternIndex)
	{
	}

	void FCollectionClothSimPatternFacade::SetDefaults()
	{
		const int32 ElementIndex = GetElementIndex();

		(*GetClothCollection()->GetSimVertices2DStart())[ElementIndex] = INDEX_NONE;
		(*GetClothCollection()->GetSimVertices2DEnd())[ElementIndex] = INDEX_NONE;
		(*GetClothCollection()->GetSimFacesStart())[ElementIndex] = INDEX_NONE;
		(*GetClothCollection()->GetSimFacesEnd())[ElementIndex] = INDEX_NONE;
		(*GetClothCollection()->GetSimPatternFabric())[ElementIndex] = INDEX_NONE;
	}

	void FCollectionClothSimPatternFacade::SetNumSimVertices2D(int32 NumSimVertices)
	{
		GetClothCollection()->SetNumElements(
			NumSimVertices,
			ClothCollectionGroup::SimVertices2D,
			GetClothCollection()->GetSimVertices2DStart(),
			GetClothCollection()->GetSimVertices2DEnd(),
			GetElementIndex());
	}

	TArrayView<int32> FCollectionClothSimPatternFacade::GetSimVertex3DLookup()
	{
		return GetClothCollection()->GetElements(
			GetClothCollection()->GetSimVertex3DLookup(),
			GetClothCollection()->GetSimVertices2DStart(),
			GetClothCollection()->GetSimVertices2DEnd(),
			GetElementIndex());
	}

	TSharedRef<class FClothCollection> FCollectionClothSimPatternFacade::GetClothCollection()
	{
		return StaticCastSharedRef<class FClothCollection>(ConstCastSharedRef<class FConstClothCollection>(ClothCollection));
	}
}  // End namespace UE::Chaos::ClothAsset
