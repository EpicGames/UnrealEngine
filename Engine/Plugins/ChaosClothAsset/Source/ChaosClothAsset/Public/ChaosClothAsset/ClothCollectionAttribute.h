// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"

namespace UE::Chaos::ClothAsset
{
	/**
	 * Cloth collection reserved attribute names.
	 */
	namespace ClothCollectionAttribute
	{
		// Lods Group
		inline const FName PhysicsAssetSoftObjectPathName(TEXT("PhysicsAssetSoftObjectPathName"));
		inline const FName SkeletalMeshSoftObjectPathName(TEXT("SkeletalMeshSoftObjectPathName"));
		inline const FName ReferenceBoneName(TEXT("ReferenceBoneName"));
		inline const FName CopyAllUserAttributesToSimModel(TEXT("CopyAllUserAttributesToSimModel"));

		// Solvers Group
		inline const FName SolverGravity(TEXT("SolverGravity"));
		inline const FName SolverAirDamping(TEXT("SolverAirDamping"));
		inline const FName SolverSubSteps(TEXT("SolverSubSteps"));
		inline const FName SolverTimeStep(TEXT("SolverTimeStep"));

		// Fabrics Group
		inline const FName FabricBendingStiffness(TEXT("FabricBendingStiffness"));
		inline const FName FabricBucklingStiffness(TEXT("FabricBucklingStiffness"));
		inline const FName FabricStretchStiffness(TEXT("FabricStretchStiffness"));
		inline const FName FabricBucklingRatio(TEXT("FabricBucklingRatio"));
		inline const FName FabricDensity(TEXT("FabricClothDensity"));
		inline const FName FabricFriction(TEXT("FabricClothFriction"));
		inline const FName FabricDamping(TEXT("FabricClothDamping"));
		inline const FName FabricPressure(TEXT("FabricPressure"));
		inline const FName FabricLayer(TEXT("FabricLayer"));
		inline const FName FabricCollisionThickness(TEXT("FabricollisionThickness"));

		// Seam Group
		inline const FName SeamStitchStart(TEXT("SeamStitchStart"));
		inline const FName SeamStitchEnd(TEXT("SeamStitchEnd"));

		// Seam Stitches Group
		inline const FName SeamStitch2DEndIndices(TEXT("SeamStitch2DEndIndices"));
		inline const FName SeamStitch3DIndex(TEXT("SeamStitch3DIndex"));

		// Sim Patterns Group
		inline const FName SimVertices2DStart(TEXT("SimVertices2DStart"));
		inline const FName SimVertices2DEnd(TEXT("SimVertices2DEnd"));
		inline const FName SimFacesStart(TEXT("SimFacesStart"));
		inline const FName SimFacesEnd(TEXT("SimFacesEnd"));
		inline const FName SimPatternFabric(TEXT("SimPatternFabric"));

		// Render Patterns Group
		inline const FName RenderVerticesStart(TEXT("RenderVerticesStart"));
		inline const FName RenderVerticesEnd(TEXT("RenderVerticesEnd"));
		inline const FName RenderFacesStart(TEXT("RenderFacesStart"));
		inline const FName RenderFacesEnd(TEXT("RenderFacesEnd"));
		inline const FName RenderMaterialSoftObjectPathName(TEXT("RenderMaterialSoftObjectPathName"));
		inline const FName RenderDeformerNumInfluences(TEXT("RenderDeformerNumInfluences"));

		// Sim Faces Group
		inline const FName SimIndices2D(TEXT("SimIndices2D"));
		inline const FName SimIndices3D(TEXT("SimIndices3D"));

		// Sim Vertices 2D Group
		inline const FName SimPosition2D(TEXT("SimPosition2D"));
		inline const FName SimVertex3DLookup(TEXT("SimVertex3DLookup"));
		inline const FName SimImportVertexID(TEXT("SimImportVertexID"));

		// Sim Vertices 3D Group
		inline const FName SimPosition3D(TEXT("SimPosition3D"));
		inline const FName PreResizedSimPosition3D(TEXT("PreResizedSimPosition3D"));
		inline const FName SimNormal(TEXT("SimNormal"));
		inline const FName SimBoneIndices(TEXT("SimBoneIndices"));
		inline const FName SimBoneWeights(TEXT("SimBoneWeights"));
		inline const FName TetherKinematicIndex(TEXT("TetherKinematicIndex"));
		inline const FName TetherReferenceLength(TEXT("TetherReferenceLength"));
		inline const FName SimVertex2DLookup(TEXT("SimVertex2DLookup"));
		inline const FName SeamStitchLookup(TEXT("SeamStitchLookup"));
		inline const FName SimCustomResizingBlend(TEXT("SimCustomResizingBlend"));

		// Sim Vertices 3D Group -- Sim Accessory Mesh
		inline const FName SimAccessoryMeshPosition3DPrefix(TEXT("SimAccessoryMeshPosition3D"));
		inline const FName SimAccessoryMeshNormalPrefix(TEXT("SimAccessoryMeshNormal"));
		inline const FName SimAccessoryMeshBoneIndicesPrefix(TEXT("SimAccessoryMeshBoneIndices"));
		inline const FName SimAccessoryMeshBoneWeightsPrefix(TEXT("SimAccessoryMeshBoneWeights"));

		// Sim Morph Targets Group
		inline const FName SimMorphTargetName(TEXT("SimMorphTargetName"));
		inline const FName SimMorphTargetVerticesStart(TEXT("SimMorphTargetVerticesStart"));
		inline const FName SimMorphTargetVerticesEnd(TEXT("SimMorphTargetVerticesEnd"));

		// Sim Morph Target Vertices Group
		inline const FName SimMorphTargetPositionDelta(TEXT("SimMorphTargetPositionDelta"));
		inline const FName SimMorphTargetTangentZDelta(TEXT("SimMorphTargetTangentZDelta"));
		inline const FName SimMorphTargetSimVertex3DIndex(TEXT("SimMorphTargetSimVertex3DIndex"));

		// Render Faces Group
		inline const FName RenderIndices(TEXT("RenderIndices"));

		// Render Vertices Group
		inline const FName RenderPosition(TEXT("RenderPosition"));
		inline const FName RenderNormal(TEXT("RenderNormal"));
		inline const FName RenderTangentU(TEXT("RenderTangentU"));
		inline const FName RenderTangentV(TEXT("RenderTangentV"));
		inline const FName RenderUVs(TEXT("RenderUVs"));
		inline const FName RenderColor(TEXT("RenderColor"));
		inline const FName RenderBoneIndices(TEXT("RenderBoneIndices"));
		inline const FName RenderBoneWeights(TEXT("RenderBoneWeights"));
		inline const FName RenderDeformerPositionBaryCoordsAndDist(TEXT("RenderDeformerPositionBaryCoordsAndDist"));
		inline const FName RenderDeformerNormalBaryCoordsAndDist(TEXT("RenderDeformerNormalBaryCoordsAndDist"));
		inline const FName RenderDeformerTangentBaryCoordsAndDist(TEXT("RenderDeformerTangentBaryCoordsAndDist"));
		inline const FName RenderDeformerSimIndices3D(TEXT("RenderDeformerSimIndices3D"));
		inline const FName RenderDeformerWeight(TEXT("RenderDeformerWeight"));
		inline const FName RenderDeformerSkinningBlend(TEXT("RenderDeformerSkinningBlend"));
		inline const FName RenderCustomResizingBlend(TEXT("RenderCustomResizingBlend"));

		// Custom Resizing Regions Group
		inline const FName CustomResizingRegionSet(TEXT("CustomResizingRegionSet"));
		inline const FName CustomResizingRegionType(TEXT("CustomResizingRegionType"));
		// Sim Accessory Meshes Group
		inline const FName SimAccessoryMeshName(TEXT("SimAccessoryMeshName"));
		inline const FName SimAccessoryMeshPosition3DAttribute(TEXT("SimAccessoryMeshPosition3DAttribute"));
		inline const FName SimAccessoryMeshNormalAttribute(TEXT("SimAccessoryMeshNormalAttribute"));
		inline const FName SimAccessoryMeshBoneIndicesAttribute(TEXT("SimAccessoryMeshBoneIndicesAttribute"));
		inline const FName SimAccessoryMeshBoneWeightsAttribute(TEXT("SimAccessoryMeshBoneWeightsAttribute"));

		//~ Deprecated attributes
		// Lods Group
		UE_DEPRECATED(5.7, "Use PhysicsAssetSoftObjectPath instead");
		inline const FName PhysicsAssetPathName(TEXT("PhysicsAssetPathName"));
		UE_DEPRECATED(5.7, "Use SkeletalMeshSoftObjectPathName instead");
		inline const FName SkeletalMeshPathName(TEXT("SkeletalMeshPathName"));

		// Render Patterns Group
		UE_DEPRECATED(5.7, "Use RenderMaterialSoftObjectPathName instead");
		inline const FName RenderMaterialPathName(TEXT("RenderMaterialPathName"));
	}
}  // End namespace UE::Chaos::ClothAsset
