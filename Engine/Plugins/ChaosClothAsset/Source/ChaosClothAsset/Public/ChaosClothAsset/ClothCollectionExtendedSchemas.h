// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Misc/EnumClassFlags.h"

namespace UE::Chaos::ClothAsset
{
	/** Cloth collection extended schemas. */
	enum class EClothCollectionExtendedSchemas : uint32
	{
		None = 0,
		RenderDeformer = 1 << 1,
		Solvers = 1 << 2,
		Fabrics = 1 << 3,
		SimMorphTargets = 1 << 4,
		CookedOnly = 1 << 5, // If this flag is found, only cooked schemas will be considered.
		Resizing = 1 << 6,
		SimAccessoryMeshes = 1 << 7,
		Import = 1 << 8
	};
	ENUM_CLASS_FLAGS(EClothCollectionExtendedSchemas)
}  // End namespace UE::Chaos::ClothAsset
