// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"

namespace UE::Chaos::ClothAsset
{
	/**
	 * Cloth collection group names.
	 */
	namespace ClothCollectionGroup
	{
		/** LOD information (only one LOD per collection). */
		inline const FName Lods = FName(TEXT("Lods"));
		/** Solvers information (only one Solver per collection for now). */
		inline const FName Solvers = FName(TEXT("Solvers"));
		/** Collection of seam stitches. */
		inline const FName Seams = FName(TEXT("Seams"));
		/** Contains pairs of stitched sim vertex indices. */
		inline const FName SeamStitches = FName(TEXT("SeamStitches"));
		/** Contains sim pattern relationships to other groups. */
		inline const FName SimPatterns = FName(TEXT("SimPatterns"));
		/** Contains render pattern relationships to other groups. */
		inline const FName RenderPatterns = FName(TEXT("RenderPatterns"));
		/** Contains indices to sim vertices. */
		inline const FName SimFaces = FName(TEXT("SimFaces"));
		/** Contains 2D positions. */
		inline const FName SimVertices2D = FName(TEXT("SimVertices2D"));
		/** Contains 3D positions. */
		inline const FName SimVertices3D = FName(TEXT("SimVertices3D"));
		/** Contains indices to render vertex. */
		inline const FName RenderFaces = FName(TEXT("RenderFaces"));
		/** Contains 3D render model. */
		inline const FName RenderVertices = FName(TEXT("RenderVertices"));
		/** Contains all the fabrics simulation parameters used by the sim patterns . */
		inline const FName Fabrics = FName(TEXT("Fabrics"));
		/** Contains sim morph target relationships to other groups. */
		inline const FName SimMorphTargets = FName(TEXT("SimMorphTargets"));
		/** Contains morph target vertices */
		inline const FName SimMorphTargetVertices = FName(TEXT("SimMorphTargetVertices"));
		/** Contains custom region resizing data */
		inline const FName CustomResizingRegions = FName(TEXT("CustomResizingRegions"));
		/** Contains sim accessory mesh relationships to other groups. */
		inline const FName SimAccessoryMeshes = FName(TEXT("SimAccessoryMeshes"));
	}
}  // End namespace UE::Chaos::ClothAsset
