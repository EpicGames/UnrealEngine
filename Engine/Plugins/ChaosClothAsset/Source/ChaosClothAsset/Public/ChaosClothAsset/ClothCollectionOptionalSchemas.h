// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Misc/CoreMiscDefines.h"

UE_DEPRECATED_HEADER(5.6, "Use ClothCollectionExtendedSchemas.h instead of ClothCollectionOptionalSchemas.h")

#include "ChaosClothAsset/ClothCollectionExtendedSchemas.h"

namespace UE::Chaos::ClothAsset
{
	UE_DEPRECATED(5.6, "Use EClothCollectionExtendedSchemas instead") typedef enum EClothCollectionExtendedSchemas EClothCollectionOptionalSchemas;
}  // End namespace UE::Chaos::ClothAsset
