// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

#define UE_API CHAOSCLOTHASSET_API

struct FManagedArrayCollection;
namespace UE::Geometry { class FDynamicMesh3; }
struct FMeshDescription;

namespace UE::Chaos::ClothAsset
{
	class FCollectionClothFacade;
	class FCollectionClothConstFacade;

	/**
	 * Geometry tools operating on cloth collections.
	 */
	struct FClothGeometryTools
	{

		/** Return whether at least one pattern of this collection has any faces to simulate. */
		static UE_API bool HasSimMesh(const TSharedRef<const FManagedArrayCollection>& ClothCollection);

		/** Return whether at least one pattern of this collection has any faces to render. */
		static UE_API bool HasRenderMesh(const TSharedRef<const FManagedArrayCollection>& ClothCollection);

		/** Delete the render mesh data. */
		static UE_API void DeleteRenderMesh(const TSharedRef<FManagedArrayCollection>& ClothCollection);

		/** Delete the sim mesh data. */
		static UE_API void DeleteSimMesh(const TSharedRef<FManagedArrayCollection>& ClothCollection);

		/** Remove all tethers. */
		static UE_API void DeleteTethers(const TSharedRef<FManagedArrayCollection>& ClothCollection);

		/** Remove all selections, or the selections from a specific group if specified. */
		static UE_API void DeleteSelections(const TSharedRef<FManagedArrayCollection>& ClothCollection, const FName Group = NAME_None);

		/** Turn the sim mesh portion of this ClothCollection into a render mesh. */
		static UE_API void CopySimMeshToRenderMesh(const TSharedRef<FManagedArrayCollection>& ClothCollection, const FSoftObjectPath& RenderMaterialPathName, bool bSingleRenderPattern);

		/** Reverse the mesh normals. Will reverse all normals if pattern selection is empty. */
		static UE_API void ReverseMesh(
			const TSharedRef<FManagedArrayCollection>& ClothCollection,
			bool bReverseSimMeshNormals,
			bool bReverseSimMeshWindingOrder,
			bool bReverseRenderMeshNormals,
			bool bReverseRenderMeshWindingOrder,
			const TArray<int32>& SimPatternSelection,
			const TArray<int32>& RenderPatternSelection);

		/** Recalculate the render mesh normals. */
		static UE_API void RecalculateRenderMeshNormals(const TSharedRef<FManagedArrayCollection>& ClothCollection);

		/**
		 * Set the skinning weights for all of the sim/render vertices in ClothCollection to be bound to the root node.
		 *
		 * @param Lods if empty will apply the change to all LODs. Otherwise only LODs specified in the array (if exist) are affected.
		 */
		static UE_API void BindMeshToRootBone(
			const TSharedRef<FManagedArrayCollection>& ClothCollection,
			bool bBindSimMesh,
			bool bBindRenderMesh);

		/**
		* Build (or add to) a ClothCollection Sim Mesh from the given 2D and 3D mesh data. Uses a Polygroup Attribute Layer to specify Pattern topology.
		* 
		* @param ClothCollection					The cloth collection whose sim mesh (2D and 3D) will be modified
		* @param Mesh2D								Input 2D sim mesh data
		* @param Mesh3D								Input 3D sim mesh data
		* @param PatternIndexLayerId				Specifies which PolyGroup layer on Mesh2D contains pattern index per triangle information
		* @param bTransferWeightMaps				Copy any weight map layers from Mesh2D into the ClothCollection sim mesh
		* @param bTransferSimSkinningData			Copy any skinning weight data from Mesh2D into the ClothCollection sim mesh
		* @param bAppend							Whether to add the new mesh data to the existing sim mesh, or crete new sim mesh in the collection
		* @param OutDynamicMeshToClothVertexMap		(Output) Computed map of vertex indices in the input Meshes to vertex indices in the output ClothCollection
		*/
		static UE_API void BuildSimMeshFromDynamicMeshes(
			const TSharedRef<FManagedArrayCollection>& ClothCollection,
			const UE::Geometry::FDynamicMesh3& Mesh2D,
			const UE::Geometry::FDynamicMesh3& Mesh3D,
			int32 PatternIndexLayerId,
			bool bTransferWeightMaps,
			bool bTransferSimSkinningData,
			bool bAppend,
			TMap<int, int32>& OutDynamicMeshToClothVertexMap);

		/**
		* Unwrap and build SimMesh data from a DynamicMesh
		* Normals are only imported if the DynamicMesh has both a UVOverlay and a NormalOverlay
		*/
		static UE_API void BuildSimMeshFromDynamicMesh(
			const TSharedRef<FManagedArrayCollection>& ClothCollection,
			const UE::Geometry::FDynamicMesh3& DynamicMesh, int32 UVChannelIndex, const FVector2f& UVScale, bool bAppend, bool bImportNormals = false, TArray<int32>* OutSim2DToSourceIndex = nullptr);

		/**
		* Remove (topologically) degenerate triangles. Remove any vertices that aren't in a triangle. Compact any lookup arrays that contain INDEX_NONEs.
		* Remove any empty patterns.
		*/
		static UE_API void CleanupAndCompactMesh(const TSharedRef<FManagedArrayCollection>& ClothCollection);

		/** Strip maps, sets, etc. that are not referenced by properties from a cloth collection. */
		static UE_API void StripUnusedUserAttributes(const TSharedRef<FManagedArrayCollection>& ClothCollection, bool bStripUnusedSimWeightMaps, const TArray<FName>& ExtraSimWeightMaps, bool bStripUnusedSets, const TArray<FName>& ExtraSets, bool bStripUnusedFaceIntMaps, const TArray<FName>& ExtraFaceIntMaps, bool bStripUnusedRenderWeightMaps, const TArray<FName>& ExtraRenderWeightMaps);

		/**
		 * Find sets of connected stitches for the input stitches given in random order.
	     * Stitch (A, B) is connected to stitch (C, D) if there exist edges {(A, C), (B, D)} *or* {(A, D), (B, C)} in the given DynamicMesh.
	     */
		static UE_API void BuildConnectedSeams(const TArray<FIntVector2>& InputStitches,
			const UE::Geometry::FDynamicMesh3& Mesh,
			TArray<TArray<FIntVector2>>& Seams);

		/**
		* Find sets of connected stitches for the given seam.
		* Stitch (A, B) is connected to stitch (C, D) if there exist edges {(A, C), (B, D)} *or* {(A, D), (B, C)} in the given DynamicMesh.
		* ClothCollection meshes must be manifold.
		*/
		static UE_API void BuildConnectedSeams2D(const TSharedRef<const FManagedArrayCollection>& ClothCollection, 
			int32 SeamIndex,
			const UE::Geometry::FDynamicMesh3& Mesh,
			TArray<TArray<FIntVector2>>& Seams);


		/**
		 * Use Poisson disk sampling to get a set of evenly-spaced vertices
		 * 
		 * @param VertexPositions set of vertex points to sample from
		 * @param CullDiameterSq squared minimum distance between samples
		 * @param OutVertexSet indices of the sampled subset of VertexPositions
		 */
		static UE_API void SampleVertices(const TConstArrayView<FVector3f> VertexPositions, float CullDiameterSq, TSet<int32>& OutVertexSet);

		/**
		 * Get a copy of the selection, converting to the desired group if possible.
		 * Currently only conversions between vertex and face components on the same mesh type are supported.
		 * @param ClothCollection to query
		 * @param SelectionName the selection name
		 * @param GroupName the group name
		 * @param bSecondarySelection get the secondary selection
		 * @param OutSelectionSet copy of the selection. Unchanged when function returns false.
		 * @return success (will return false if the selection is not found or conversion is not possible)
		 */
		UE_DEPRECATED(5.5, "Please use the version with no bSecondarySelection parameter")
		static UE_API bool ConvertSelectionToNewGroupType(const TSharedRef<const FManagedArrayCollection>& ClothCollection, const FName& SelectionName, const FName& GroupName, bool bSecondarySelection, TSet<int32>& OutSelectionSet);

		/**
		 * Get a copy of the selection, converting to the desired group if possible.
		 * Currently only conversions between vertex and face components on the same mesh type are supported.
		 * @param ClothCollection to query
		 * @param SelectionName the selection name
		 * @param GroupName the group name
		 * @param OutSelectionSet copy of the selection. Unchanged when function returns false.
		 * @return success (will return false if the selection is not found or conversion is not possible)
		 */
		static UE_API bool ConvertSelectionToNewGroupType(const TSharedRef<const FManagedArrayCollection>& ClothCollection, const FName& SelectionName, const FName& GroupName, TSet<int32>& OutSelectionSet);

		/**
		* Create a selection set that selects all members of a group. 
		*/
		static UE_API void SelectAllInGroupType(const TSharedRef<FManagedArrayCollection>& ClothCollection, const FName& SelectionName, const FName& GroupName);

		/**
		 * Transfer a vertex weight map from a source to target mesh
		 */
		static UE_API void TransferWeightMap(
			const TConstArrayView<FVector3f>& SourcePositions,
			const TConstArrayView<FIntVector3>& InSourceIndices,
			const TConstArrayView<float>& SourceWeights,
			const TConstArrayView<FVector3f>& TargetPositions,
			const TConstArrayView<FVector3f>& TargetNormals,
			const TConstArrayView<FIntVector3>& InTargetIndices,
			const TArrayView<float>& TargetWeights);

		/**
		 * Generate KinematicVertices3D set from the given MaxDistance weight map, MaxDistance values, and any additional kinematic vertices.
		 */
		static UE_API TSet<int32> GenerateKinematicVertices3D(const TSharedRef<FManagedArrayCollection>& ClothCollection, const FName& MaxDistanceMap, const FVector2f& MaxDistanceValues, const FName& InputKinematicVertices, float KinematicDistanceThreshold = 0.1f);

		/**
		 * Update the render mesh by applying the proxy deformer.
		 */
		static UE_API void ApplyProxyDeformer(const TSharedRef<FManagedArrayCollection>& ClothCollection, bool bIgnoreSkinningBlend);
		static UE_API void ApplyProxyDeformer(FCollectionClothFacade& ClothFacade, bool bIgnoreSkinningBlend, const TConstArrayView<FVector3f>& SimPositions, const TConstArrayView<FVector3f>& SimNormals);
	};
}  // End namespace UE::Chaos::ClothAsset

#undef UE_API
