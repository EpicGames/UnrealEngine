// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

#define UE_API CHAOSCLOTHASSET_API

namespace UE::Chaos::ClothAsset
{
	struct FDefaultFabric
	{
		inline static constexpr float BendingStiffness = 100.0f;
		inline static constexpr float StretchStiffness = 100.0f;
		inline static constexpr float BucklingRatio = 0.5f;
		inline static constexpr float BucklingStiffness = 50.0f;
		inline static constexpr float Density = 0.35f;
		inline static constexpr float Friction = 0.8f;
		inline static constexpr float Damping = 0.1f;
		inline static constexpr float Pressure = 0.0f;
		inline static constexpr int32 Layer = INDEX_NONE;
		inline static constexpr float CollisionThickness = 1.0f;
		inline static constexpr float ClothCollisionThickness = 0.f;
		inline static constexpr float SelfFriction = 0.0f;
		inline static constexpr float SelfCollisionThickness = 0.5f;
	};
	
	/**
	 * Cloth Asset collection fabric facade class to access cloth fabric data.
	 * Constructed from FCollectionClothConstFacade.
	 * Const access (read only) version.
	 */
	class FCollectionClothFabricConstFacade
	{
	public:
		FCollectionClothFabricConstFacade() = delete;

		FCollectionClothFabricConstFacade(const FCollectionClothFabricConstFacade&) = delete;
		FCollectionClothFabricConstFacade& operator=(const FCollectionClothFabricConstFacade&) = delete;

		FCollectionClothFabricConstFacade(FCollectionClothFabricConstFacade&&) = default;
		FCollectionClothFabricConstFacade& operator=(FCollectionClothFabricConstFacade&&) = default;

		virtual ~FCollectionClothFabricConstFacade() = default;

		/** Anisotropic fabric datas structure (Weft,Warp,Bias) */
		struct FAnisotropicData
		{
			UE_API FAnisotropicData(const float WeftValue, const float WarpValue, const float BiasValue);
			UE_API FAnisotropicData(const FVector3f& VectorDatas);
			UE_API FAnisotropicData(const float& FloatDatas);
			
			UE_API FVector3f GetVectorDatas() const;
			
			
			float Weft;
			float Warp;
			float Bias;
		};

		/** Return the anisotropic bending stiffness */
		UE_API FAnisotropicData GetBendingStiffness() const;

		/** Return the buckling ratio */
		UE_API float GetBucklingRatio() const;

		/** Return the anisotropic buckling stiffness */
		UE_API FAnisotropicData GetBucklingStiffness() const;

		/** Return the anisotropic stretch stiffness */
		UE_API FAnisotropicData GetStretchStiffness() const;

		/** Return the fabric density */
		UE_API float GetDensity() const;
		
		/** Return the fabric damping */
		UE_API float GetDamping() const;

		/** Return the fabric friction */
		UE_API float GetFriction() const;

		/** Return the fabric pressure */
		UE_API float GetPressure() const;

		/** Return the fabric layer */
		UE_API int32 GetLayer() const;

		/** Return the collision thickness */
		UE_API float GetCollisionThickness() const;

		/** Get the global element index */
		int32 GetElementIndex() const { return GetBaseElementIndex() + FabricIndex; }

	protected:
		friend class FCollectionClothFabricFacade;  // For other instances access
		friend class FCollectionClothConstFacade;
		FCollectionClothFabricConstFacade(const TSharedRef<const class FConstClothCollection>& InClothCollection, int32 InFabricIndex);

		static constexpr int32 GetBaseElementIndex() { return 0; }

		/** Cloth collection modified by  the fabric facade */
		TSharedRef<const class FConstClothCollection> ClothCollection;

		/** Fabric index that will be referred in the sim patterns */
		int32 FabricIndex;
	};

	/**
	 * Cloth Asset collection fabric facade class to access cloth fabric data.
	 * Constructed from FCollectionClothFacade.
	 * Non-const access (read/write) version.
	 */
	class FCollectionClothFabricFacade final : public FCollectionClothFabricConstFacade
	{
	public:
		FCollectionClothFabricFacade() = delete;

		FCollectionClothFabricFacade(const FCollectionClothFabricFacade&) = delete;
		FCollectionClothFabricFacade& operator=(const FCollectionClothFabricFacade&) = delete;

		FCollectionClothFabricFacade(FCollectionClothFabricFacade&&) = default;
		FCollectionClothFabricFacade& operator=(FCollectionClothFabricFacade&&) = default;

		virtual ~FCollectionClothFabricFacade() override = default;

		/** Initialize the cloth fabric with simulation parameters. */
		UE_API void Initialize(const FAnisotropicData& BendingStiffness, const float BucklingRatio,
			const FAnisotropicData& BucklingStiffness, const FAnisotropicData& StretchStiffness,
			const float Density, const float Friction, const float Damping, const float Pressure, const int32 Layer, const float CollisionThickness);

		/** Initialize the cloth fabric with another one. */
		UE_API void Initialize(const FCollectionClothFabricConstFacade& OtherFabricFacade);

		/** Initialize the cloth fabric from another one and from pattern datas */
		UE_API void Initialize(const FCollectionClothFabricConstFacade& OtherFabricFacade,
					const float Pressure, const int32 Layer, const float CollisionThickness);

	private:
		friend class FCollectionClothFacade;
		FCollectionClothFabricFacade(const TSharedRef<class FClothCollection>& InClothCollection, int32 InFabricIndex);

		/** Set default values to the fabric properties */
		void SetDefaults();

		/** Reset the fabric values properties */
		void Reset();

		/** Get the non const cloth collection */
		TSharedRef<class FClothCollection> GetClothCollection();
	};
}  // End namespace UE::Chaos::ClothAsset

#undef UE_API
