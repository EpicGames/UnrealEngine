// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Vector.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "ChaosClothAsset/ClothCollectionOptionalSchemas.h"
#endif
#include "ChaosClothAsset/ClothCollectionExtendedSchemas.h"
#include "ChaosClothAsset/CollectionClothFabricFacade.h"
#include "ChaosClothAsset/CollectionClothRenderPatternFacade.h"
#include "ChaosClothAsset/CollectionClothSeamFacade.h"
#include "ChaosClothAsset/CollectionClothSimAccessoryMeshFacade.h"
#include "ChaosClothAsset/CollectionClothSimMorphTargetFacade.h"
#include "ChaosClothAsset/CollectionClothSimPatternFacade.h"
#include "ChaosClothAsset/IsUserAttributeType.h"

#define UE_API CHAOSCLOTHASSET_API

namespace Chaos
{
class FChaosArchive;
}

namespace UE::Chaos::ClothAsset
{
	struct FDefaultSolver
	{
		inline static const FVector3f Gravity = FVector3f(0.0f, 0.0f, -980.665f);
		inline static constexpr float AirDamping = 0.035f;
		inline static constexpr int32 SubSteps = 1;
		inline static constexpr float TimeStep = 0.033f;
	};

	/**
	 * Cloth Asset collection facade class focused on draping and pattern information.
	 * Const access (read only) version.
	 */
	class FCollectionClothConstFacade
	{
	public:
		UE_API explicit FCollectionClothConstFacade(const TSharedRef<const FManagedArrayCollection>& ManagedArrayCollection);

		FCollectionClothConstFacade() = delete;

		FCollectionClothConstFacade(const FCollectionClothConstFacade&) = delete;
		FCollectionClothConstFacade& operator=(const FCollectionClothConstFacade&) = delete;

		FCollectionClothConstFacade(FCollectionClothConstFacade&&) = default;
		FCollectionClothConstFacade& operator=(FCollectionClothConstFacade&&) = default;

		virtual ~FCollectionClothConstFacade() = default;

		/** Return whether the facade is defined on the collection. */
		UE_API bool IsValid(EClothCollectionExtendedSchemas OptionalSchemas = EClothCollectionExtendedSchemas::None) const;

		/** Return whether the facade has a non-empty simulation mesh data. */
		UE_API bool HasValidSimulationData() const;

		/** Return whether the facade has a non-empty sim mesh data. */
		UE_API bool HasValidRenderData() const;

		/** Return whether the facade has a non-empty sim and render mesh data. */
		UE_API bool HasValidData() const;

		UE_API uint32 CalculateTypeHash(bool bIncludeWeightMaps, uint32 PreviousHash = 0) const;
		UE_API uint32 CalculateWeightMapTypeHash(uint32 PreviousHash = 0) const;
		template<typename T UE_REQUIRES(TIsUserAttributeType<T>::Value)>
		uint32 CalculateUserDefinedAttributesTypeHash(const FName& GroupName, uint32 PreviousHash = 0) const;

		//~ LOD (single per collection) Group
		/** Return the physics asset path names used for this collection. */
		UE_API const FSoftObjectPath& GetPhysicsAssetSoftObjectPathName() const;
		/** Return the skeleton asset path names used for this collection. */
		UE_API const FSoftObjectPath& GetSkeletalMeshSoftObjectPathName() const;
		UE_DEPRECATED(5.7, "Use GetPhysicsAssetSoftObjectPathName instead")
		const FString& GetPhysicsAssetPathName() const
		{
			static const FString EmptyString;
			return EmptyString;
		}
		UE_DEPRECATED(5.7, "Use GetSkeletalMeshSoftObjectPathName instead")
		const FString& GetSkeletalMeshPathName() const
		{
			static const FString EmptyString;
			return EmptyString;
		}
		/** Return the reference bone name used for this collection. */
		UE_API FName GetReferenceBoneName() const;
		UE_API bool GetCopyAllUserAttributesToSimModel() const;

		//~ Solver (single per collection) Group
		/** Return true if the solver group has one element*/
		UE_API bool HasSolverElement() const;
		/** Return the solver gravity vector used for this collection. */
		UE_API const FVector3f& GetSolverGravity() const;
		/** Return the solver air damping used for this collection. */
		UE_API float GetSolverAirDamping() const;
		/** Return the solver time step used for this collection. */
		UE_API float GetSolverTimeStep() const;
		/** Return the solver sub steps used for this collection. */
		UE_API int32 GetSolverSubSteps() const;

		//~ Sim Vertices 2D Group
		/** Return the total number of 2D simulation vertices for this collection. */
		UE_API int32 GetNumSimVertices2D() const;
		UE_API TConstArrayView<FVector2f> GetSimPosition2D() const;
		UE_API TConstArrayView<int32> GetSimVertex3DLookup() const;
		UE_API TConstArrayView<int32> GetSimImportVertexID() const;

		//~ Sim Vertices 3D Group
		/** Return the total number of 3D simulation vertices for this collection. */
		UE_API int32 GetNumSimVertices3D() const;
		UE_API TConstArrayView<FVector3f> GetSimPosition3D() const;
		UE_API TConstArrayView<FVector3f> GetPreResizedSimPosition3D() const;
		UE_API TConstArrayView<FVector3f> GetSimNormal() const;
		UE_API TConstArrayView<TArray<int32>> GetSimBoneIndices() const;
		UE_API TConstArrayView<TArray<float>> GetSimBoneWeights() const;
		UE_API TConstArrayView<TArray<int32>> GetTetherKinematicIndex() const;
		UE_API TConstArrayView<TArray<float>> GetTetherReferenceLength() const;
		UE_API TConstArrayView<TArray<int32>> GetSimVertex2DLookup() const;
		UE_API TConstArrayView<TArray<int32>> GetSeamStitchLookup() const;
		UE_API TConstArrayView<float> GetSimCustomResizingBlend() const;

		//~ Sim Faces Group
		/** Return the total number of simulation faces for this collection across all patterns. */
		UE_API int32 GetNumSimFaces() const;
		UE_API TConstArrayView<FIntVector3> GetSimIndices2D() const;
		UE_API TConstArrayView<FIntVector3> GetSimIndices3D() const;

		//~ Sim Patterns Group
		/** Return the number of patterns in this collection. */
		UE_API int32 GetNumSimPatterns() const;
		/** Return a pattern facade for the specified pattern index. */
		UE_API FCollectionClothSimPatternConstFacade GetSimPattern(int32 PatternIndex) const;
		/** Convenience to find which sim pattern a 2D vertex belongs to */
		UE_API int32 FindSimPatternByVertex2D(int32 Vertex2DIndex) const;
		/** Convenience to find which sim pattern a sim face belongs to */
		UE_API int32 FindSimPatternByFaceIndex(int32 FaceIndex) const;

		//~ Sim Morph Targets Group
		/** Return the number of sim morph targets in this collection. */
		UE_API int32 GetNumSimMorphTargets() const;
		/** Lookup sim morph target by name. Returns INDEX_NONE if not found. */
		UE_API int32 FindSimMorphTargetIndexByName(const FString& MorphTargetName) const;
		/** Return a sim morph target facade for the specified morph target index. */
		UE_API FCollectionClothSimMorphTargetConstFacade GetSimMorphTarget(int32 MorphTargetIndex) const;
		/** Return a view of all sim morph target names */
		UE_API TConstArrayView<FString> GetSimMorphTargetName() const;

		//~ Sim Accessory Meshes Group
		/** Return the number of accessory meshes in this collection. */
		UE_API int32 GetNumSimAccessoryMeshes() const;
		/** Lookup sim accessory mesh by name. Returns INDEX_NONE if not found. */
		UE_API int32 FindSimAccessoryMeshIndexByName(const FName& AccessoryMeshName) const;
		/** Return the sim accessory mesh facade for the specified accessor mesh index */
		UE_API FCollectionClothSimAccessoryMeshConstFacade GetSimAccessoryMesh(int32 MeshIndex) const;
		UE_API TConstArrayView<FName> GetSimAccessoryMeshName() const;
		UE_API TConstArrayView<FName> GetSimAccessoryMeshPosition3DAttribute() const;
		UE_API TConstArrayView<FName> GetSimAccessoryMeshNormalAttribute() const;
		UE_API TConstArrayView<FName> GetSimAccessoryMeshBoneIndicesAttribute() const;
		UE_API TConstArrayView<FName> GetSimAccessoryMeshBoneWeightsAttribute() const;

		//~ Sim Morph Target Vertices Group
		/** Return the total number of sim morph target vertices in this collection. */
		UE_API int32 GetNumSimMorphTargetVertices() const;
		/** Return a view of all sim morph target position deltas */
		UE_API TConstArrayView<FVector3f> GetSimMorphTargetPositionDelta() const;
		/** Return a view of all sim morph target tangent z deltas */
		UE_API TConstArrayView<FVector3f> GetSimMorphTargetTangentZDelta() const;
		/** Return a view of all sim morph target sim vertex 3d indices */
		UE_API TConstArrayView<int32> GetSimMorphTargetSimVertex3DIndex() const;

		//~ Render Patterns Group
		/** Return the number of patterns in this collection. */
		UE_API int32 GetNumRenderPatterns() const;
		/** Return a pattern facade for the specified pattern index. */
		UE_API FCollectionClothRenderPatternConstFacade GetRenderPattern(int32 PatternIndex) const;
		/** Return a view of all the render deformer number of influences used on this collection across all patterns. */
		UE_API TConstArrayView<int32> GetRenderDeformerNumInfluences() const;
		/** Return a view of all the render materials used on this collection across all patterns. */
		UE_API TConstArrayView<FSoftObjectPath> GetRenderMaterialSoftObjectPathName() const;
		/** Convenience to find which render pattern a render vertex belongs to */
		UE_API int32 FindRenderPatternByVertex(int32 VertexIndex) const;
		/** Convenience to find which render pattern a render face belongs to */
		UE_API int32 FindRenderPatternByFaceIndex(int32 FaceIndex) const;
		UE_DEPRECATED(5.7, "Use GetRenderMaterialSoftObjectPathName instead")
		TConstArrayView<FString> GetRenderMaterialPathName() const
		{
			return TConstArrayView<FString>();
		}

		//~ Seam Group
		/** Return the number of seams in this collection. */
		UE_API int32 GetNumSeams() const;
		/** Return a seam facade for the specified seam index. */
		UE_API FCollectionClothSeamConstFacade GetSeam(int32 SeamIndex) const;

		//~ Fabric Group
		/** Return the number of fabrics in this collection. */
		UE_API int32 GetNumFabrics() const;
		/** Return a fabric facade for the specified fabric index. */
		UE_API FCollectionClothFabricConstFacade GetFabric(int32 FabricIndex) const;

		//~ Render Vertices Group
		/** Return the total number of render vertices for this collection. */
		UE_API int32 GetNumRenderVertices() const;
		UE_API TConstArrayView<FVector3f> GetRenderPosition() const;
		UE_API TConstArrayView<FVector3f> GetRenderNormal() const;
		UE_API TConstArrayView<FVector3f> GetRenderTangentU() const;
		UE_API TConstArrayView<FVector3f> GetRenderTangentV() const;
		UE_API TConstArrayView<TArray<FVector2f>> GetRenderUVs() const;
		UE_API TConstArrayView<FLinearColor> GetRenderColor() const;
		UE_API TConstArrayView<TArray<int32>> GetRenderBoneIndices() const;
		UE_API TConstArrayView<TArray<float>> GetRenderBoneWeights() const;
		UE_API TConstArrayView<TArray<FVector4f>> GetRenderDeformerPositionBaryCoordsAndDist() const;
		UE_API TConstArrayView<TArray<FVector4f>> GetRenderDeformerNormalBaryCoordsAndDist() const;
		UE_API TConstArrayView<TArray<FVector4f>> GetRenderDeformerTangentBaryCoordsAndDist() const;
		UE_API TConstArrayView<TArray<FIntVector3>> GetRenderDeformerSimIndices3D() const;
		UE_API TConstArrayView<TArray<float>> GetRenderDeformerWeight() const;
		UE_API TConstArrayView<float> GetRenderDeformerSkinningBlend() const;
		UE_API TConstArrayView<float> GetRenderCustomResizingBlend() const;

		//~ Render Faces Group
		UE_API int32 GetNumRenderFaces() const;
		UE_API TConstArrayView<FIntVector3> GetRenderIndices() const;

		//~ Custom Resizing Regions Group
		UE_API int32 GetNumCustomResizingRegions() const;
		UE_API TConstArrayView<FString> GetCustomResizingRegionSet() const;
		UE_API TConstArrayView<int32> GetCustomResizingRegionType() const;

		//~ Weight Maps
		/** Return whether this cloth collection has the specified weight map. */
		UE_API bool HasWeightMap(const FName& Name) const;
		/** Return the name of all user weight maps on this cloth collection. */
		UE_API TArray<FName> GetWeightMapNames() const;
		UE_API TConstArrayView<float> GetWeightMap(const FName& Name) const;

		//~ Other User-Defined Attributes (not instantiated for bools)
		template<typename T UE_REQUIRES(TIsUserAttributeType<T>::Value)>
		bool HasUserDefinedAttribute(const FName& Name, const FName& GroupName) const;
		template<typename T UE_REQUIRES(TIsUserAttributeType<T>::Value)>
		TArray<FName> GetUserDefinedAttributeNames(const FName& GroupName) const;
		template<typename T UE_REQUIRES(TIsUserAttributeType<T>::Value)>
		TConstArrayView<T> GetUserDefinedAttribute(const FName& Name, const FName& GroupName) const;
		static UE_API bool IsValidClothCollectionGroupName(const FName& GroupName);
		static UE_API TArray<FName> GetValidClothCollectionGroupName();

		UE_API void BuildSimulationMesh(TArray<FVector3f>& Positions, TArray<FVector3f>& Normals, TArray<uint32>& Indices, TArray<FVector2f>& PatternsPositions, TArray<uint32>& PatternsIndices, 
			TArray<uint32>& PatternToWeldedIndices, TArray<TArray<int32>>* OptionalWeldedToPatternIndices = nullptr) const;

	protected:
		explicit FCollectionClothConstFacade(const TSharedRef<const class FConstClothCollection>& ClothCollection);
		
		friend class FCollectionClothFacade;
		TSharedRef<const class FConstClothCollection> ClothCollection;

		friend class FCollectionClothFacade;  // To enable access from a different instance
		friend class FCollectionClothSeamFacade;
		friend class FCollectionClothSeamConstFacade;
		friend class FCollectionClothFabricFacade;
		friend class FCollectionClothFabricConstFacade;
	};

	/**
	 * Cloth Asset collection facade class focused on draping and pattern information.
	 * Non-const access (read/write) version.
	 */
	class FCollectionClothFacade final : public FCollectionClothConstFacade
	{
	public:
		UE_API explicit FCollectionClothFacade(const TSharedRef<FManagedArrayCollection>& ManagedArrayCollection);

		FCollectionClothFacade() = delete;

		FCollectionClothFacade(const FCollectionClothFacade&) = delete;
		FCollectionClothFacade& operator=(const FCollectionClothFacade&) = delete;

		FCollectionClothFacade(FCollectionClothFacade&&) = default;
		FCollectionClothFacade& operator=(FCollectionClothFacade&&) = default;
		virtual ~FCollectionClothFacade() override = default;

		/** Create this facade's groups and attributes. */
		UE_API void DefineSchema(EClothCollectionExtendedSchemas OptionalSchemas = EClothCollectionExtendedSchemas::None);

		/** Post-Serialize upgrade cloth collection */
		UE_API void PostSerialize(const FArchive& Ar);

		/** Remove all LODs from this cloth. */
		UE_API void Reset();

		/** Initialize the cloth using another cloth collection. */
		UE_API void Initialize(const FCollectionClothConstFacade& Other);

		/** Append data from another cloth collection. */
		UE_API void Append(const FCollectionClothConstFacade& Other);

		/** Copy only data that is not stripped on cook */
		UE_API void InitializeCookedOnly(const FCollectionClothConstFacade& Other);

		//~ LOD (single per collection) Group
		/** Set the physics asset path name. */
		UE_API void SetPhysicsAssetSoftObjectPathName(const FSoftObjectPath& PathName);
		/** Set the skeletal mesh asset path name and the reference skeleton that will be used with this asset. */
		UE_API void SetSkeletalMeshSoftObjectPathName(const FSoftObjectPath& PathName);
		UE_DEPRECATED(5.7, "Use SetPhysicsAssetSoftObjectPathName instead")
		void SetPhysicsAssetPathName(const FString& PathName){}
		UE_DEPRECATED(5.7, "Use SetSkeletalMeshSoftObjectPathName instead")
		void SetSkeletalMeshPathName(const FString& PathName){}
		/** Set the reference bone name for this asset.*/
		UE_API void SetReferenceBoneName(const FName& BoneName);
		/** Set whether all attributes should be copied to the FChaosClothSimulationModel (else legacy behavior of skipping unused sets and face int maps will be followed).*/
		UE_API void SetCopyAllUserAttributesToSimModel(bool bValue);

		//~ Solver (max 1 per collection) Group
		/** Set the solver gravity */
		UE_API void SetSolverGravity(const FVector3f& SolverGravity);
		/** Set the solver air damping */
		UE_API void SetSolverAirDamping(const float SolverAirDamping);
		/** Set the solver time step */
		UE_API void SetSolverTimeStep(const float SolverTimeStep);
		/** Set the solver substeps */
		UE_API void SetSolverSubSteps(const int32 SolverSubSteps);

		//~ Pattern Sim Vertices 2D Group
		/** SetNumSimVertices2D per pattern within pattern facade. */
		UE_API TArrayView<FVector2f> GetSimPosition2D();
		UE_API TArrayView<int32> GetSimImportVertexID();

		//~ Pattern Sim Vertices 3D Group
		UE_API TArrayView<FVector3f> GetSimPosition3D();
		UE_API TArrayView<FVector3f> GetPreResizedSimPosition3D();
		UE_API TArrayView<FVector3f> GetSimNormal();
		UE_API TArrayView<TArray<int32>> GetSimBoneIndices();
		UE_API TArrayView<TArray<float>> GetSimBoneWeights();
		UE_API TArrayView<TArray<int32>> GetTetherKinematicIndex();
		UE_API TArrayView<TArray<float>> GetTetherReferenceLength();
		UE_API TArrayView<float> GetSimCustomResizingBlend();

		/** This will remove the 3D vertices, but the associated seams and 2D vertices will still exist, and point to INDEX_NONE */
		UE_API void RemoveSimVertices3D(int32 NumSimVertices);
		void RemoveAllSimVertices3D() { RemoveSimVertices3D(GetNumSimVertices3D()); }
		UE_API void RemoveSimVertices3D(const TArray<int32>& SortedDeletionList);
		/** Compact SimVertex2DLookup to remove any references to INDEX_NONE that may have been created by deleting 2D vertices. */
		UE_API void CompactSimVertex2DLookup();
		/** Compact SeamStitchLookup to remove any references to INDEX_NONE that may have been created by deleting stitches. */
		UE_API void CompactSeamStitchLookup();

		//~ Pattern Sim Faces Group
		/** SetNumSimFaces per pattern within pattern facade. */
		UE_API TArrayView<FIntVector3> GetSimIndices2D();
		UE_API TArrayView<FIntVector3> GetSimIndices3D();

		//~ Sim Patterns Group
		/** Set the new number of patterns to this cloth LOD. */
		UE_API void SetNumSimPatterns(int32 NumPatterns);
		/** Add a new pattern to this cloth LOD and return its index in the LOD pattern list. */
		UE_API int32 AddSimPattern();
		/** Return a pattern facade for the specified pattern index. */
		UE_API FCollectionClothSimPatternFacade GetSimPattern(int32 PatternIndex);
		/** Add a new pattern to this cloth LOD, and return the cloth pattern facade set to its index. */
		FCollectionClothSimPatternFacade AddGetSimPattern() { return GetSimPattern(AddSimPattern()); }
		/** Remove a sorted list of sim patterns. */
		UE_API void RemoveSimPatterns(const TArray<int32>& SortedDeletionList);

		//~ Sim Morph Targets Group
		/** Set the new number of sim morph targets to this cloth. */
		UE_API void SetNumSimMorphTargets(int32 NumMorphTargets, bool bCookedOnly = false);
		/** Add a sim morph target to this cloth and return its index in the sim morph target pattern list */
		UE_API int32 AddSimMorphTarget();
		/** Return a sim morph target facade for the specified morph target index. */
		UE_API FCollectionClothSimMorphTargetFacade GetSimMorphTarget(int32 MorphTargetIndex);
		/** Add a new sim morph target to this cloth, and return the morph target facade set to its index. */
		FCollectionClothSimMorphTargetFacade AddGetSimMorphTarget() { return GetSimMorphTarget(AddSimMorphTarget()); }
		/** Remove a sorted list of sim morph targets. */
		UE_API void RemoveSimMorphTargets(const TArray<int32>& SortedDeletionList);
		/** Remove all morph target vertices with invalid indices. This will also remove any morph targets that are empty. */
		UE_API void CompactSimMorphTargets();

		//~ Sim Morph Target Vertices Group
		/** Return a view of all sim morph target position deltas */
		UE_API TArrayView<FVector3f> GetSimMorphTargetPositionDelta();
		/** Return a view of all sim morph target tangent z deltas */
		UE_API TArrayView<FVector3f> GetSimMorphTargetTangentZDelta();
		/** Return a view of all sim morph target sim vertex 3d indices */
		UE_API TArrayView<int32> GetSimMorphTargetSimVertex3DIndex();
		/** Remove a sorted list of sim morph target vertices. */
		UE_API void RemoveSimMorphTargetVertices3D(const TArray<int32>& SortedDeletionList);

		//~ Sim Accessory Meshes Group
		/** Set the new number of accessory meshes to this cloth. */
		UE_API void SetNumSimAccessoryMeshes(int32 NumMeshes, bool bCookedOnly = false);
		/** Add a new accessory mesh to this cloth and return its index in the accessory mesh list. */
		UE_API int32 AddSimAccessoryMesh();
		/** Return an accessory mesh facade for the specified accessory mesh index. */
		UE_API FCollectionClothSimAccessoryMeshFacade GetSimAccessoryMesh(int32 MeshIndex);
		/** Add a new accessory mesh to this cloth and return the accessory mesh facade set to its index. */
		FCollectionClothSimAccessoryMeshFacade AddGetSimAccessoryMesh() { return GetSimAccessoryMesh(AddSimAccessoryMesh()); }
		/** Remove a sorted list of accessory meshes. */
		UE_API void RemoveSimAccessoryMeshes(const TArray<int32>& SortedDeletionList);
		UE_API TArrayView<FName> GetSimAccessoryMeshName();
		UE_API TArrayView<FName> GetSimAccessoryMeshPosition3DAttribute();
		UE_API TArrayView<FName> GetSimAccessoryMeshNormalAttribute();
		UE_API TArrayView<FName> GetSimAccessoryMeshBoneIndicesAttribute();
		UE_API TArrayView<FName> GetSimAccessoryMeshBoneWeightsAttribute();

		//~ Render Patterns Group
		/** Set the new number of patterns to this cloth LOD. */
		UE_API void SetNumRenderPatterns(int32 NumPatterns);
		/** Add a new pattern to this cloth LOD and return its index in the LOD pattern list. */
		UE_API int32 AddRenderPattern();
		/** Return a pattern facade for the specified pattern index. */
		UE_API FCollectionClothRenderPatternFacade GetRenderPattern(int32 PatternIndex);
		/** Add a new pattern to this cloth LOD, and return the cloth pattern facade set to its index. */
		FCollectionClothRenderPatternFacade AddGetRenderPattern() { return GetRenderPattern(AddRenderPattern()); }
		/** Remove a sorted list of render patterns. */
		UE_API void RemoveRenderPatterns(const TArray<int32>& SortedDeletionList);
		/** Return a view of all the render deformer number of influences used on this collection across all patterns. */
		UE_API TArrayView<int32> GetRenderDeformerNumInfluences();
		/** Return a view of all the render materials used on this collection across all patterns. */
		UE_API TArrayView<FSoftObjectPath> GetRenderMaterialSoftObjectPathName();
		UE_DEPRECATED(5.7, "Use GetRenderMaterialSoftObjectPathName instead")
		TArrayView<FString> GetRenderMaterialPathName() 
		{ 
			return TArrayView<FString>(); 
		}

		//~ Seam Group
		/** Set the new number of seams to this cloth. */
		UE_API void SetNumSeams(int32 NumSeams);
		/** Add a new seam to this cloth and return its index in the seam list. */
		UE_API int32 AddSeam();
		/** Return a seam facade for the specified seam index. */
		UE_API FCollectionClothSeamFacade GetSeam(int32 SeamIndex);
		/** Add a new seam to this cloth and return the seam facade set to its index. */
		FCollectionClothSeamFacade AddGetSeam() { return GetSeam(AddSeam()); }
		/** Remove a sorted list of seams. */
		UE_API void RemoveSeams(const TArray<int32>& SortedDeletionList);
		
		//~ Fabric Group
		/** Set the new number of fabrics to this cloth. */
		UE_API void SetNumFabrics(int32 NumFabrics);
		/** Add a new fabric to this cloth and return its index in the fabric list. */
		UE_API int32 AddFabric();
		/** Return a fabric facade for the specified fabric index. */
		UE_API FCollectionClothFabricFacade GetFabric(int32 FabricIndex);
		/** Add a new fabric to this cloth and return the fabric facade set to its index. */
		FCollectionClothFabricFacade AddGetFabric() { return GetFabric(AddFabric()); }
		/** Remove a sorted list of fabrics. */
		UE_API void RemoveFabrics(const TArray<int32>& SortedDeletionList);

		//~ Render Vertices Group
		/** SetNumRenderVertices per pattern within pattern facade. */
		UE_API TArrayView<FVector3f> GetRenderPosition();
		UE_API TArrayView<FVector3f> GetRenderNormal();
		UE_API TArrayView<FVector3f> GetRenderTangentU();
		UE_API TArrayView<FVector3f> GetRenderTangentV();
		UE_API TArrayView<TArray<FVector2f>> GetRenderUVs();
		UE_API TArrayView<FLinearColor> GetRenderColor();
		UE_API TArrayView<TArray<int32>> GetRenderBoneIndices();
		UE_API TArrayView<TArray<float>> GetRenderBoneWeights();
		UE_API TArrayView<TArray<FVector4f>> GetRenderDeformerPositionBaryCoordsAndDist();
		UE_API TArrayView<TArray<FVector4f>> GetRenderDeformerNormalBaryCoordsAndDist();
		UE_API TArrayView<TArray<FVector4f>> GetRenderDeformerTangentBaryCoordsAndDist();
		UE_API TArrayView<TArray<FIntVector3>> GetRenderDeformerSimIndices3D();
		UE_API TArrayView<TArray<float>> GetRenderDeformerWeight();
		UE_API TArrayView<float> GetRenderDeformerSkinningBlend();
		UE_API TArrayView<float> GetRenderCustomResizingBlend();

		//~ Render Faces Group
		/** SetNumRenderFaces per pattern within pattern facade. */
		UE_API TArrayView<FIntVector3> GetRenderIndices();

		//~ Custom Resizing Regions Group
		UE_API void SetNumCustomResizingRegions(int32 NumGroups);
		UE_API TArrayView<FString> GetCustomResizingRegionSet();
		UE_API TArrayView<int32> GetCustomResizingRegionType();

		//~ Weight Maps
		/** Add a new weight map to this cloth. Access is then done per pattern. */
		UE_API void AddWeightMap(const FName& Name);
		/** Remove a weight map from this cloth. */
		UE_API void RemoveWeightMap(const FName& Name);
		UE_API TArrayView<float> GetWeightMap(const FName& Name);

		//~ Other User-Defined Attributes (not instantiated for bools)
		/** GroupName must be an existing group as defined in ClothCollectionGroup. Returns success */
		template<typename T UE_REQUIRES(TIsUserAttributeType<T>::Value)>
		bool AddUserDefinedAttribute(const FName& Name, const FName& GroupName, const FName& GroupDependency = NAME_None);
		UE_API void RemoveUserDefinedAttribute(const FName& Name, const FName& GroupName);
		template<typename T UE_REQUIRES(TIsUserAttributeType<T>::Value)>
		TArrayView<T> GetUserDefinedAttribute(const FName& Name, const FName& GroupName);

	private:
		
		void SetDefaults();

		TSharedRef<class FClothCollection> GetClothCollection();

		friend class FCollectionClothSeamFacade;
		friend class FCollectionClothSimPatternFacade;
		friend class FCollectionClothFabricFacade;

		explicit FCollectionClothFacade(const TSharedRef<class FClothCollection>& InClothCollection);

		// These methods are private because they're managed by the FCollectionClothSeamFacade.
		//~ Sim Vertices 2D Group
		TArrayView<int32> GetSimVertex3DLookupPrivate();

		//~ Sim Vertices 3D Group
		TArrayView<TArray<int32>> GetSeamStitchLookupPrivate();
		TArrayView<TArray<int32>> GetSimVertex2DLookupPrivate();
	};
}  // End namespace UE::Chaos::ClothAsset

#undef UE_API
