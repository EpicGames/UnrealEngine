// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

#define UE_API CHAOSCLOTHASSET_API

struct FManagedArrayCollection;

namespace UE::Chaos::ClothAsset
{
	/**
	 * Cloth Asset collection render pattern facade class to access cloth render pattern data.
	 * Constructed from FCollectionClothConstFacade.
	 * Const access (read only) version.
	 */
	class FCollectionClothRenderPatternConstFacade
	{
	public:
		FCollectionClothRenderPatternConstFacade() = delete;

		FCollectionClothRenderPatternConstFacade(const FCollectionClothRenderPatternConstFacade&) = delete;
		FCollectionClothRenderPatternConstFacade& operator=(const FCollectionClothRenderPatternConstFacade&) = delete;

		FCollectionClothRenderPatternConstFacade(FCollectionClothRenderPatternConstFacade&&) = default;
		FCollectionClothRenderPatternConstFacade& operator=(FCollectionClothRenderPatternConstFacade&&) = default;

		virtual ~FCollectionClothRenderPatternConstFacade() = default;

		/** Return the render deformer number of influences for this pattern. */
		UE_API int32 GetRenderDeformerNumInfluences() const;
		/** Return the render material for this pattern. */
		UE_API const FSoftObjectPath& GetRenderMaterialSoftObjectPathName() const;
		UE_DEPRECATED(5.7, "Use GetRenderMaterialSoftObjectPathName instead")
		const FString& GetRenderMaterialPathName() const
		{
			static const FString EmptyString;
			return EmptyString;
		}

		//~ Render Vertices Group
		// Note: Use the FCollectionClothConstFacade accessors instead of these for the array indices to match the RenderIndices values
		/** Return the total number of render vertices for this pattern. */
		UE_API int32 GetNumRenderVertices() const;
		/** Return the render vertices offset for this pattern in the render vertices for the collection. */
		UE_API int32 GetRenderVerticesOffset() const;
		UE_API TConstArrayView<FVector3f> GetRenderPosition() const;
		UE_API TConstArrayView<FVector3f> GetRenderNormal() const;
		UE_API TConstArrayView<FVector3f> GetRenderTangentU() const;
		UE_API TConstArrayView<FVector3f> GetRenderTangentV() const;
		UE_API TConstArrayView<TArray<FVector2f>> GetRenderUVs() const;
		UE_API TConstArrayView<FLinearColor> GetRenderColor() const;
		UE_API TConstArrayView<TArray<int32>> GetRenderBoneIndices() const;
		UE_API TConstArrayView<TArray<float>> GetRenderBoneWeights() const;
		UE_API TConstArrayView<TArray<FVector4f>> GetRenderDeformerPositionBaryCoordsAndDist() const;
		UE_API TConstArrayView<TArray<FVector4f>> GetRenderDeformerNormalBaryCoordsAndDist() const;
		UE_API TConstArrayView<TArray<FVector4f>> GetRenderDeformerTangentBaryCoordsAndDist() const;
		UE_API TConstArrayView<TArray<FIntVector3>> GetRenderDeformerSimIndices3D() const;
		UE_API TConstArrayView<TArray<float>> GetRenderDeformerWeight() const;
		UE_API TConstArrayView<float> GetRenderDeformerSkinningBlend() const;
		UE_API TConstArrayView<float> GetRenderCustomResizingBlend() const;

		//~ Render Faces Group
		// Note: RenderIndices points to the collection arrays, not the pattern arrays
		UE_API int32 GetNumRenderFaces() const;
		/** Return the render faces offset for this pattern in the render faces */
		UE_API int32 GetRenderFacesOffset() const;
		UE_API TConstArrayView<FIntVector3> GetRenderIndices() const;

		UE_API bool IsEmpty() const;

		/** Return the Pattern index this facade has been created with. */
		int32 GetPatternIndex() const { return PatternIndex; }

	protected:
		friend class FCollectionClothRenderPatternFacade;  // For other instances access
		friend class FCollectionClothConstFacade;
		FCollectionClothRenderPatternConstFacade(const TSharedRef<const class FConstClothCollection>& InClothCollection, int32 InPatternIndex);

		static constexpr int32 GetBaseElementIndex() { return 0; }
		int32 GetElementIndex() const { return GetBaseElementIndex() + PatternIndex; }

		TSharedRef<const class FConstClothCollection> ClothCollection;
		int32 PatternIndex;
	};

	/**
	 * Cloth Asset collection render pattern facade class to access cloth render pattern data.
	 * Constructed from FCollectionClothFacade.
	 * Non-const access (read/write) version.
	 */
	class FCollectionClothRenderPatternFacade final : public FCollectionClothRenderPatternConstFacade
	{
	public:
		FCollectionClothRenderPatternFacade() = delete;

		FCollectionClothRenderPatternFacade(const FCollectionClothRenderPatternFacade&) = delete;
		FCollectionClothRenderPatternFacade& operator=(const FCollectionClothRenderPatternFacade&) = delete;

		FCollectionClothRenderPatternFacade(FCollectionClothRenderPatternFacade&&) = default;
		FCollectionClothRenderPatternFacade& operator=(FCollectionClothRenderPatternFacade&&) = default;

		virtual ~FCollectionClothRenderPatternFacade() override = default;

		/** Remove all geometry from this cloth pattern. */
		UE_API void Reset();

		/** Initialize from another render pattern. Assumes all indices match between source and target. */
		UE_API void Initialize(const FCollectionClothRenderPatternConstFacade& Other, int32 SimVertex3DOffset);

		/** Initialize from another render pattern. Assumes all indices match between source and target. */
		UE_DEPRECATED(5.4, "Use Initialize with the SimVertex3DOffset instead")
		void Initialize(const FCollectionClothRenderPatternConstFacade& Other) { Initialize(Other, 0); }

		/** Set the render deformer number of influences for this pattern. */
		UE_API void SetRenderDeformerNumInfluences(int32 NumInfluences);
		/** Set the render material for this pattern. */
		UE_API void SetRenderMaterialSoftObjectPathName(const FSoftObjectPath& PathName);
		UE_DEPRECATED(5.7, "Use SetRenderMaterialSoftObjectPathName instead")
		void SetRenderMaterialPathName(const FString& PathName){}

		//~ Render Vertices Group
		// Note: Use the FCollectionClothFacade accessors instead of these for the array indices to match the RenderIndices values
		/** Grow or shrink the space reserved for render vertices for this pattern within the cloth collection and return its start index. */
		UE_API void SetNumRenderVertices(int32 NumRenderVertices);
		UE_API void RemoveRenderVertices(const TArray<int32>& SortedDeletionList);
		UE_API TArrayView<FVector3f> GetRenderPosition();
		UE_API TArrayView<FVector3f> GetRenderNormal();
		UE_API TArrayView<FVector3f> GetRenderTangentU();
		UE_API TArrayView<FVector3f> GetRenderTangentV();
		UE_API TArrayView<TArray<FVector2f>> GetRenderUVs();
		UE_API TArrayView<FLinearColor> GetRenderColor();
		UE_API TArrayView<TArray<int32>> GetRenderBoneIndices();
		UE_API TArrayView<TArray<float>> GetRenderBoneWeights();
		UE_API TArrayView<TArray<FVector4f>> GetRenderDeformerPositionBaryCoordsAndDist();
		UE_API TArrayView<TArray<FVector4f>> GetRenderDeformerNormalBaryCoordsAndDist();
		UE_API TArrayView<TArray<FVector4f>> GetRenderDeformerTangentBaryCoordsAndDist();
		UE_API TArrayView<TArray<FIntVector3>> GetRenderDeformerSimIndices3D();
		UE_API TArrayView<TArray<float>> GetRenderDeformerWeight();
		UE_API TArrayView<float> GetRenderDeformerSkinningBlend();
		UE_API TArrayView<float> GetRenderCustomResizingBlend();

		//~ Render Faces Group
		/** Grow or shrink the space reserved for render faces for this pattern within the cloth collection and return its start index. */
		UE_API void SetNumRenderFaces(int32 NumRenderFaces);
		UE_API void RemoveRenderFaces(const TArray<int32>& SortedDeletionList);
		UE_API TArrayView<FIntVector3> GetRenderIndices();

	private:
		friend class FCollectionClothFacade;
		FCollectionClothRenderPatternFacade(const TSharedRef<class FClothCollection>& InClothCollection, int32 InPatternIndex);

		void SetDefaults();

		TSharedRef<class FClothCollection> GetClothCollection();
	};
}  // End namespace UE::Chaos::ClothAsset

#undef UE_API
