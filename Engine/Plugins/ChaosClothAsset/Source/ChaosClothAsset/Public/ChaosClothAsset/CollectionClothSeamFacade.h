// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

#define UE_API CHAOSCLOTHASSET_API

namespace UE::Chaos::ClothAsset
{
	/**
	 * Cloth Asset collection seam facade class to access cloth seam data.
	 * Constructed from FCollectionClothConstFacade.
	 * Const access (read only) version.
	 */
	class FCollectionClothSeamConstFacade
	{
	public:
		FCollectionClothSeamConstFacade() = delete;

		FCollectionClothSeamConstFacade(const FCollectionClothSeamConstFacade&) = delete;
		FCollectionClothSeamConstFacade& operator=(const FCollectionClothSeamConstFacade&) = delete;

		FCollectionClothSeamConstFacade(FCollectionClothSeamConstFacade&&) = default;
		FCollectionClothSeamConstFacade& operator=(FCollectionClothSeamConstFacade&&) = default;

		virtual ~FCollectionClothSeamConstFacade() = default;

		/** Return the total number of stitches for this seam. */
		UE_API int32 GetNumSeamStitches() const;

		/** Return the seam stitch offset for this seam in the seam stitches for the cloth. */
		UE_API int32 GetSeamStitchesOffset() const;

		//~ Seam Stitches Group
		// Indices correspond with the FCollectionClothConstFacade indices
		UE_API TConstArrayView<FIntVector2> GetSeamStitch2DEndIndices() const;
		UE_API TConstArrayView<int32> GetSeamStitch3DIndex() const;

		/** Return the seam index this facade has been created with. */
		int32 GetSeamIndex() const { return SeamIndex; }

		UE_API void ValidateSeam() const;

	protected:
		friend class FCollectionClothSeamFacade;  // For other instances access
		friend class FCollectionClothConstFacade;
		FCollectionClothSeamConstFacade(const TSharedRef<const class FConstClothCollection>& InClothCollection, int32 SeamIndex);

		static constexpr int32 GetBaseElementIndex() { return 0; }
		int32 GetElementIndex() const { return GetBaseElementIndex() + SeamIndex; }

		TSharedRef<const class FConstClothCollection> ClothCollection;
		int32 SeamIndex;
	};

	/**
	 * Cloth Asset collection seam facade class to access cloth seam data.
	 * Constructed from FCollectionClothFacade.
	 * Non-const access (read/write) version.
	 */
	class FCollectionClothSeamFacade final : public FCollectionClothSeamConstFacade
	{
	public:
		FCollectionClothSeamFacade() = delete;

		FCollectionClothSeamFacade(const FCollectionClothSeamFacade&) = delete;
		FCollectionClothSeamFacade& operator=(const FCollectionClothSeamFacade&) = delete;

		FCollectionClothSeamFacade(FCollectionClothSeamFacade&&) = default;
		FCollectionClothSeamFacade& operator=(FCollectionClothSeamFacade&&) = default;

		virtual ~FCollectionClothSeamFacade() override = default;

		/** Remove all stitches from this seam. */
		UE_API void Reset();

		/** Initialize from a list of stitches */
		UE_API void Initialize(TConstArrayView<FIntVector2> Stitches);

		/** Initialize from another seam. */
		UE_API void Initialize(const FCollectionClothSeamConstFacade& Other, const int32 SimVertex2DOffset, const int32 SimVertex3DOffset);

		/** Clean up references to invalid indices, including updating stitches to maintain topology. */
		UE_API void CleanupAndCompact();

	private:
		friend class FCollectionClothFacade;
		FCollectionClothSeamFacade(const TSharedRef<class FClothCollection>& InClothCollection, int32 SeamIndex);

		void SetDefaults();

		//~ Seam Stitches Group
		// Indices correspond with the FCollectionClothConstFacade indices (e.g., not pattern indices)
		void SetNumSeamStitches(const int32 NumStitches);
		TArrayView<FIntVector2> GetSeamStitch2DEndIndices();
		TArrayView<int32> GetSeamStitch3DIndex();

		TSharedRef<class FClothCollection> GetClothCollection();
	};
}  // End namespace UE::Chaos::ClothAsset

#undef UE_API
