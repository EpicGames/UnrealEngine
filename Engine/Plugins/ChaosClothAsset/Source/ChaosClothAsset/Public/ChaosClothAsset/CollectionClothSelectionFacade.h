// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Templates/SharedPointer.h"
#include "GeometryCollection/ManagedArray.h"

struct FManagedArrayCollection;

namespace UE::Chaos::ClothAsset
{
	/**
	 * Cloth Asset collection Selection facade class. Each Selection consists of a set of integer indices, a name, and a string representing the type of element being indexed.
	 * Const access (read only) version.
	 */
	class FCollectionClothSelectionConstFacade
	{
	public:
		CHAOSCLOTHASSET_API explicit FCollectionClothSelectionConstFacade(const TSharedRef<const FManagedArrayCollection>& ManagedArrayCollection);

		FCollectionClothSelectionConstFacade() = delete;
		FCollectionClothSelectionConstFacade(const FCollectionClothSelectionConstFacade&) = delete;
		FCollectionClothSelectionConstFacade& operator=(const FCollectionClothSelectionConstFacade&) = delete;
		FCollectionClothSelectionConstFacade(FCollectionClothSelectionConstFacade&&) = default;
		FCollectionClothSelectionConstFacade& operator=(FCollectionClothSelectionConstFacade&&) = default;
		virtual ~FCollectionClothSelectionConstFacade() = default;

		/** Return whether the facade is defined on the collection. */
		CHAOSCLOTHASSET_API bool IsValid() const;

		/** Return the number of selections in the collection. */
		CHAOSCLOTHASSET_API int32 GetNumSelections() const;

		/** Return an array of all the selections' name in the collection. */
		CHAOSCLOTHASSET_API TArray<FName> GetNames() const;

		/** Return whether a selection with the given name currently exists in the collection. */
		CHAOSCLOTHASSET_API bool HasSelection(const FName& Name) const;

		/** Return the selection group dependency. The selection must exist to call this function. */
		CHAOSCLOTHASSET_API FName GetSelectionGroup(const FName& Name) const;

		/** Get the selection set for the given selection name. The selection must exist to call this function. */
		CHAOSCLOTHASSET_API const TSet<int32>& GetSelectionSet(const FName& Name) const;

		/** Find a selection with the given name, or nullptr if no such selection exists. */
		CHAOSCLOTHASSET_API const TSet<int32>* FindSelectionSet(const FName& Name) const;


		/** Return whether a selection with the given name currently exists in the collection and has a secondary set associated with it. */
		UE_DEPRECATED(5.6, "Secondary selection sets are no longer supported.")
		CHAOSCLOTHASSET_API bool HasSelectionSecondarySet(const FName& Name) const;

		/** Return the secondary selection set group dependency. The selection must exist and have a secondary set to call this function. */
		UE_DEPRECATED(5.6, "Secondary selection sets are no longer supported.")
		CHAOSCLOTHASSET_API FName GetSelectionSecondaryGroup(const FName& Name) const;

		/** Get the secondary selection set for the given selection name. The selection must exist and have a secondary set to call this function. */
		UE_DEPRECATED(5.6, "Secondary selection sets are no longer supported.")
		CHAOSCLOTHASSET_API const TSet<int32>& GetSelectionSecondarySet(const FName& Name) const;

		/** Find the secondary selection set for the given selection name. Returns nullptr if no such selection exists or if it does not have an associated secondary set. */
		UE_DEPRECATED(5.6, "Secondary selection sets are no longer supported.")
		CHAOSCLOTHASSET_API const TSet<int32>* FindSelectionSecondarySet(const FName& Name) const;

	protected:
		TSharedRef<FManagedArrayCollection> ManagedArrayCollection;
	};

	/**
	 * Cloth Asset collection Selection facade class. Each Selection consists of a set of integer indices, a name, and a string representing the type of element being indexed.
	 * Non-const access (read/write) version.
	 */
	class FCollectionClothSelectionFacade final : public FCollectionClothSelectionConstFacade
	{
	public:
		CHAOSCLOTHASSET_API explicit FCollectionClothSelectionFacade(const TSharedRef<FManagedArrayCollection>& ManagedArrayCollection);

		UE_DEPRECATED(5.6, "Use the constructor that takes a non const ManagedArrayCollection instead.")
		CHAOSCLOTHASSET_API explicit FCollectionClothSelectionFacade(const TSharedRef<const FManagedArrayCollection>& ManagedArrayCollection);

		FCollectionClothSelectionFacade() = delete;
		FCollectionClothSelectionFacade(const FCollectionClothSelectionFacade&) = delete;
		FCollectionClothSelectionFacade& operator=(const FCollectionClothSelectionFacade&) = delete;
		FCollectionClothSelectionFacade(FCollectionClothSelectionFacade&&) = default;
		FCollectionClothSelectionFacade& operator=(FCollectionClothSelectionFacade&&) = default;
		virtual ~FCollectionClothSelectionFacade() override = default;

		/** Add the Selection attributes to the underlying collection. */
		CHAOSCLOTHASSET_API void DefineSchema();

		/** Get the selection set for the given selection name. The selection must exist to call this function. */
		CHAOSCLOTHASSET_API TSet<int32>& GetSelectionSet(const FName& Name);

		/** Find a selection with the given name, or nullptr if no such selection exists. */
		CHAOSCLOTHASSET_API TSet<int32>* FindSelectionSet(const FName& Name);

		/** Remove the selection with the given name if it exists. */
		CHAOSCLOTHASSET_API void RemoveSelectionSet(const FName& Name);

		/** Get the secondary selection set for the given selection name. The selection must exist and have a secondary set to call this function. */
		UE_DEPRECATED(5.6, "Secondary selection sets are no longer supported.")
		CHAOSCLOTHASSET_API TSet<int32>& GetSelectionSecondarySet(const FName& Name);

		/** Find the secondary selection set for the given selection name. Returns nullptr if no such selection exists or if it does not have an associated secondary set. */
		UE_DEPRECATED(5.6, "Secondary selection sets are no longer supported.")
		CHAOSCLOTHASSET_API TSet<int32>* FindSelectionSecondarySet(const FName& Name);

		/** Remove the secondary selection set from the selection with the given name if it exists. */
		UE_DEPRECATED(5.6, "Secondary selection sets are no longer supported.")
		CHAOSCLOTHASSET_API void RemoveSelectionSecondarySet(const FName& Name);

		/** 
		 * Append all sets from an existing collection to this collection.
		 * Matching sets (i.e., same name and type) will be unioned.
		 * Mismatching sets (same name, different type) will be handled according to bOverwriteExistingIfMismatched
		 * 
		 * @param bOverwriteExistingIfMismatched: If true, overwrite existing mismatched sets.
		 *   If false, keep the existing set.
		 */
		void Append(const FCollectionClothSelectionConstFacade& Other, bool bOverwriteExistingIfMismatched)
		{
			AppendWithOffsets(Other, bOverwriteExistingIfMismatched, TMap<FName, int32>());
		}

		/**
		 * Append all sets from an existing collection to this collection. Apply an offset to Other's data if its GroupName is in the GroupOffsets table.
		 * Matching sets (i.e., same name and type) will be unioned.
		 * Mismatching sets (same name, different type) will be handled according to bOverwriteExistingIfMismatched
		 *
		 * @param bOverwriteExistingIfMismatched: If true, overwrite existing mismatched sets.
		 *   If false, keep the existing set.
		 */
		CHAOSCLOTHASSET_API void AppendWithOffsets(const FCollectionClothSelectionConstFacade& Other, bool bOverwriteExistingIfMismatched, const TMap<FName, int32>& GroupOffsets);

		/**
		 * Find, or add if it doesn't already exist, a selection for the specified group with the given name.
		 * If the group doesn't already exists, this function will create it.
		 * If the selection already exists, but depends on a different group, the old selection will be deleted and a new one recreated with the new group dependency.
		 */
		CHAOSCLOTHASSET_API TSet<int32>& FindOrAddSelectionSet(const FName& Name, const FName& GroupName);

		/**
		 * Find, or add if it doesn't already exist, a secondary selection set for the specified group with the given name.
		 * NOTE: A selection with the given name must already exist. This function just adds a secondary set to the existing selection.
		 * If the group doesn't already exists, this function will create it.
		 * If the secondary selection set already exists, but depends on a different group, the old secondary selection set will be deleted and a new one recreated with the new group dependency.
		 */
		UE_DEPRECATED(5.6, "Secondary selection sets are no longer supported.")
		CHAOSCLOTHASSET_API TSet<int32>& FindOrAddSelectionSecondarySet(const FName& Name, const FName& GroupName);

	private:

		// Helper for FindOrAddSelectionSet and FindOrAddSelectionSecondarySet
		TSet<int32>& FindOrAddSelectionSetInternal(const FName& Name, const FName& GroupName, const FName& SelectionGroupName);

	};
}
