// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

#define UE_API CHAOSCLOTHASSET_API

struct FManagedArrayCollection;

namespace UE::Chaos::ClothAsset
{
	/**
	 * Cloth Asset collection sim accessory mesh facade class to access cloth sim accessory mesh data.
	 * Constructed from FCollectionClothConstFacade.
	 * Const access (read only) version.
	 */
	class FCollectionClothSimAccessoryMeshConstFacade
	{
	public:
		FCollectionClothSimAccessoryMeshConstFacade() = delete;

		FCollectionClothSimAccessoryMeshConstFacade(const FCollectionClothSimAccessoryMeshConstFacade&) = delete;
		FCollectionClothSimAccessoryMeshConstFacade& operator=(const FCollectionClothSimAccessoryMeshConstFacade&) = delete;

		FCollectionClothSimAccessoryMeshConstFacade(FCollectionClothSimAccessoryMeshConstFacade&&) = default;
		FCollectionClothSimAccessoryMeshConstFacade& operator=(FCollectionClothSimAccessoryMeshConstFacade&&) = default;

		virtual ~FCollectionClothSimAccessoryMeshConstFacade() = default;

		//~ Sim Accessory Meshes Group
		/** Return the name for this accessory mesh */
		UE_API FName GetSimAccessoryMeshName() const;
		/** Return the names of dynamic attributes associated with this accessory mesh */
		UE_API FName GetSimAccessoryMeshPosition3DAttribute() const;
		UE_API FName GetSimAccessoryMeshNormalAttribute() const;
		UE_API FName GetSimAccessoryMeshBoneIndicesAttribute() const;
		UE_API FName GetSimAccessoryMeshBoneWeightsAttribute() const;

		//~ Sim Vertices 3D Group
		UE_API TConstArrayView<FVector3f> GetSimAccessoryMeshPosition3D() const;
		UE_API TConstArrayView<FVector3f> GetSimAccessoryMeshNormal() const;
		UE_API TConstArrayView<TArray<int32>> GetSimAccessoryMeshBoneIndices() const;
		UE_API TConstArrayView<TArray<float>> GetSimAccessoryMeshBoneWeights() const;

		/** Return the accessory mesh index this facade has been created with. */
		int32 GetSimAccessoryMeshIndex() const { return MeshIndex; }

	protected:
		friend class FCollectionClothSimAccessoryMeshFacade;  // For other instances access
		friend class FCollectionClothConstFacade;
		FCollectionClothSimAccessoryMeshConstFacade(const TSharedRef<const class FConstClothCollection>& InClothCollection, int32 InMeshIndex);

		static constexpr int32 GetBaseElementIndex() { return 0; }
		int32 GetElementIndex() const { return GetBaseElementIndex() + MeshIndex; }

		TSharedRef<const class FConstClothCollection> ClothCollection;
		int32 MeshIndex;
	};

	/**
	 * Cloth Asset collection sim accessory mesh facade class to access cloth sim accessory mesh data.
	 * Constructed from FCollectionClothFacade.
	 * Non-const access (read/write) version.
	 */
	class FCollectionClothSimAccessoryMeshFacade final : public FCollectionClothSimAccessoryMeshConstFacade
	{
	public:
		FCollectionClothSimAccessoryMeshFacade() = delete;

		FCollectionClothSimAccessoryMeshFacade(const FCollectionClothSimAccessoryMeshFacade&) = delete;
		FCollectionClothSimAccessoryMeshFacade& operator=(const FCollectionClothSimAccessoryMeshFacade&) = delete;

		FCollectionClothSimAccessoryMeshFacade(FCollectionClothSimAccessoryMeshFacade&&) = default;
		FCollectionClothSimAccessoryMeshFacade& operator=(FCollectionClothSimAccessoryMeshFacade&&) = default;

		virtual ~FCollectionClothSimAccessoryMeshFacade() override = default;

		/** Remove all attributes from this accessory mesh. */
		UE_API void Reset();

		/** Set any default values for an accessory mesh. */
		void SetDefaults() {}

		/** Initialize from another accessory mesh. Assumes all indices match between source and target. Does not try to preserve attribute names, just values.*/
		UE_API void Initialize(const FCollectionClothSimAccessoryMeshConstFacade& Other);

		/** Initialize */
		UE_API void Initialize(const FName& Name, const TConstArrayView<FVector3f>& Positions, const TConstArrayView<FVector3f>& Normals, const TConstArrayView<TArray<int32>>& BoneIndices, const TConstArrayView<TArray<float>>& BoneWeights);

		//~ Sim Accessory Meshes Group
		/** Set the name for this accessory mesh. */
		UE_API void SetSimAccessoryMeshName(const FName& MeshName);

		//~ Sim Vertices 3D Group
		UE_API TArrayView<FVector3f> GetSimAccessoryMeshPosition3D();
		UE_API TArrayView<FVector3f> GetSimAccessoryMeshNormal();
		UE_API TArrayView<TArray<int32>> GetSimAccessoryMeshBoneIndices();
		UE_API TArrayView<TArray<float>> GetSimAccessoryMeshBoneWeights();

	private:
		friend class FCollectionClothFacade;
		FCollectionClothSimAccessoryMeshFacade(const TSharedRef<class FClothCollection>& InClothCollection, int32 InMorphTargetIndex);

		void ClearDynamicAttributes();
		void GenerateDynamicAttributesIfNecessary();

		TSharedRef<class FClothCollection> GetClothCollection();
	};
}  // End namespace UE::Chaos::ClothAsset

#undef UE_API
