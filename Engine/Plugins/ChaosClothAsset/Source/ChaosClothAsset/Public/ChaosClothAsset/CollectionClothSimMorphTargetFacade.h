// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

#define UE_API CHAOSCLOTHASSET_API

struct FManagedArrayCollection;

namespace UE::Chaos::ClothAsset
{
	/**
	 * Cloth Asset collection sim morph target facade class to access cloth sim morph target data.
	 * Constructed from FCollectionClothConstFacade.
	 * Const access (read only) version.
	 */
	class FCollectionClothSimMorphTargetConstFacade
	{
	public:
		FCollectionClothSimMorphTargetConstFacade() = delete;

		FCollectionClothSimMorphTargetConstFacade(const FCollectionClothSimMorphTargetConstFacade&) = delete;
		FCollectionClothSimMorphTargetConstFacade& operator=(const FCollectionClothSimMorphTargetConstFacade&) = delete;

		FCollectionClothSimMorphTargetConstFacade(FCollectionClothSimMorphTargetConstFacade&&) = default;
		FCollectionClothSimMorphTargetConstFacade& operator=(FCollectionClothSimMorphTargetConstFacade&&) = default;

		virtual ~FCollectionClothSimMorphTargetConstFacade() = default;

		/** Return the name for this morph target */
		UE_API const FString& GetSimMorphTargetName() const;

		//~ Sim Morph Target Vertices Group
		/** Return the total number of vertices for this morph target. */
		UE_API int32 GetNumSimMorphTargetVertices() const;
		/** Return the morph target vertices offset for this morph target in the morph target vertices for the collection. */
		UE_API int32 GetSimMorphTargetVerticesOffset() const;
		UE_API TConstArrayView<FVector3f> GetSimMorphTargetPositionDelta() const;
		UE_API TConstArrayView<FVector3f> GetSimMorphTargetTangentZDelta() const;
		UE_API TConstArrayView<int32> GetSimMorphTargetSimVertex3DIndex() const;

		UE_API bool IsEmpty() const;

		/** Return the Morph Target index this facade has been created with. */
		int32 GetSimMorphTargetIndex() const { return MorphTargetIndex; }

	protected:
		friend class FCollectionClothSimMorphTargetFacade;  // For other instances access
		friend class FCollectionClothConstFacade;
		FCollectionClothSimMorphTargetConstFacade(const TSharedRef<const class FConstClothCollection>& InClothCollection, int32 InPatternIndex);

		static constexpr int32 GetBaseElementIndex() { return 0; }
		int32 GetElementIndex() const { return GetBaseElementIndex() + MorphTargetIndex; }

		TSharedRef<const class FConstClothCollection> ClothCollection;
		int32 MorphTargetIndex;
	};

	/**
	 * Cloth Asset collection sim morph target facade class to access cloth sim morph target data.
	 * Constructed from FCollectionClothFacade.
	 * Non-const access (read/write) version.
	 */
	class FCollectionClothSimMorphTargetFacade final : public FCollectionClothSimMorphTargetConstFacade
	{
	public:
		FCollectionClothSimMorphTargetFacade() = delete;

		FCollectionClothSimMorphTargetFacade(const FCollectionClothSimMorphTargetFacade&) = delete;
		FCollectionClothSimMorphTargetFacade& operator=(const FCollectionClothSimMorphTargetFacade&) = delete;

		FCollectionClothSimMorphTargetFacade(FCollectionClothSimMorphTargetFacade&&) = default;
		FCollectionClothSimMorphTargetFacade& operator=(FCollectionClothSimMorphTargetFacade&&) = default;

		virtual ~FCollectionClothSimMorphTargetFacade() override = default;

		/** Remove all geometry from this morph target. */
		UE_API void Reset();

		/** Initialize from another morph target. Assumes all indices match between source and target. */
		UE_API void Initialize(const FCollectionClothSimMorphTargetConstFacade& Other, int32 SimVertex3DOffset);

		/** Initialize */
		UE_API void Initialize(const FString& Name, const TConstArrayView<FVector3f>& PositionDeltas, const TConstArrayView<FVector3f>& TangentZDeltas, const TConstArrayView<int32>& SimVertex3DIndices);

		/** Set the name for this morph target. */
		UE_API void SetSimMorphTargetName(const FString& MorphTargetName);

		//~ Sim Morph Target Vertices Group
		/** Grow or shrink the space reserved for morph target vertices for this morph target within the cloth collection and return its start index. */
		UE_API void SetNumSimMorphTargetVertices(int32 NumMorphTargetVertices);
		UE_API void RemoveSimMorphTargetVertices(const TArray<int32>& SortedDeletionList);
		UE_API TArrayView<FVector3f> GetSimMorphTargetPositionDelta();
		UE_API TArrayView<FVector3f> GetSimMorphTargetTangentZDelta();
		UE_API TArrayView<int32> GetSimMorphTargetSimVertex3DIndex();

		/** Remove all morph target vertices with invalid indices. */
		UE_API void Compact();

	private:
		friend class FCollectionClothFacade;
		FCollectionClothSimMorphTargetFacade(const TSharedRef<class FClothCollection>& InClothCollection, int32 InMorphTargetIndex);

		void SetDefaults();

		TSharedRef<class FClothCollection> GetClothCollection();
	};
}  // End namespace UE::Chaos::ClothAsset

#undef UE_API
