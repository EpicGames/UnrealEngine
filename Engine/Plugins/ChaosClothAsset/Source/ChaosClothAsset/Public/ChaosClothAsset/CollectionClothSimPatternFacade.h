// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

#define UE_API CHAOSCLOTHASSET_API

namespace UE::Chaos::ClothAsset
{
	/** Integral and vector types valid as index type in initializations. */
	template<typename T> struct TIsIndexType { static constexpr bool Value = false; };
	template<> struct TIsIndexType<int32> { static constexpr bool Value = true; };
	template<> struct TIsIndexType<uint32> { static constexpr bool Value = true; };
	template<> struct TIsIndexType<FIntVector3> { static constexpr bool Value = true; };

	/**
	 * Cloth Asset collection sim pattern facade class to access cloth sim pattern data.
	 * Constructed from FCollectionClothConstFacade.
	 * Const access (read only) version.
	 */
	class FCollectionClothSimPatternConstFacade
	{
	public:
		FCollectionClothSimPatternConstFacade() = delete;

		FCollectionClothSimPatternConstFacade(const FCollectionClothSimPatternConstFacade&) = delete;
		FCollectionClothSimPatternConstFacade& operator=(const FCollectionClothSimPatternConstFacade&) = delete;

		FCollectionClothSimPatternConstFacade(FCollectionClothSimPatternConstFacade&&) = default;
		FCollectionClothSimPatternConstFacade& operator=(FCollectionClothSimPatternConstFacade&&) = default;

		virtual ~FCollectionClothSimPatternConstFacade() = default;

		//~ Sim Vertices 2D Group
		// Note: Use the FCollectionClothConstFacade accessors instead of these for the array indices to match the SimIndices2D values
		/** Return the total number of simulation vertices for this pattern. */
		UE_API int32 GetNumSimVertices2D() const;
		/** Return the simulation vertices offset for this pattern in the simulation vertices for the collection. */
		UE_API int32 GetSimVertices2DOffset() const;
		UE_API TConstArrayView<FVector2f> GetSimPosition2D() const;
		UE_API TConstArrayView<int32> GetSimVertex3DLookup() const;
		UE_API TConstArrayView<int32> GetSimImportVertexID() const;

		//~ Sim Faces Group
		// Note: SimIndices points to the collection arrays, not the pattern arrays
		UE_API int32 GetNumSimFaces() const;
		/** Return the simulation faces offset for this pattern in the simulation faces. */
		UE_API int32 GetSimFacesOffset() const;
		UE_API TConstArrayView<FIntVector3> GetSimIndices2D() const;
		UE_API TConstArrayView<FIntVector3> GetSimIndices3D() const;

		/** Whether or not this pattern is empty. */
		UE_API bool IsEmpty() const;

		/** Return the Pattern index this facade has been created with. */
		int32 GetPatternIndex() const { return PatternIndex; }
		
		/** Return the Pattern index this facade has been created with. */
        UE_API int32 GetFabricIndex() const;

	protected:
		friend class FCollectionClothSimPatternFacade;  // For other instances access
		friend class FCollectionClothConstFacade;
		FCollectionClothSimPatternConstFacade(const TSharedRef<const class FConstClothCollection>& InClothCollection, int32 InPatternIndex);

		static constexpr int32 GetBaseElementIndex() { return 0; }
		int32 GetElementIndex() const { return GetBaseElementIndex() + PatternIndex; }

		TSharedRef<const class FConstClothCollection> ClothCollection;
		int32 PatternIndex;
	};

	/**
	 * Cloth Asset collection sim pattern facade class to access cloth sim pattern data.
	 * Constructed from FCollectionClothFacade.
	 * Non-const access (read/write) version.
	 */
	class FCollectionClothSimPatternFacade final : public FCollectionClothSimPatternConstFacade
	{
	public:
		FCollectionClothSimPatternFacade() = delete;

		FCollectionClothSimPatternFacade(const FCollectionClothSimPatternFacade&) = delete;
		FCollectionClothSimPatternFacade& operator=(const FCollectionClothSimPatternFacade&) = delete;

		FCollectionClothSimPatternFacade(FCollectionClothSimPatternFacade&&) = default;
		FCollectionClothSimPatternFacade& operator=(FCollectionClothSimPatternFacade&&) = default;

		virtual ~FCollectionClothSimPatternFacade() override = default;

		/** Remove all geometry from this cloth pattern. */
		UE_API void Reset();

		/** Initialize the cloth pattern using the specified 3D and 2D positions, and topology. If Normals are not supplied, they will be calculated from the face normals.*/
		template<typename IndexType UE_REQUIRES(TIsIndexType<IndexType>::Value)>
		void Initialize(const TArray<FVector2f>& Positions2D, const TArray<FVector3f>& Positions3D, const TArray<IndexType>& Indices, const int32 FabricIndex = INDEX_NONE, const TArray<FVector3f>& Normals = TArray<FVector3f>());

		/** Initialize this pattern using another pattern collection. */
		UE_API void Initialize(const FCollectionClothSimPatternConstFacade& Other, const int32 SimVertex3DOffset, const int32 FabricsOffset = 0);

		//~ Sim Vertices 2D Group
		// Note: Use the FCollectionClothConstFacade accessors instead of these for the array indices to match the SimIndices2D values
		UE_API TArrayView<FVector2f> GetSimPosition2D();
		UE_API TArrayView<int32> GetSimImportVertexID();

		/** This will remove the 2D vertices, but the associated seams and 3D vertices will still exist, and point to INDEX_NONE */
		UE_API void RemoveSimVertices2D(int32 NumSimVertices);
		void RemoveAllSimVertices2D() { RemoveSimVertices2D(GetNumSimVertices2D()); }
		UE_API void RemoveSimVertices2D(const TArray<int32>& SortedDeletionList);

		//~ Sim Faces Group
		// Note: SimIndices points to the collection arrays, not the pattern arrays
		/** Grow or shrink the space reserved for simulation faces for this pattern within the cloth collection and return its start index. */
		UE_API void SetNumSimFaces(int32 NumSimFaces);
		UE_API TArrayView<FIntVector3> GetSimIndices2D();
		UE_API TArrayView<FIntVector3> GetSimIndices3D();
		UE_API void RemoveSimFaces(const TArray<int32>& SortedDeletionList);

		/** Set the fabric index used by this pattern */
		UE_API void SetFabricIndex(const int32 InFabricIndex);

	private:
		friend class FCollectionClothFacade;
		FCollectionClothSimPatternFacade(const TSharedRef<class FClothCollection>& InClothCollection, int32 InPatternIndex);

		void SetDefaults();

		/** Append this many 2D and 3D SimVertices which correspond to each other. Returns {2D start index (local pattern index), 3D start index}. */
		FIntVector2 AppendSimVertices(int32 NumSimVertices);

		void SetNumSimVertices2D(int32 NumSimVertices);
		TArrayView<int32> GetSimVertex3DLookup();

		TSharedRef<class FClothCollection> GetClothCollection();
	};
}  // End namespace UE::Chaos::ClothAsset

#undef UE_API
