// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Vector.h"

namespace UE::Chaos::ClothAsset
{
	/** User defined attribute types. */
	template<typename T> struct TIsUserAttributeType { static constexpr bool Value = false; };
	template<> struct TIsUserAttributeType<bool> { static constexpr bool Value = true; };
	template<> struct TIsUserAttributeType<int32> { static constexpr bool Value = true; };
	template<> struct TIsUserAttributeType<float> { static constexpr bool Value = true; };
	template<> struct TIsUserAttributeType<FVector3f> { static constexpr bool Value = true; };
	template<> struct TIsUserAttributeType<TArray<int32>> { static constexpr bool Value = true; };
}  // End namespace UE::Chaos::ClothAsset
