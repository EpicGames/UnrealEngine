// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ChaosClothAssetEngine : ModuleRules
{
	public ChaosClothAssetEngine(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"DataflowCore",
				"DataflowEngine",
				"DataflowSimulation",
				"ClothingSystemRuntimeCommon"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"Engine",
				"CoreUObject",
				"RenderCore",
				"RHI",
				"ChaosClothAsset",
				"ChaosCloth",
				"ChaosCaching",
				"ClothingSystemRuntimeInterface"
			}
		);
		SetupModulePhysicsSupport(Target);

		if (Target.bBuildEditor || Target.bCompileAgainstEditor)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"PropertyEditor",   // For adding the Cloth Component "Cloth Sim" section to the Details panel UI
					"Slate",  // For Slate notifications
					"StructUtilsEditor"
				}
			);
		}

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"DerivedDataCache",
		});
	}
}
