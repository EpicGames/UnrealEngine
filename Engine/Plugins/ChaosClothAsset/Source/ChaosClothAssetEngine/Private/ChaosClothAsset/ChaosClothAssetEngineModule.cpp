// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ClothAssetPrivate.h"
#include "ChaosClothAsset/ClothAssetSKMClothingSimulation.h"
#include "ChaosClothAsset/ClothComponentCacheAdapter.h"
#include "Features/IModularFeatures.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#if WITH_EDITOR
#include "ChaosClothAsset/ClothComponent.h"
#include "ChaosClothAsset/ClothAssetInteractorDataAsset.h"
#include "PropertyBagDetails.h"
#include "PropertyEditorModule.h"
#endif

DEFINE_LOG_CATEGORY(LogChaosClothAsset);

#define LOCTEXT_NAMESPACE "ClothAssetEngineModule"

namespace UE::Chaos::ClothAsset
{
	class FClothAssetEngineModule
		: public IModuleInterface
		, public IClothingSimulationFactoryClassProvider
	{
	public:
		//~ Begin IModuleInterface
		virtual void StartupModule() override
		{
			ClothComponentCacheAdapter = MakeUnique<FClothComponentCacheAdapter>();
			RegisterAdapter(ClothComponentCacheAdapter.Get());

#if WITH_EDITOR
			// Add the Cloth Component "Cloth Sim" section to the Details panel UI
			FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
			const TSharedPtr<FPropertySection> Section = PropertyModule.FindOrCreateSection(UChaosClothComponent::StaticClass()->GetFName(), TEXT("ClothSim"), LOCTEXT("ClothSim", "Cloth Sim"));
			Section->AddCategory("Cloth Component");

			PropertyModule.RegisterCustomPropertyTypeLayout(FClothAssetInteractorPropertyBag::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FPropertyBagDetails::MakeInstance));

			PropertyModule.NotifyCustomizationModuleChanged();
#endif
			IModularFeatures::Get().RegisterModularFeature(IClothingSimulationFactoryClassProvider::FeatureName, static_cast<IClothingSimulationFactoryClassProvider*>(this));
		}

		virtual void ShutdownModule() override
		{
			IModularFeatures::Get().UnregisterModularFeature(IClothingSimulationFactoryClassProvider::FeatureName, static_cast<IClothingSimulationFactoryClassProvider*>(this));
#if WITH_EDITOR
			if (UObjectInitialized())
			{
				// Remove sections
				FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
				PropertyModule.RemoveSection(UChaosClothComponent::StaticClass()->GetFName(), TEXT("ClothSim"));

				PropertyModule.UnregisterCustomPropertyTypeLayout(FClothAssetInteractorPropertyBag::StaticStruct()->GetFName());
			}
#endif
		}
		//~ End IModuleInterface

		//~ Begin IClothingSimulationFactoryClassProvider
		virtual TSubclassOf<class UClothingSimulationFactory> GetClothingSimulationFactoryClass() const override
		{
			return UChaosClothAssetSKMClothingSimulationFactory::StaticClass();
		}
		//~ End IClothingSimulationFactoryClassProvider

	private:
		TUniquePtr<FClothComponentCacheAdapter> ClothComponentCacheAdapter;
	};
}
IMPLEMENT_MODULE(UE::Chaos::ClothAsset::FClothAssetEngineModule, ChaosClothAssetEngine);

#undef LOCTEXT_NAMESPACE
