// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ClothAsset.h"
#include "ChaosClothAsset/ClothAssetBuilder.h"
#include "ChaosClothAsset/ClothAssetPrivate.h"
#include "ChaosClothAsset/ClothCollectionGroup.h"
#include "ChaosClothAsset/ClothEngineTools.h"
#include "ChaosClothAsset/ClothGeometryTools.h"
#include "ChaosClothAsset/ClothSimulationModel.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "ChaosClothAsset/CollectionClothSelectionFacade.h"
#include "Chaos/CollectionEmbeddedSpringConstraintFacade.h"
#include "Chaos/CollectionPropertyFacade.h"
#include "Animation/Skeleton.h"
#if WITH_EDITORONLY_DATA
#include "Animation/AnimationAsset.h"
#endif
#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAssetAsyncCompileUtils.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "UObject/Package.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "EngineUtils.h"
#if WITH_EDITOR
#include "DerivedDataCacheInterface.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClothAsset)

// If Chaos cloth asset derived data needs to be rebuilt (new format, serialization differences, etc.) replace the version GUID below with a new one. 
// In case of merge conflicts with DDC versions, you MUST generate a new GUID and set this new GUID as the version.
#define CHAOS_CLOTH_ASSET_DERIVED_DATA_VERSION TEXT("479D81081F3A4A22B3C22ED4B278680E")

#define LOCTEXT_NAMESPACE "ChaosClothAsset"

namespace UE::Chaos::ClothAsset::Private
{
	static bool HasValidSkinweights(const TConstArrayView<TArray<int32>> BoneIndices, TConstArrayView<TArray<float>> BoneWeights, const FReferenceSkeleton* RefSkeleton)
	{
		if (!RefSkeleton)
		{
			return false;
		}

		check(BoneIndices.Num() == BoneWeights.Num());
		for (int32 Index = 0; Index < BoneIndices.Num(); ++Index)
		{
			if (!BoneIndices[Index].Num() || !BoneWeights[Index].Num() || BoneIndices[Index].Num() != BoneWeights[Index].Num())
			{
				return false;
			}
			for (const int32 BoneIndex : BoneIndices[Index])
			{
				if (!RefSkeleton->IsValidIndex(BoneIndex))
				{
					return false;
				}
			}

		}
		return true;
	}

	static ::Chaos::FChaosArchive& Serialize(::Chaos::FChaosArchive& Ar, TArray<TSharedRef<const FManagedArrayCollection>>& ClothCollections)
	{
		Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);

		if (Ar.IsLoading() && Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::ClothCollectionSingleLodSchema)
		{
			// Cloth assets before this version had a single ClothCollection with a completely different schema.
			ClothCollections.Empty(1);
			TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>();
			ClothCollection->Serialize(Ar);

			// Now we're just going to hard reset and define a new schema.
			ClothCollection->Reset();
			FCollectionClothFacade ClothFacade(ClothCollection);
			ClothFacade.DefineSchema();

			ClothCollections.Emplace(MoveTemp(ClothCollection));

			return Ar;
		}
		else
		{
			// This is following Serialize for Arrays
			ClothCollections.CountBytes(Ar);
			int32 NumClothCollections = Ar.IsLoading() ? 0 : ClothCollections.Num();
			Ar << NumClothCollections;
			if (NumClothCollections == 0)
			{
				// if we are loading, then we have to reset the size to 0, in case it isn't currently 0
				if (Ar.IsLoading())
				{
					ClothCollections.Empty();
				}
				return Ar;
			}
			check(NumClothCollections >= 0);

			if (Ar.IsError() || NumClothCollections < 0)
			{
				Ar.SetError();
				return Ar;
			}
			if (Ar.IsLoading())
			{
				// Required for resetting ArrayNum
				ClothCollections.Empty(NumClothCollections);

				for (int32 Index = 0; Index < NumClothCollections; ++Index)
				{
					TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>();
					ClothCollection->Serialize(Ar);

					// Property Facade may need to upgrade
					::Chaos::Softs::FCollectionPropertyMutableFacade PropertyFacade(ClothCollection);
					PropertyFacade.PostSerialize(Ar);

					// Cloth Facade may need to upgrade
					FCollectionClothFacade ClothFacade(ClothCollection);
					ClothFacade.PostSerialize(Ar);

					ClothCollections.Emplace(MoveTemp(ClothCollection));
				}
			}
			else
			{
				check(NumClothCollections == ClothCollections.Num());
#if WITH_EDITORONLY_DATA
				if (Ar.IsCooking() && Ar.IsSaving())
				{
					const bool bLog = !Ar.IsObjectReferenceCollector();  // Suppress logs during the package harvester pass, the cooker calls Serialize again on the linker save pass
					const FString AssetPath = FClothEngineTools::GetCookedAssetPath(Ar);
					for (int32 Index = 0; Index < NumClothCollections; ++Index)
					{
						const TSharedRef<const FManagedArrayCollection> Trimmed =
							FClothEngineTools::TrimClothCollectionOnCook(
								FString::Printf(TEXT("%s:[%d]"), *AssetPath, Index),
								ClothCollections[Index],
								bLog);
						ConstCastSharedRef<FManagedArrayCollection>(Trimmed)->Serialize(Ar);
					}
				}
				else
#endif
				{
					for (int32 Index = 0; Index < NumClothCollections; ++Index)
					{
						ConstCastSharedRef<FManagedArrayCollection>(ClothCollections[Index])->Serialize(Ar);
					}
				}
			}

			return Ar;
		}
	}
}

UChaosClothAsset::UChaosClothAsset(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	DataflowInstance.SetDataflowTerminal(TEXT("ClothAssetTerminal"));
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Setup a single LOD's Cloth Collection
	TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>();
	UE::Chaos::ClothAsset::FCollectionClothFacade ClothFacade(ClothCollection);
	ClothFacade.DefineSchema();
	GetClothCollectionsInternal().Emplace(MoveTemp(ClothCollection));
}

UChaosClothAsset::UChaosClothAsset(FVTableHelper& Helper)
	: Super(Helper)
{
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
UChaosClothAsset::~UChaosClothAsset() = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UChaosClothAsset::Serialize(FArchive& Ar)
{
	using namespace UE::Chaos::ClothAsset;

	LLM_SCOPE_BYNAME(TEXT("Physics/Cloth"));
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);

	bool bCooked = Ar.IsCooking();
	Ar << bCooked;

	{
		Chaos::FChaosArchive ChaosArchive(Ar);
		Private::Serialize(ChaosArchive, GetClothCollectionsInternal());
	}

	if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::AddClothAssetBase)
	{
		Ar << GetRefSkeleton(); // Moved to Cloth Asset Base serialization
	}

	if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::ClothAssetSkinweightsValidation)
	{
		// Fix the skeleton mesh binding, which can cause crashes in the render code, or make the sim mesh disappear when missing
		for (TSharedRef<const FManagedArrayCollection>& ClothCollection : GetClothCollectionsInternal())
		{
			const FCollectionClothConstFacade ClothConstFacade(ClothCollection);
			if (ClothConstFacade.IsValid())
			{
				const bool bHasValidSimSkinweights = Private::HasValidSkinweights(ClothConstFacade.GetSimBoneIndices(), ClothConstFacade.GetSimBoneWeights(), &GetRefSkeleton());
				const bool bHasValidRenderSkinweights = Private::HasValidSkinweights(ClothConstFacade.GetRenderBoneIndices(), ClothConstFacade.GetRenderBoneWeights(), &GetRefSkeleton());
				if (!bHasValidSimSkinweights || !bHasValidRenderSkinweights)
				{
					TSharedRef<FManagedArrayCollection> NewClothCollection = MakeShared<FManagedArrayCollection>(*ClothCollection);
					FClothGeometryTools::BindMeshToRootBone(NewClothCollection, !bHasValidSimSkinweights, !bHasValidRenderSkinweights);
					ClothCollection = MoveTemp(NewClothCollection);

					UE_CLOGF(!bHasValidSimSkinweights, LogChaosClothAsset, Warning, "%ls had invalid simulation mesh skin weights. This asset must be resaved.", *GetFullName());
					UE_CLOGF(!bHasValidRenderSkinweights, LogChaosClothAsset, Warning, "%ls had invalid render mesh skin weights. This asset must be resaved.", *GetFullName());
				}
			}
		}
	}

	if (bCooked && !IsTemplate() && !Ar.IsCountingMemory())  // Counting of these resources are done in GetResourceSizeEx, so skip these when counting memory
	{
		{
			LLM_SCOPE_BYNAME(TEXT("Physics/ClothRendering"));
			if (Ar.IsLoading())
			{
				SetResourceForRendering(MakeUnique<FSkeletalMeshRenderData>());
				GetResourceForRendering()->Serialize(Ar, this);
#if WITH_EDITOR
				EstablishCookedHeadKey();
#else
				UE_LOGF(LogChaosClothAsset, Display, "[%ls] Serialize: cooked load.", *GetName());
#endif
			}
			else if (Ar.IsSaving())
			{
#if WITH_EDITOR
				if (FSkeletalMeshRenderData* const LocalRenderData = GetSerializeRenderData(Ar))
				{
					LocalRenderData->Serialize(Ar, this);
				}
				else
				{
					return;  // SetError already called inside GetSerializeRenderData
				}
#else
				GetResourceForRendering()->Serialize(Ar, this);
#endif
			}
			else
			{
				GetResourceForRendering()->Serialize(Ar, this);
			}
		}

		if (!ClothSimulationModel.IsValid())
		{
			ClothSimulationModel = MakeShared<FChaosClothSimulationModel>();
		}
		UScriptStruct* const Struct = FChaosClothSimulationModel::StaticStruct();
		Struct->SerializeTaggedProperties(Ar, (uint8*)ClothSimulationModel.Get(), Struct, nullptr);
	}
}

void UChaosClothAsset::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	if (DataflowAsset_DEPRECATED != nullptr)
	{
		SetDataflow(DataflowAsset_DEPRECATED);
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		DataflowInstance.SetDataflowTerminal(FName(DataflowTerminal_DEPRECATED));
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		DataflowAsset_DEPRECATED = nullptr;
		DataflowTerminal_DEPRECATED.Empty();
	}
#endif  // WITH_EDITORONLY_DATA
}

#if WITH_EDITOR
void UChaosClothAsset::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UChaosClothAsset, PhysicsAsset))
	{
		OnAssetChanged();
	}
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UChaosClothAsset::ExecuteBuildInternal(FSkinnedAssetBuildContext& Context)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UChaosClothAsset::ExecuteBuildInternal);

	// This scope allows us to use any locked properties without causing stalls
	FSkinnedAssetAsyncBuildScope AsyncBuildScope(this);

	// rebuild render data from imported model
	CacheDerivedData(&Context);

	// Build the material channel data used by the texture streamer
	UpdateUVChannelData(true);
}

void UChaosClothAsset::BeginBuildInternal(FSkinnedAssetBuildContext& Context)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UChaosClothAsset::BeginBuildInternal);

	SetInternalFlags(EInternalObjectFlags::Async);

	// Unregister all instances of this component
	Context.RecreateRenderStateContext = MakeUnique<FSkinnedMeshComponentRecreateRenderStateContext>(this, false);

	// Release the render data resources.
	ReleaseResources();

	// Flush the resource release commands to the rendering thread to ensure that the build doesn't occur while a resource is still
	// allocated, and potentially accessing the UChaosClothAsset.
	ReleaseResourcesFence.Wait();

	// Lock all properties that should not be modified/accessed during async post-load
	USkinnedAsset::AcquireAsyncProperty();
}

void UChaosClothAsset::FinishBuildInternal(FSkinnedAssetBuildContext& Context)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UChaosClothAsset::FinishBuildInternal);

	ClearInternalFlags(EInternalObjectFlags::Async);

	USkinnedAsset::ReleaseAsyncProperty();
}
#endif // #if WITH_EDITOR

#if WITH_EDITOR
void UChaosClothAsset::BeginPostLoadAssetImpl(FSkinnedAssetPostLoadContext& Context)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UChaosClothAsset::BeginPostLoadAssetImpl);

	using namespace UE::Chaos::ClothAsset;

	// Make sure that there is at least one valid collection
	if (GetClothCollectionsInternal().IsEmpty())
	{
		UE_LOGF(LogChaosClothAsset, Warning, "Invalid Cloth Collection (no LODs) found while loading Cloth Asset %ls.", *GetFullName());
		TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>();
		FCollectionClothFacade ClothFacade(ClothCollection);
		ClothFacade.DefineSchema();
		GetClothCollectionsInternal().Emplace(MoveTemp(ClothCollection));
	}

	// Check that all LODs have the cloth schema
	const int32 NumLods = GetClothCollectionsInternal().Num();
	check(NumLods >= 1);  // The default LOD 0 should be present now if it ever was missing
	for (int32 LODIndex = 0; LODIndex < NumLods; ++LODIndex)
	{
		const TSharedRef<const FManagedArrayCollection>& ClothCollection = GetClothCollectionsInternal()[LODIndex];

		const FCollectionClothConstFacade ClothConstFacade(ClothCollection);
		if (!ClothConstFacade.IsValid())
		{
			UE_LOGF(LogChaosClothAsset, Warning, "Invalid Cloth Collection found at LOD %i while loading Cloth Asset %ls.", LODIndex, *GetFullName());
			TSharedRef<FManagedArrayCollection> NewClothCollection = MakeShared<FManagedArrayCollection>();
			FCollectionClothFacade NewClothFacade = FCollectionClothFacade(NewClothCollection);
			NewClothFacade.DefineSchema();
			GetClothCollectionsInternal()[LODIndex] = MoveTemp(NewClothCollection);
		}
	}

	// We're done touching the ClothCollections, so can unlock for read
	ReleaseAsyncProperty(EClothAssetAsyncProperties::ClothCollection, ESkinnedAssetAsyncPropertyLockType::WriteOnly);

	// Build the cloth simulation model
	BuildClothSimulationModel();  // TODO: Cache ClothSimulationModel in the DDC

	// Convert PerPlatForm data to PerQuality if perQuality data have not been serialized.
	// Also test default value, since PerPlatformData can have Default !=0 and no PerPlatform data overrides.
	const bool bConvertMinLODData = (MinQualityLevelLOD.PerQuality.Num() == 0 && MinQualityLevelLOD.Default == 0) && (MinLod.PerPlatform.Num() != 0 || MinLod.Default != 0);
	if (IsMinLodQualityLevelEnable() && bConvertMinLODData)
	{
		constexpr bool bRequireAllPlatformsKnownTrue = true;
		MinQualityLevelLOD.ConvertQualityLevelDataUsingCVar(MinLod.PerPlatform, MinLod.Default, bRequireAllPlatformsKnownTrue);
	}
}

void UChaosClothAsset::ExecutePostLoadAssetImpl(FSkinnedAssetPostLoadContext& Context)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UChaosClothAsset::ExecutePostLoadAssetImpl);

	if (!GetOutermost()->bIsCookedForEditor)
	{
		if (GetResourceForRendering() == nullptr)
		{
			CacheDerivedData(&Context);
			Context.bHasCachedDerivedData = true;
		}
	}
}

#endif // WITH_EDITOR

void UChaosClothAsset::SetClothCollections(TArray<TSharedRef<const FManagedArrayCollection>>&& InClothCollections, bool bReregisterComponents)
{
	WaitUntilAsyncPropertyReleased(EClothAssetAsyncProperties::ClothCollection, ESkinnedAssetAsyncPropertyLockType::WriteOnly);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	ClothCollections = MoveTemp(InClothCollections);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	OnPropertyChanged(bReregisterComponents);
}

void UChaosClothAsset::Build(
	const TArray<TSharedRef<const FManagedArrayCollection>>& InClothCollections,
	TArray<FChaosClothAssetLodTransitionDataCache>* InOutTransitionCache,
	FText* ErrorText,
	FText* VerboseText)
{
	using namespace UE::Chaos::ClothAsset;

	// Reset the asset's collection
	TArray<TSharedRef<const FManagedArrayCollection>>& OutClothCollections = GetClothCollectionsInternal();
	OutClothCollections.Reset(InClothCollections.Num());

	// Reset the asset's material list
	TArray<FSkeletalMaterial>& OutMaterials = GetMaterials();
	OutMaterials.Reset();

	// Iterate through the LODs
	FSoftObjectPath PhysicsAssetPathName;
	for (int32 LodIndex = 0; LodIndex < InClothCollections.Num(); ++LodIndex)
	{
		// New LOD
		TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>();
		UE::Chaos::ClothAsset::FCollectionClothFacade ClothFacade(ClothCollection);
		ClothFacade.DefineSchema();

		const FCollectionClothConstFacade InClothFacade(InClothCollections[LodIndex]);
		if (!InClothFacade.HasValidRenderData() && !InClothFacade.HasValidSimulationData())  // The cloth collection must at least have a mesh
		{
			if (ErrorText && ErrorText->IsEmpty())
			{
				*ErrorText = LOCTEXT("BuildErrorText", "Invalid LOD.");
				if (VerboseText)
				{
					*VerboseText = FText::Format(LOCTEXT("BuildVerboseTextFirstError", "LOD {0} has no valid data."), LodIndex);
				}
			}
			else if (ErrorText && VerboseText)
			{
				*VerboseText = FText::Format(LOCTEXT("BuildVerboseTextThereafter", "{0}\nLOD {1} has no valid data."), *VerboseText, LodIndex);
			}
			// else no error reporting

			OutClothCollections.Emplace(MoveTemp(ClothCollection));
			continue;
		}

		// Copy input LOD to current output LOD
		ClothFacade.Initialize(InClothFacade);

		// Add this LOD's materials to the asset
		const int32 NumLodMaterials = ClothFacade.GetNumRenderPatterns();

		OutMaterials.Reserve(OutMaterials.Num() + NumLodMaterials);

		const TConstArrayView<FSoftObjectPath> LodRenderMaterialPathName = ClothFacade.GetRenderMaterialSoftObjectPathName();
		for (int32 LodMaterialIndex = 0; LodMaterialIndex < NumLodMaterials; ++LodMaterialIndex)
		{
			const FSoftObjectPath& RenderMaterialPathName = LodRenderMaterialPathName[LodMaterialIndex];

			if (UMaterialInterface* const Material = Cast<UMaterialInterface>(RenderMaterialPathName.TryLoad()))
			{
				OutMaterials.Emplace(Material, true, false, Material->GetFName());
			}
			else
			{
				OutMaterials.Emplace();  // Must keep one material slots per render pattern, do not change this behavior without updating the fix up code in GetOutfitClothCollectionsNode.cpp
			}
		}

		// Set properties
		constexpr bool bUpdateExistingProperties = false;
		Chaos::Softs::FCollectionPropertyMutableFacade PropertyFacade(ClothCollection);
		PropertyFacade.Append(InClothCollections[LodIndex].ToSharedPtr(), bUpdateExistingProperties);

		// Set selections
		FCollectionClothSelectionFacade Selection(ClothCollection);
		FCollectionClothSelectionConstFacade InSelection(InClothCollections[LodIndex]);
		if (InSelection.IsValid())
		{
			Selection.DefineSchema();
			const TArray<FName> InSelectionNames = InSelection.GetNames();
			for (const FName& InSelectionName : InSelectionNames)
			{
				const FName SelectionGroup = InSelection.GetSelectionGroup(InSelectionName);
				if (SelectionGroup == ClothCollectionGroup::SimVertices3D ||
					SelectionGroup == ClothCollectionGroup::SimFaces)
				{
					Selection.FindOrAddSelectionSet(InSelectionName, SelectionGroup) = InSelection.GetSelectionSet(InSelectionName);
				}
			}
		}

		// Set springs
		Chaos::Softs::FEmbeddedSpringFacade EmbeddedSpringFacade(ClothCollection.Get(), ClothCollectionGroup::SimVertices3D);
		const Chaos::Softs::FEmbeddedSpringFacade InEmbeddedSpringFacade(InClothCollections[LodIndex].Get(), ClothCollectionGroup::SimVertices3D);
		if (InEmbeddedSpringFacade.IsValid())
		{
			EmbeddedSpringFacade.DefineSchema();
			constexpr int32 VertexOffset = 0;
			EmbeddedSpringFacade.Append(InEmbeddedSpringFacade, VertexOffset);
		}

		// Set physics asset and skeleton source only with LOD 0 at the moment
		if (LodIndex == 0)
		{
			using namespace ::Chaos::Softs;
			PhysicsAssetPathName = InClothFacade.GetPhysicsAssetSoftObjectPathName();
			const FSoftObjectPath& SkeletalMeshPathName = InClothFacade.GetSkeletalMeshSoftObjectPathName();
			USkeletalMesh* const SkeletalMesh = Cast<USkeletalMesh>(SkeletalMeshPathName.TryLoad());
			if (!SkeletalMeshPathName.IsNull() && !SkeletalMesh && ErrorText)
			{
				*ErrorText = ErrorText->IsEmpty() ?
					FText::Format(LOCTEXT("BuildErrorTextMissingSkeletalMeshFirstError", "Skeletal Mesh {0} is missing cannot be loaded and used as source skeleton for this asset."), FText::FromString(SkeletalMeshPathName.ToString())) :
					FText::Format(LOCTEXT("BuildErrorTextMissingSkeletalMeshThereafter", "{0}\nSkeletal Mesh {1} is missing cannot be loaded and used as source skeleton for this asset."), *ErrorText, FText::FromString(SkeletalMeshPathName.ToString()));
			}

			// Set reference skeleton
			SetSkeleton(SkeletalMesh ? SkeletalMesh->GetSkeleton() : nullptr);  // For completion only, this is not being used and might mismatch the skeletal mesh's reference skeleton
			Super::SetReferenceSkeleton(SkeletalMesh ? &SkeletalMesh->GetRefSkeleton() : nullptr);  // When nullptr is specified a single root bone skeleton is created
		}

		// Fix the skeleton mesh binding if needed, which can cause crashes in the render code, or make the sim mesh disappear
		const bool bHasValidSimSkinweights = Private::HasValidSkinweights(InClothFacade.GetSimBoneIndices(), InClothFacade.GetSimBoneWeights(), &GetRefSkeleton());
		const bool bHasValidRenderSkinweights = Private::HasValidSkinweights(InClothFacade.GetRenderBoneIndices(), InClothFacade.GetRenderBoneWeights(), &GetRefSkeleton());
		if (!ensureAlwaysMsgf(bHasValidSimSkinweights && bHasValidRenderSkinweights, TEXT("A Dataflow node, likely an import node, has generated missing or invalid skin weights in this collection LOD. This must be fixed ASAP!")))
		{
			FClothGeometryTools::BindMeshToRootBone(ClothCollection, !bHasValidSimSkinweights, !bHasValidRenderSkinweights);
		}

		OutClothCollections.Emplace(MoveTemp(ClothCollection));
	}

	// Make sure that whatever happens there is always at least one empty LOD to avoid crashing the render data
	if (OutClothCollections.Num() < 1)
	{
		TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>();
		FCollectionClothFacade ClothFacade(ClothCollection);
		ClothFacade.DefineSchema();
		OutClothCollections.Emplace(MoveTemp(ClothCollection));
	}

	// Set physics asset (note: the cloth asset's physics asset is only replaced if a collection path name is found valid)
	PhysicsAsset = Cast<UPhysicsAsset>(PhysicsAssetPathName.TryLoad());

	SetHasVertexColors(true);

	// Rebuild the asset static data
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Build(InOutTransitionCache);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UChaosClothAsset::Build(TArray<FChaosClothAssetLodTransitionDataCache>* InOutTransitionCache)
{
	using namespace UE::Chaos::ClothAsset;

	// Unregister dependent components, the context will reregister them at the end of the scope
	const FMultiComponentReregisterContext MultiComponentReregisterContext(GetDependentComponents());

#if WITH_EDITOR
	FSkinnedAssetAsyncBuildScope AsyncBuildScope(this);

	FSkinnedAssetBuildContext Context;
	BeginBuildInternal(Context);
#else
	ReleaseResources();
#endif

	// Set a new Guid to invalidate the DDC
	AssetGuid = FGuid::NewGuid();

	// Rebuild matrices
	CalculateInvRefMatrices();

	// Add LODs to the render data
	const int32 NumLods = FMath::Max(GetClothCollectionsInternal().Num(), 1);  // The render data will always look for at least one default LOD 0

	// Rebuild LOD Infos
	LODInfo.Reset(NumLods);
	LODInfo.AddDefaulted(NumLods);  // TODO: Expose some properties to fill up the LOD infos

	// Build simulation model
	BuildClothSimulationModel(InOutTransitionCache);

#if WITH_EDITOR
	// Load/save render data from/to DDC
	ExecuteBuildInternal(Context);

	// Update bounds (must be after render data is built, since CalculateBounds reads the position buffer)
	CalculateBounds();
#endif

	if (FApp::CanEverRender())
	{
		InitResources();
	}

#if WITH_EDITOR
	FinishBuildInternal(Context);
#endif

	// Update any components using this asset
	constexpr bool bReregisterComponents = false;  // Do not reregister twice, this is already done at the function scope
	OnAssetChanged(bReregisterComponents);
}

void UChaosClothAsset::BuildClothSimulationModel(TArray<FChaosClothAssetLodTransitionDataCache>* InOutTransitionCache)
{
	ClothSimulationModel = MakeShared<FChaosClothSimulationModel>(const_cast<const UChaosClothAsset*>(this)->GetClothCollections(), 
		GetRefSkeleton(), InOutTransitionCache);
}

FString UChaosClothAsset::GetAsyncPropertyName(uint64 Property) const
{
	// Async property bits are partitioned: base class uses bits 0-31, derived classes use bits 32-63
	// (the generic AcquireAsyncProperty template in UChaosClothAssetBase shifts derived enum values left by 32).
	constexpr uint64 DerivedShift = 32ull;
	constexpr uint64 BaseMask = (1ull << DerivedShift) - 1ull;

	FString Result;
	if (const uint64 DerivedBits = Property >> DerivedShift)
	{
		Result = StaticEnum<EClothAssetAsyncProperties>()->GetValueOrBitfieldAsString(DerivedBits);
	}
	if (const uint64 BaseBits = Property & BaseMask)
	{
		const FString BaseName = Super::GetAsyncPropertyName(BaseBits);
		if (!BaseName.IsEmpty())
		{
			if (!Result.IsEmpty())
			{
				Result += TEXT("|");
			}
			Result += BaseName;
		}
	}
	return Result;
}

#if WITH_EDITOR
const TCHAR* UChaosClothAsset::GetDerivedDataVersion() const
{
	return CHAOS_CLOTH_ASSET_DERIVED_DATA_VERSION;
}

void UChaosClothAsset::CacheDerivedData(FSkinnedAssetCompilationContext* Context)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UChaosClothAsset::CacheDerivedData);
	check(Context);

	// Cache derived data for the running platform.
	ITargetPlatform* const RunningPlatform = GetTargetPlatformManagerRef().GetRunningTargetPlatform();
	if (!RunningPlatform)
	{
		UE_LOGF(LogChaosClothAsset, Warning, "[%ls] DDC cache skipped: no running target platform.", *GetName());
		return;
	}

	// FSkeletalMeshRenderData::Cache writes a richer blob than the base's bare Serialize.
	// Use it for both fetch and store so the cloth DDC format stays consistent.
	SetResourceForRendering(MakeUnique<FSkeletalMeshRenderData>());
	PrepareMeshModel();
	GetResourceForRendering()->Cache(RunningPlatform, this, Context);
}

void UChaosClothAsset::PrepareMeshModel()
{
	// Size from the cloth-collection LOD count; the head render data is empty here.
	// ReadOnly wait on ClothCollection because BeginPostLoadAssetImpl already released the WriteOnly half.
	WaitUntilAsyncPropertyReleased(EClothAssetAsyncProperties::ImportedModel);
	WaitUntilAsyncPropertyReleased(EClothAssetAsyncProperties::ClothCollection, ESkinnedAssetAsyncPropertyLockType::ReadOnly);

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const int32 NumLods = FMath::Max(ClothCollections.Num(), 1);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	MeshModel = MakeShared<FSkeletalMeshModel>();
	MeshModel->LODModels.Reset(NumLods);
	for (int32 LodIndex = 0; LodIndex < NumLods; ++LodIndex)
	{
		MeshModel->LODModels.Add(new FSkeletalMeshLODModel());
	}

	UE_LOGF(LogChaosClothAsset, Verbose, "[%ls] PrepareMeshModel: NumLODs=%d.", *GetName(), NumLods);
}

void UChaosClothAsset::BuildLODModel(FSkeletalMeshRenderData& RenderData, const ITargetPlatform* TargetPlatform, int32 LODIndex)
{
	check(GetImportedModel() && GetImportedModel()->LODModels.IsValidIndex(LODIndex));
	FBuilder::BuildLod(GetImportedModel()->LODModels[LODIndex], *this, LODIndex, TargetPlatform);
}

FString UChaosClothAsset::BuildDerivedDataKey(const ITargetPlatform* TargetPlatform)
{
	FString KeySuffix;
	KeySuffix += AssetGuid.ToString();

	// AssetGuid is regenerated on every Build, so it covers source cloth-collection changes.
	// The imported model's IdString is transient and unavailable at every lifted entry point.

	AppendBuildSettingsToDDCKey(KeySuffix, TargetPlatform);

	return FDerivedDataCacheInterface::BuildCacheKey(
		GetDerivedDataPrefix(),
		GetDerivedDataVersion(),
		*KeySuffix);
}
#endif // WITH_EDITOR

bool UChaosClothAsset::HasValidClothSimulationModels() const
{
	return ClothSimulationModel && ClothSimulationModel->GetNumLods();
}

void UChaosClothAsset::SetPhysicsAsset(UPhysicsAsset* InPhysicsAsset)
{
	using namespace UE::Chaos::ClothAsset;

	PhysicsAsset = InPhysicsAsset;
}

void UChaosClothAsset::SetReferenceSkeleton(const FReferenceSkeleton* ReferenceSkeleton, bool bRebuildModels, bool bRebindMeshes)
{
	using namespace UE::Chaos::ClothAsset;

	// Update the reference skeleton
	Super::SetReferenceSkeleton(ReferenceSkeleton);

	// Rebuild the models
	if (bRebuildModels)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		Build();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

void UChaosClothAsset::UpdateSkeletonFromCollection(bool /*bRebuildModels*/)
{
	using namespace UE::Chaos::ClothAsset;

	check(GetClothCollectionsInternal().Num());
	FCollectionClothConstFacade ClothFacade(GetClothCollectionsInternal()[0]);
	check(ClothFacade.IsValid());

	const FSoftObjectPath& SkeletalMeshPathName = ClothFacade.GetSkeletalMeshSoftObjectPathName();
	USkeletalMesh* const SkeletalMesh = Cast<USkeletalMesh>(SkeletalMeshPathName.TryLoad());

	SetSkeleton(SkeletalMesh ? SkeletalMesh->GetSkeleton() : nullptr); // For completion only, this is not being used and might mismatch the skeletal mesh's reference skeleton
	Super::SetReferenceSkeleton(SkeletalMesh ? &SkeletalMesh->GetRefSkeleton() : nullptr);
}

void UChaosClothAsset::CopySimMeshToRenderMesh(UMaterialInterface* Material)
{
	using namespace UE::Chaos::ClothAsset;
	check(GetClothCollectionsInternal().Num());

	// Add a default material if none is specified
	const FSoftObjectPath RenderMaterialPathName = FSoftObjectPath(Material ?
		Material->GetPathName() :
		FString(TEXT("/Engine/EditorMaterials/Cloth/CameraLitDoubleSided.CameraLitDoubleSided")));

	bool bAnyLodHasRenderMesh = false;
	for (TSharedRef<const FManagedArrayCollection>& ClothCollection : GetClothCollectionsInternal())
	{
		TSharedRef<FManagedArrayCollection> NewClothCollection = MakeShared<FManagedArrayCollection>(*ClothCollection);
		constexpr bool bSingleRenderPattern = true;
		FClothGeometryTools::CopySimMeshToRenderMesh(NewClothCollection, RenderMaterialPathName, bSingleRenderPattern);
		bAnyLodHasRenderMesh = bAnyLodHasRenderMesh || FClothGeometryTools::HasRenderMesh(NewClothCollection);
		ClothCollection = MoveTemp(NewClothCollection);
	}

	// Set new material
	Materials.Reset(1);
	if (bAnyLodHasRenderMesh)
	{
		if (UMaterialInterface* const LoadedMaterial = Cast<UMaterialInterface>(RenderMaterialPathName.TryLoad()))
		{
			Materials.Emplace(LoadedMaterial, true, false, LoadedMaterial->GetFName());
		}
	}
}

#if WITH_EDITOR
UChaosClothAssetBase* UChaosClothAsset::CreatePreviewAssetCopy(UObject* Outer, EObjectFlags Flags) const
{
	using namespace UE::Chaos::ClothAsset;

	FSkeletalMeshRenderData* const RenderData = GetResourceForRendering();
	const bool bHasRenderData = RenderData &&
		RenderData->LODRenderData.Num() &&
		RenderData->LODRenderData[0].GetTotalFaces();

	return !bHasRenderData && HasValidClothSimulationModels() ?
		CreateSimModelPreviewAsset(*this, Outer, Flags, FClothEngineTools::GetSimPreviewMaterial()) :
		nullptr;  // The asset already has render data; the thumbnail renderer will use the asset itself.
}
#endif

#undef LOCTEXT_NAMESPACE
