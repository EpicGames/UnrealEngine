// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ClothAssetBase.h"
#if WITH_EDITORONLY_DATA
#include "Animation/AnimationAsset.h"
#endif
#include "EditorFramework/AssetImportData.h"
#include "ChaosClothAsset/ClothAssetSKMClothingAsset.h"
#include "ChaosClothAsset/ClothAssetPrivate.h"
#include "ChaosClothAsset/ClothComponent.h"
#include "ChaosClothAsset/ClothEngineTools.h"
#include "ChaosClothAsset/ClothSimulationModel.h"
#include "ChaosClothAsset/SkeletalMeshConverterClassProvider.h"
#include "Components/SkeletalMeshComponent.h"
#if WITH_EDITORONLY_DATA
#include "Engine/Blueprint.h"
#include "Engine/BlueprintGeneratedClass.h"
#endif
#include "Engine/Engine.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAssetAsyncCompileUtils.h"
#if WITH_EDITOR
#include "DerivedDataCacheInterface.h"
#include "Engine/RendererSettings.h"
#include "Features/IModularFeatures.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "ProfilingDebugging/ScopedTimers.h"
#include "Rendering/SkeletalMeshLODModel.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Serialization/LargeMemoryReader.h"
#include "Serialization/LargeMemoryWriter.h"
#endif
#include "HAL/IConsoleManager.h"
#include "ProfilingDebugging/ResourceSize.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "UObject/ObjectMacros.h"
#if WITH_EDITOR
#include "UObject/PackageReload.h"
#endif
#include "UObject/UObjectGlobals.h"
#include "ClothingSimulationFactory.h"
#include "ClothingSimulationInstance.h"
#include "ClothingSimulationInteractor.h"
#if WITH_EDITOR
#include "GPUSkinVertexFactory.h"
#include "IMeshBuilderModule.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClothAssetBase)
namespace UE::Chaos::ClothAsset::Private
{

	const TCHAR* MinLodQualityLevelCVarName = TEXT("p.ClothAsset.MinLodQualityLevel");
	const TCHAR* MinLodQualityLevelScalabilitySection = TEXT("ViewDistanceQuality");
	int32 MinLodQualityLevel = -1;
	FAutoConsoleVariableRef CVarClothAssetMinLodQualityLevel(
		MinLodQualityLevelCVarName,
		MinLodQualityLevel,
		TEXT("The quality level for the Min stripping LOD. \n"),
		FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* /*Variable*/)
			{
#if WITH_EDITOR || PLATFORM_DESKTOP
				if (GEngine && GEngine->UseClothAssetMinLODPerQualityLevels)
				{
					for (TObjectIterator<UChaosClothAssetBase> It; It; ++It)
					{
						UChaosClothAssetBase* ClothAsset = *It;
						if (ClothAsset && ClothAsset->GetQualityLevelMinLod().PerQuality.Num() > 0)
						{
							FSkinnedMeshComponentRecreateRenderStateContext Context(ClothAsset, false);
						}
					}
				}
#endif
			}),
		ECVF_Scalability);
		
#if !UE_BUILD_SHIPPING
	FAutoConsoleCommandWithArgsAndOutputDevice CmdClothMemoryReport(
		TEXT("p.ClothAsset.MemoryReport"),
		TEXT("Outputs memory usage for the provided cloth asset name"),
		FConsoleCommandWithArgsAndOutputDeviceDelegate::CreateLambda([](TArray<FString> Args, FOutputDevice& Ar)
		{
			if (Args.Num() != 1)
			{
				Ar.Logf(TEXT("Invalid usage, should be: p.ClothAsset.MemoryReport <ObjectName>"));
				return;
			}

			const FString& ObjectName = Args[0];
			UChaosClothAssetBase* Obj = FindFirstObject<UChaosClothAssetBase>(*ObjectName, EFindFirstObjectOptions::None, ELogVerbosity::Warning, TEXT("p.ClothAsset.MemoryReport"));

			if (!Obj)
			{
				return;
			}

			FString MemoryReport;
			MemoryReport.Appendf(TEXT("---- Memory report for [%s] [%s] ----"), *Obj->StaticClass()->GetName(), *Obj->GetName());

			FResourceSizeEx RenderDataResourceSize;
			if (Obj->GetResourceForRendering())
			{
				for (int32 LodIndex = 0; LodIndex < Obj->GetResourceForRendering()->LODRenderData.Num(); ++LodIndex)
				{
					FResourceSizeEx LODRenderDataResourceSize;
					Obj->GetResourceForRendering()->LODRenderData[LodIndex].GetResourceSizeEx(LODRenderDataResourceSize);
					MemoryReport.Appendf(TEXT("\n LODRenderData LOD%d size: %zu bytes"), LodIndex, LODRenderDataResourceSize.GetTotalMemoryBytes());
				}

				Obj->GetResourceForRendering()->GetResourceSizeEx(RenderDataResourceSize);
			}
			MemoryReport.Appendf(TEXT("\n Total RenderData size: %zu bytes"), RenderDataResourceSize.GetTotalMemoryBytes());

			FResourceSizeEx ClothSimulationModelsResourceSize;
			for (int32 ModelIndex = 0; ModelIndex < Obj->GetNumClothSimulationModels(); ++ModelIndex)
			{
				if (const TSharedPtr<FChaosClothSimulationModel> ClothSimulationModel = ConstCastSharedPtr<FChaosClothSimulationModel>(Obj->GetClothSimulationModel(ModelIndex)))
				{
					for (int32 LodIndex = 0; LodIndex < ClothSimulationModel->GetNumLods(); ++LodIndex)
					{
						FResourceSizeEx ClothSimulationLodModelResourceSize;
						ClothSimulationModel->ClothSimulationLodModels[LodIndex].GetResourceSizeEx(ClothSimulationLodModelResourceSize);
						MemoryReport.Appendf(TEXT("\n ClothSimulationModel%d LOD%d size: %zu bytes"), ModelIndex, LodIndex, ClothSimulationLodModelResourceSize.GetTotalMemoryBytes());
					}

					ClothSimulationModel->GetResourceSizeEx(ClothSimulationModelsResourceSize);
				}
			}
			MemoryReport.Appendf(TEXT("\n Total ClothSimulationModel(s) size: %zu bytes"), ClothSimulationModelsResourceSize.GetTotalMemoryBytes());

			const int64 TotalResourceSize = RenderDataResourceSize.GetTotalMemoryBytes() + ClothSimulationModelsResourceSize.GetTotalMemoryBytes();
			MemoryReport.Appendf(
				TEXT("\n Total resource size for Cloth Asset [%s]: %lld bytes (%.3f MB)"),
				*Obj->GetName(),
				TotalResourceSize,
				(float)TotalResourceSize / (1024.f * 1024.f));

			FResourceSizeEx CumulativeResourceSize(EResourceSizeMode::EstimatedTotal);
			Obj->GetResourceSizeEx(CumulativeResourceSize);
			const int64 TotalSize = CumulativeResourceSize.GetTotalMemoryBytes();
			MemoryReport.Appendf(
				TEXT("\n Total size for Cloth Asset [%s]: %lld bytes (%.3f MB)"),
				*Obj->GetName(),
				TotalSize,
				(float)TotalSize / (1024.f * 1024.f));

			UE_LOGF(LogChaosClothAsset, Display, "\n%ls", *MemoryReport);
		}),
		ECVF_Cheat
	);
#endif //!UE_BUILD_SHIPPING
}

UChaosClothAssetBase::UChaosClothAssetBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	, DataflowInstance(this)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	, MinQualityLevelLOD(0)
	, DisableBelowMinLodStripping(FPerPlatformBool(false))
	, MinLod(0)
    , bSupportRayTracing(true)
    , RayTracingMinLOD(0)
{

	// Add the LODInfo for the default LOD 0
	LODInfo.SetNum(1);

	// Set default skeleton (must be done after having added the LOD)
	SetReferenceSkeleton(nullptr);

	MinQualityLevelLOD.SetQualityLevelCVarForCooking(UE::Chaos::ClothAsset::Private::MinLodQualityLevelCVarName, UE::Chaos::ClothAsset::Private::MinLodQualityLevelScalabilitySection);
}

void UChaosClothAssetBase::SetReferenceSkeleton(const FReferenceSkeleton* ReferenceSkeleton)
{
	// Update the reference skeleton
	if (ReferenceSkeleton)
	{
		GetRefSkeleton() = *ReferenceSkeleton;
	}
	else
	{
		// Create a default reference skeleton
		GetRefSkeleton().Empty(1);
		FReferenceSkeletonModifier ReferenceSkeletonModifier(GetRefSkeleton(), nullptr);

		FMeshBoneInfo MeshBoneInfo;
		constexpr const TCHAR* RootName = TEXT("Root");
		MeshBoneInfo.ParentIndex = INDEX_NONE;
#if WITH_EDITORONLY_DATA
		MeshBoneInfo.ExportName = RootName;
#endif
		MeshBoneInfo.Name = FName(RootName);
		ReferenceSkeletonModifier.Add(MeshBoneInfo, FTransform::Identity);
	}
}

TObjectPtr<UDataflowBaseContent> UChaosClothAssetBase::CreateDataflowContent()
{
	TObjectPtr<UDataflowBaseContent> DataflowContent = UE::DataflowContextHelpers::CreateNewDataflowContent<UDataflowBaseContent>(this);

	DataflowContent->SetDataflowOwner(this);
	DataflowContent->SetTerminalAsset(this);

	WriteDataflowContent(DataflowContent);

	DataflowContent->OnContentDataChanged.AddUObject(this, &UChaosClothAssetBase::UpdateSimulationActor);

	return DataflowContent;
}

void UChaosClothAssetBase::WriteDataflowContent(const TObjectPtr<UDataflowBaseContent>& DataflowContent) const
{
	if (DataflowContent)
	{
		DataflowContent->SetDataflowAsset(GetDataflowInstance().GetDataflowAsset());
		DataflowContent->SetDataflowTerminal(GetDataflowInstance().GetDataflowTerminal().ToString());
	}
}

void UChaosClothAssetBase::ReadDataflowContent(const TObjectPtr<UDataflowBaseContent>& DataflowContent)
{
}

void UChaosClothAssetBase::UpdateSimulationActor(TObjectPtr<AActor>& SimulationActor) const 
{
	TInlineComponentArray<UChaosClothComponent*> ChaosClothComponents(SimulationActor);
	for(UChaosClothComponent* ClothComponent : ChaosClothComponents)
	{
		if(ClothComponent->GetAsset() == this)
		{
#if WITH_EDITOR
			// Update the config properties on the component from the asset.
			ClothComponent->UpdateConfigProperties();
#endif
		}
	}
}

#if WITH_EDITOR
void UChaosClothAssetBase::RegisterOnPackageReloadedDelegate()
{
	UnregisterOnPackageReloadedDelegate();
	FCoreUObjectDelegates::OnPackageReloaded.AddWeakLambda(this, [this](const EPackageReloadPhase PackageReloadPhase, FPackageReloadedEvent* PackageReloadedEvent)
		{
			UObject* ReloadedObject;
			if (PackageReloadPhase == EPackageReloadPhase::PostPackageFixup &&
				PackageReloadedEvent && PackageReloadedEvent->GetRepointedObject(this, ReloadedObject))
			{
				if (UChaosClothAssetBase* const ReloadedClothAsset = Cast<UChaosClothAssetBase>(ReloadedObject))
				{
					constexpr bool bReregisterComponents = true;
					ReloadedClothAsset->OnAssetChanged(bReregisterComponents);
				}
			}
		});
}

void UChaosClothAssetBase::UnregisterOnPackageReloadedDelegate()
{
	FCoreUObjectDelegates::OnPackageReloaded.RemoveAll(this);
}
#endif

const FDataflowInstance& UChaosClothAssetBase::GetDataflowInstance() const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return DataflowInstance;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

FDataflowInstance& UChaosClothAssetBase::GetDataflowInstance()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return DataflowInstance;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UChaosClothAssetBase::BeginDestroy()
{
	check(IsInGameThread());

	Super::BeginDestroy();

#if WITH_EDITOR
	UnregisterOnPackageReloadedDelegate();
#endif

	// Release the mesh's render resources now
	ReleaseResources();
}

bool UChaosClothAssetBase::IsReadyForFinishDestroy()
{
	if (!Super::IsReadyForFinishDestroy())
	{
		return false;
	}

	ReleaseResources();

	// see if we have hit the resource flush fence
	return ReleaseResourcesFence.IsFenceComplete();
}

void UChaosClothAssetBase::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::AddClothAssetBase)
	{
		return;
	}
	Ar << GetRefSkeleton();
}

#if WITH_EDITORONLY_DATA
template<typename T>
void UChaosClothAssetBase::SyncBPPreviewAsset(TSoftObjectPtr<T>& PreviewAsset, const TCHAR* const VarName)
{
	// Note:
	// This function allows the Cloth Editor preview assets to keep in sync with the Dataflow Editor preview assets set by the BP until the migration to the new editor is complete.
	// Note this is technically not a complete sync, the values are only replaced when they are null as to not change current user preferences in the two editors.
	// Also missing BP variables aren't added, this isn't the purpose of this code.
	if (const UDataflow* const DataflowAsset = GetDataflowInstance().GetDataflowAsset())
	{
		if (const UBlueprintGeneratedClass* const BlueprintGeneratedClass = Cast<const UBlueprintGeneratedClass>(DataflowAsset->PreviewBlueprintClass))
		{
			if (const UBlueprint* const Blueprint = Cast<const UBlueprint>(BlueprintGeneratedClass->ClassGeneratedBy))
			{
				FInstancedPropertyBag& BlueprintVariables = GetDataflowInstance().GetPreviewBlueprintVariableOverrides();

				if (const FBPVariableDescription* const BPVariableDescription =
					Blueprint->NewVariables.FindByPredicate([VarName](const FBPVariableDescription& Element)
						{
							return Element.VarName == VarName;
						}))
				{
					const FName VarGuidAsName(BPVariableDescription->VarGuid.ToString());  // VarGuid instead of the VarName, because the variable name can change and FInstancedPropertyBag::SanitizePropertyName
					if (const TValueOrError<UObject*, EPropertyBagResult> ValueOrError =
						BlueprintVariables.GetValueObject(VarGuidAsName, T::StaticClass());
						ValueOrError.HasValue())
					{
						if (T* const Value = Cast<T>(ValueOrError.GetValue()))
						{
							if (PreviewAsset.IsNull())
							{
								PreviewAsset = Value;  // Keep the the old preview asset up to date, in case it is reopened from inside the Cloth Editor
							}
						}
						else
						{
							BlueprintVariables.SetValueObject(VarGuidAsName, PreviewAsset.LoadSynchronous());  // The variable isn't set, update it with whatever asset we find in the old preview asset
						}
					}
				}
			}
		}
	}
}
#endif

void UChaosClothAssetBase::PostLoad()
{
	Super::PostLoad();

	GetDataflowInstance().PostLoad();

#if WITH_EDITORONLY_DATA
	// Sync preview assets after moving from the old preview assets to the new BP variable system
	SyncBPPreviewAsset(PreviewSceneSkeletalMesh, TEXT("SkeletalMesh"));  // TODO: Deprecate PreviewSceneSkeletalMesh and PreviewSceneAnimation
	SyncBPPreviewAsset(PreviewSceneAnimation, TEXT("AnimationAsset"));   //       and remove these calls once the old Cloth Editor is retired
#endif  // WITH_EDITORONLY_DATA
}

void UChaosClothAssetBase::PostInitProperties()
{
	Super::PostInitProperties();

#if WITH_EDITOR
	if (!IsTemplate())
	{
		RegisterOnPackageReloadedDelegate();
	}
#endif
}

void UChaosClothAssetBase::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	// Dataflow specific tags
	UE::Dataflow::InstanceUtils::GetAssetRegistryTags(this, Context);

#if WITH_EDITORONLY_DATA
	if (AssetImportData)
	{
		// Create the standard asset registry tag with our asset import data info (source filename, timestamp, etc.)
		Context.AddTag(FAssetRegistryTag(SourceFileTagName(), AssetImportData->GetSourceData().ToJson(), FAssetRegistryTag::TT_Hidden));
	}
#endif

	Super::GetAssetRegistryTags(Context);
}

#if WITH_EDITOR
void UChaosClothAssetBase::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UChaosClothAssetBase, OverlayMaterial))
	{
		ReregisterComponents();
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	// This invalidates the IDataflowContentOwner, not the Dataflow itself
	InvalidateDataflowContents();

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // #if WITH_EDITOR

void UChaosClothAssetBase::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);

	if (GetResourceForRendering())
	{
		GetResourceForRendering()->GetResourceSizeEx(CumulativeResourceSize);
	}

	if (!GetClass()->HasAnyClassFlags(CLASS_Abstract))
	{
		for (int32 ModelIndex = 0; ModelIndex < GetNumClothSimulationModels(); ++ModelIndex)
		{
			if (const TSharedPtr<FChaosClothSimulationModel> ClothSimulationModel = ConstCastSharedPtr<FChaosClothSimulationModel>(GetClothSimulationModel(ModelIndex)))
			{
				ClothSimulationModel->GetResourceSizeEx(CumulativeResourceSize);
			}
		}
	}
}

FSkeletalMeshLODInfo* UChaosClothAssetBase::GetLODInfo(int32 Index)
{
	return LODInfo.IsValidIndex(Index) ? &LODInfo[Index] : nullptr;
}

const FSkeletalMeshLODInfo* UChaosClothAssetBase::GetLODInfo(int32 Index) const
{
	return LODInfo.IsValidIndex(Index) ? &LODInfo[Index] : nullptr;
}

FReferenceSkeleton& UChaosClothAssetBase::GetRefSkeleton()
{
	WaitUntilAsyncPropertyReleased(EChaosClothAssetBaseAsyncProperties::RefSkeleton);
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return RefSkeleton;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}
const FReferenceSkeleton& UChaosClothAssetBase::GetRefSkeleton() const
{
	WaitUntilAsyncPropertyReleased(EChaosClothAssetBaseAsyncProperties::RefSkeleton, ESkinnedAssetAsyncPropertyLockType::ReadOnly);
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return RefSkeleton;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

FSkeletalMeshRenderData* UChaosClothAssetBase::GetResourceForRendering() const
{
	WaitUntilAsyncPropertyReleased(EChaosClothAssetBaseAsyncProperties::RenderData);
	return SkeletalMeshRenderData.Get();
}

UMaterialInterface* UChaosClothAssetBase::GetOverlayMaterial() const
{
	WaitUntilAsyncPropertyReleased(EChaosClothAssetBaseAsyncProperties::OverlayMaterial, ESkinnedAssetAsyncPropertyLockType::ReadOnly);
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return OverlayMaterial;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

float UChaosClothAssetBase::GetOverlayMaterialMaxDrawDistance() const
{
	WaitUntilAsyncPropertyReleased(EChaosClothAssetBaseAsyncProperties::OverlayMaterialMaxDrawDistance, ESkinnedAssetAsyncPropertyLockType::ReadOnly);
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return OverlayMaterialMaxDrawDistance;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

const FMeshUVChannelInfo* UChaosClothAssetBase::GetUVChannelData(int32 MaterialIndex) const
{
	if (GetMaterials().IsValidIndex(MaterialIndex))
	{
		// TODO: enable ensure when UVChannelData is setup
		//ensure(GetMaterials()[MaterialIndex].UVChannelData.bInitialized);
		return &GetMaterials()[MaterialIndex].UVChannelData;
	}

	return nullptr;
}

int32 UChaosClothAssetBase::GetMinLodIdx(bool bForceLowestLODIdx) const
{
	if (IsMinLodQualityLevelEnable())
	{
		return bForceLowestLODIdx ? GetQualityLevelMinLod().GetLowestValue() : GetQualityLevelMinLod().GetValue(UE::Chaos::ClothAsset::Private::MinLodQualityLevel);
	}
	else
	{
		return GetMinLod().GetValue();
	}
}

bool UChaosClothAssetBase::GetHasVertexColors() const
{
	WaitUntilAsyncPropertyReleased(EChaosClothAssetBaseAsyncProperties::HasVertexColors, ESkinnedAssetAsyncPropertyLockType::ReadOnly);
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return bHasVertexColors != 0;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

int32 UChaosClothAssetBase::GetPlatformMinLODIdx(const ITargetPlatform* TargetPlatform) const
{
#if WITH_EDITOR
	check(TargetPlatform);
	if (IsMinLodQualityLevelEnable())
	{
		// get all supported quality level from scalability + engine ini files
		return GetQualityLevelMinLod().GetValueForPlatform(TargetPlatform);
	}
	else
	{
		return GetMinLod().GetValueForPlatform(*TargetPlatform->IniPlatformName());
	}
#else
	return 0;
#endif
}

bool UChaosClothAssetBase::IsMinLodQualityLevelEnable() const
{
	return (GEngine && GEngine->UseClothAssetMinLODPerQualityLevels);
}

void UChaosClothAssetBase::SetOverlayMaterial(UMaterialInterface* NewOverlayMaterial)
{
	WaitUntilAsyncPropertyReleased(EChaosClothAssetBaseAsyncProperties::OverlayMaterial);
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	OverlayMaterial = NewOverlayMaterial;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UChaosClothAssetBase::SetOverlayMaterialMaxDrawDistance(float InMaxDrawDistance)
{
	WaitUntilAsyncPropertyReleased(EChaosClothAssetBaseAsyncProperties::OverlayMaterialMaxDrawDistance);
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	OverlayMaterialMaxDrawDistance = InMaxDrawDistance;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UChaosClothAssetBase::SetDataflow(UDataflow* InDataflow)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	DataflowInstance.SetDataflowAsset(InDataflow);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

UDataflow* UChaosClothAssetBase::GetDataflow()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return DataflowInstance.GetDataflowAsset();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

const UDataflow* UChaosClothAssetBase::GetDataflow() const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return DataflowInstance.GetDataflowAsset();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UChaosClothAssetBase::InitResources()
{
	LLM_SCOPE_BYNAME(TEXT("ClothAsset/InitResources"));

	// Build the material channel data used by the texture streamer
	UpdateUVChannelData(false);

	if (SkeletalMeshRenderData.IsValid())
	{
		SkeletalMeshRenderData->InitResources(GetHasVertexColors(), this);
	}
}

void UChaosClothAssetBase::ReleaseResources()
{
	if (SkeletalMeshRenderData && SkeletalMeshRenderData->IsInitialized())
	{
		if (GIsEditor && !GIsPlayInEditorWorld)
		{
			// Flush the rendering command to be sure there is no command left that can create/modify a rendering resource
			FlushRenderingCommands();
		}

		SkeletalMeshRenderData->ReleaseResources();

		// Insert a fence to signal when these commands completed
		ReleaseResourcesFence.BeginFence();
	}
}

void UChaosClothAssetBase::CalculateInvRefMatrices()
{
	auto GetRefPoseMatrix = [this](int32 BoneIndex)->FMatrix
		{
			check(BoneIndex >= 0 && BoneIndex < GetRefSkeleton().GetRawBoneNum());
			FTransform BoneTransform = GetRefSkeleton().GetRawRefBonePose()[BoneIndex];
			BoneTransform.NormalizeRotation();  // Make sure quaternion is normalized!
			return BoneTransform.ToMatrixWithScale();
		};

	const int32 NumRealBones = GetRefSkeleton().GetRawBoneNum();

	RefBasesInvMatrix.Empty(NumRealBones);
	RefBasesInvMatrix.AddUninitialized(NumRealBones);

	// Reset cached mesh-space ref pose
	TArray<FMatrix> ComposedRefPoseMatrices;
	ComposedRefPoseMatrices.SetNumUninitialized(NumRealBones);

	// Precompute the Mesh.RefBasesInverse
	for (int32 BoneIndex = 0; BoneIndex < NumRealBones; ++BoneIndex)
	{
		// Render the default pose
		ComposedRefPoseMatrices[BoneIndex] = GetRefPoseMatrix(BoneIndex);

		// Construct mesh-space skeletal hierarchy
		if (BoneIndex > 0)
		{
			int32 Parent = GetRefSkeleton().GetRawParentIndex(BoneIndex);
			ComposedRefPoseMatrices[BoneIndex] = ComposedRefPoseMatrices[BoneIndex] * ComposedRefPoseMatrices[Parent];
		}

		FVector XAxis, YAxis, ZAxis;
		ComposedRefPoseMatrices[BoneIndex].GetScaledAxes(XAxis, YAxis, ZAxis);
		if (XAxis.IsNearlyZero(UE_SMALL_NUMBER) &&
			YAxis.IsNearlyZero(UE_SMALL_NUMBER) &&
			ZAxis.IsNearlyZero(UE_SMALL_NUMBER))
		{
			// This is not allowed, warn them
			UE_LOGF(
				LogChaosClothAsset,
				Warning,
				"Reference Pose for asset %ls for joint (%ls) includes NIL matrix. Zero scale isn't allowed on ref pose.",
				*GetPathName(),
				*GetRefSkeleton().GetBoneName(BoneIndex).ToString());
		}

		// Precompute inverse so we can use from-refpose-skin vertices
		RefBasesInvMatrix[BoneIndex] = FMatrix44f(ComposedRefPoseMatrices[BoneIndex].Inverse());
	}
}

void UChaosClothAssetBase::SetResourceForRendering(TUniquePtr<FSkeletalMeshRenderData>&& InSkeletalMeshRenderData)
{
	WaitUntilAsyncPropertyReleased(EChaosClothAssetBaseAsyncProperties::RenderData);
	SkeletalMeshRenderData = MoveTemp(InSkeletalMeshRenderData);
}

#if WITH_EDITORONLY_DATA
void UChaosClothAssetBase::SetPreviewSceneSkeletalMesh(USkeletalMesh* Mesh)
{
	PreviewSceneSkeletalMesh = Mesh;
}

USkeletalMesh* UChaosClothAssetBase::GetPreviewSceneSkeletalMesh() const
{
	// Load the SkeletalMesh asset if it's not already loaded
	return PreviewSceneSkeletalMesh.LoadSynchronous();
}

void UChaosClothAssetBase::SetPreviewSceneAnimation(UAnimationAsset* Animation)
{
	PreviewSceneAnimation = Animation;
}

UAnimationAsset* UChaosClothAssetBase::GetPreviewSceneAnimation() const
{
	// Load the animation asset if it's not already loaded
	return PreviewSceneAnimation.LoadSynchronous();
}
#endif  // #if WITH_EDITORONLY_DATA

void UChaosClothAssetBase::SetHasVertexColors(bool InbHasVertexColors)
{
	WaitUntilAsyncPropertyReleased(EChaosClothAssetBaseAsyncProperties::HasVertexColors);
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	bHasVertexColors = InbHasVertexColors;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

FString UChaosClothAssetBase::GetAsyncPropertyName(uint64 Property) const
{
	return StaticEnum<EChaosClothAssetBaseAsyncProperties>()->GetValueOrBitfieldAsString(Property);
}

void UChaosClothAssetBase::BeginPostLoadInternal(FSkinnedAssetPostLoadContext& Context)
{
#if WITH_EDITOR
	TRACE_CPUPROFILER_EVENT_SCOPE(UChaosClothAssetBase::BeginPostLoadInternal);

	checkf(IsInGameThread(), TEXT("Cannot execute function UChaosClothAssetBase::BeginPostLoadInternal asynchronously. Asset: %s"), *GetFullName());
	SetInternalFlags(EInternalObjectFlags::Async);

	// Lock all properties that should not be modified/accessed during async post-load
	USkinnedAsset::AcquireAsyncProperty();

	// This scope allows us to use any locked properties without causing stalls
	FSkinnedAssetAsyncBuildScope AsyncBuildScope(this);

	BeginPostLoadAssetImpl(Context);
#endif
}

void UChaosClothAssetBase::ExecutePostLoadInternal(FSkinnedAssetPostLoadContext& Context)
{
#if WITH_EDITOR
	TRACE_CPUPROFILER_EVENT_SCOPE(UChaosClothAssetBase::ExecutePostLoadInternal);

	// This scope allows us to use any locked properties without causing stalls
	FSkinnedAssetAsyncBuildScope AsyncBuildScope(this);

	ExecutePostLoadAssetImpl(Context);
#endif
}

void UChaosClothAssetBase::FinishPostLoadInternal(FSkinnedAssetPostLoadContext& Context)
{
#if WITH_EDITOR
	TRACE_CPUPROFILER_EVENT_SCOPE(UChaosClothAssetBase::FinishPostLoadInternal);

	checkf(IsInGameThread(), TEXT("Cannot execute function UChaosClothAssetBase::FinishPostLoadInternal asynchronously. Asset: %s"), *GetFullName());
	ClearInternalFlags(EInternalObjectFlags::Async);

	// This scope allows us to use any locked properties without causing stalls
	FSkinnedAssetAsyncBuildScope AsyncBuildScope(this);

	// Match Bounds to the just-produced render data, otherwise rely on the serialized bounds
	if (Context.bHasCachedDerivedData)
	{
		CalculateBounds();
	}
#endif

	if (FApp::CanEverRender())
	{
		InitResources();
	}
	else
	{
		// Update any missing data when cooking.
		UpdateUVChannelData(false);
	}

	CalculateInvRefMatrices();

#if WITH_EDITOR
	USkinnedAsset::ReleaseAsyncProperty();
#endif
}

#if WITH_EDITOR
namespace UE::Chaos::ClothAssetBase::Private
{
	// Show the piece-GUID portion of the suffix (between "<Prefix>_<version>_" and the "_<NumLODs>" separator)
	static FString FormatDDCKeyTail(const FString& Key)
	{
		const int32 FirstUnderscore = Key.Find(TEXT("_"));
		const int32 SecondUnderscore = FirstUnderscore != INDEX_NONE ?
			Key.Find(TEXT("_"), ESearchCase::CaseSensitive, ESearchDir::FromStart, FirstUnderscore + 1) :
			INDEX_NONE;
		if (SecondUnderscore == INDEX_NONE)
		{
			return Key;
		}
		const int32 ThirdUnderscore = Key.Find(TEXT("_"), ESearchCase::CaseSensitive, ESearchDir::FromStart, SecondUnderscore + 1);
		return ThirdUnderscore == INDEX_NONE ?
			Key.Mid(SecondUnderscore + 1) :
			Key.Mid(SecondUnderscore + 1, ThirdUnderscore - SecondUnderscore - 1);
	}
}

void UChaosClothAssetBase::AppendBuildSettingsToDDCKey(FString& InOutKeySuffix, const ITargetPlatform* TargetPlatform)
{
	// Max GPU skin bones (deprecated, kept for DDC key backward compatibility)
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const int32 MaxGPUSkinBones = FGPUBaseSkinVertexFactory::GetMaxGPUSkinBones(TargetPlatform);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	InOutKeySuffix += FString::FromInt(MaxGPUSkinBones);

	// Mesh builder module settings
	IMeshBuilderModule::GetForPlatform(TargetPlatform).AppendToDDCKey(InOutKeySuffix, true);

	// Unlimited bone influences mode
	const bool bUnlimitedBoneInfluences = FGPUBaseSkinVertexFactory::GetUnlimitedBoneInfluences(TargetPlatform);
	InOutKeySuffix += bUnlimitedBoneInfluences ? TEXT("1") : TEXT("0");

	// Default bone influence limit
	InOutKeySuffix += FString::FromInt(
		GetDefault<URendererSettings>()->DefaultBoneInfluenceLimit.GetValueForPlatform(
			*TargetPlatform->IniPlatformName()));

	// LODInfo build GUIDs
	TArray<FSkeletalMeshLODInfo>& LODInfos = GetLODInfoArray();
	for (int32 LODIndex = 0; LODIndex < GetLODNum(); ++LODIndex)
	{
		check(LODInfos.IsValidIndex(LODIndex));
		FSkeletalMeshLODInfo& LOD = LODInfos[LODIndex];
		LOD.BuildGUID = LOD.ComputeDeriveDataCacheKey(nullptr);
		InOutKeySuffix += LOD.BuildGUID.ToString(EGuidFormats::Digits);
	}
}

void UChaosClothAssetBase::CacheDerivedData(FSkinnedAssetCompilationContext* Context)
{
	using namespace UE::Chaos::ClothAssetBase::Private;
	TRACE_CPUPROFILER_EVENT_SCOPE(UChaosClothAssetBase::CacheDerivedData);

	// Try DDC fetch for the running platform. On hit, sets render data.
	// On miss, render data is not set, caller should do full Build + StoreDerivedData.
	ITargetPlatform* const RunningPlatform = GetTargetPlatformManagerRef().GetRunningTargetPlatform();
	if (!RunningPlatform)
	{
		UE_LOGF(LogChaosClothAsset, Warning, "[%ls] DDC fetch skipped: no running target platform.", *GetName());
		return;
	}

	const FString Key = BuildDerivedDataKey(RunningPlatform);
	TArray64<uint8> DerivedData;
	double FetchSeconds = 0.0;
	bool bHit = false;
	{
		FScopedDurationTimer Timer(FetchSeconds);
		bHit = GetDerivedDataCacheRef().GetSynchronous(*Key, DerivedData, GetPathName());
	}

	if (bHit)
	{
		SetResourceForRendering(MakeUnique<FSkeletalMeshRenderData>());
		FLargeMemoryReader Ar(DerivedData.GetData(), DerivedData.Num(), ELargeMemoryReaderFlags::Persistent);
		GetResourceForRendering()->Serialize(Ar, this);
		GetResourceForRendering()->DerivedDataKey = Key;
		UE_LOGF(LogChaosClothAsset, Display,
			"[%ls] DDC hit: loaded %" INT64_FMT " bytes for [%ls] in %.1f ms (key=%ls).",
			*GetName(),
			DerivedData.Num(),
			*RunningPlatform->PlatformName(),
			FetchSeconds * 1000.0,
			*FormatDDCKeyTail(Key));
	}
	else
	{
		UE_LOGF(LogChaosClothAsset, Display,
			"[%ls] DDC miss for [%ls] in %.1f ms (key=%ls).",
			*GetName(),
			*RunningPlatform->PlatformName(),
			FetchSeconds * 1000.0,
			*FormatDDCKeyTail(Key));
	}
}

void UChaosClothAssetBase::StoreDerivedData()
{
	using namespace UE::Chaos::ClothAssetBase::Private;
	TRACE_CPUPROFILER_EVENT_SCOPE(UChaosClothAssetBase::StoreDerivedData);

	if (!GetResourceForRendering() || GetResourceForRendering()->LODRenderData.Num() == 0)
	{
		UE_LOGF(LogChaosClothAsset, Warning, "[%ls] DDC store skipped: no render data.", *GetName());
		return;
	}

	ITargetPlatform* const RunningPlatform = GetTargetPlatformManagerRef().GetRunningTargetPlatform();
	if (!RunningPlatform)
	{
		UE_LOGF(LogChaosClothAsset, Warning, "[%ls] DDC store skipped: no running target platform.", *GetName());
		return;
	}

	const FString Key = BuildDerivedDataKey(RunningPlatform);

	// Skip DDC store for transient/preview assets but still set the head key so IsInitialBuildDone() reports true
	if (GetOutermost() == GetTransientPackage())
	{
		GetResourceForRendering()->DerivedDataKey = Key;
		UE_LOGF(LogChaosClothAsset, Verbose, "[%ls] DDC store skipped: transient asset.", *GetName());
		return;
	}

	int64 DerivedDataSize = 0;
	double StoreSeconds = 0.0;
	{
		FScopedDurationTimer Timer(StoreSeconds);
		// FLargeMemoryWriter matches FSkeletalMeshRenderData::Cache's archive choice, which is the
		// engine's standard pattern for routing FSkeletalMeshRenderData::Serialize bytes through DDC.
		FLargeMemoryWriter Ar(0, /*bIsPersistent=*/true);
		GetResourceForRendering()->Serialize(Ar, this);
		DerivedDataSize = Ar.TotalSize();
		TArrayView64<const uint8> ArView(Ar.GetData(), DerivedDataSize);
		GetDerivedDataCacheRef().Put(*Key, ArView, GetPathName());
	}
	GetResourceForRendering()->DerivedDataKey = Key;

	UE_LOGF(LogChaosClothAsset, Display,
		"[%ls] DDC store: %" INT64_FMT " bytes for [%ls] in %.1f ms (key=%ls).",
		*GetName(), DerivedDataSize, *RunningPlatform->PlatformName(),
		StoreSeconds * 1000.0, *FormatDDCKeyTail(Key));
}

void UChaosClothAssetBase::PrepareMeshModel()
{
	const FSkeletalMeshRenderData* const SourceRenderData = GetResourceForRendering();
	const int32 NumLods = (SourceRenderData && SourceRenderData->LODRenderData.Num() > 0) ?
		SourceRenderData->LODRenderData.Num() : 1;

	MeshModel = MakeShared<FSkeletalMeshModel>();
	MeshModel->LODModels.Reset(NumLods);
	for (int32 LodIndex = 0; LodIndex < NumLods; ++LodIndex)
	{
		MeshModel->LODModels.Add(new FSkeletalMeshLODModel());
	}

	UE_LOGF(LogChaosClothAsset, Verbose, "[%ls] PrepareMeshModel: NumLODs=%d.", *GetName(), NumLods);
}

bool UChaosClothAssetBase::IsInitialBuildDone() const
{
	// The DerivedDataKey is set only after Build->StoreDerivedData, CacheDerivedData (DDC hit), or Serialize cooked-load
	const FSkeletalMeshRenderData* const RenderData = GetResourceForRendering();
	return RenderData && !RenderData->DerivedDataKey.IsEmpty();
}

FSkeletalMeshRenderData& UChaosClothAssetBase::GetPlatformSkeletalMeshRenderData(const ITargetPlatform* TargetPlatform, bool bIsSerializeSaving)
{
	using namespace UE::Chaos::ClothAssetBase::Private;
	check(TargetPlatform);

	FSkeletalMeshRenderData* const HeadRenderData = GetResourceForRendering();
	check(HeadRenderData);

	if (GetOutermost()->bIsCookedForEditor)
	{
		// Cooked-for-editor packages can't rebuild for other platforms - the head is the only data we have.
		UE_LOGF(LogChaosClothAsset, Verbose,
			"[%ls] GetPlatformRenderData: cooked-for-editor, returning head for [%ls].",
			*GetName(),
			*TargetPlatform->PlatformName());
		return *HeadRenderData;
	}

	// Unbuilt asset: no source, no DDC entry, no key on the head. Avoid running Cache() against the empty
	// placeholder for non-host platforms, which would silently produce empty per-platform render data
	if (HeadRenderData->DerivedDataKey.IsEmpty())
	{
		UE_LOGF(LogChaosClothAsset, Error,
			"[%ls] GetPlatformRenderData: asset is unbuilt; returning empty head render data for [%ls] (no per-platform Cache).",
			*GetName(), *TargetPlatform->PlatformName());
		return *HeadRenderData;
	}

	// Linked-list invariant: the head's DerivedDataKey is the running platform's key.
	// CacheDerivedData (DDC hit), StoreDerivedData (DDC miss), and Serialize() cooked-load all establish this.
	const ITargetPlatform* const RunningPlatform = GetTargetPlatformManagerRef().GetRunningTargetPlatform();
	check(RunningPlatform);
	const FString RunningPlatformDerivedDataKey = BuildDerivedDataKey(RunningPlatform);
	checkf(HeadRenderData->DerivedDataKey == RunningPlatformDerivedDataKey,
		TEXT("Head linked-list render data must hold the running platform's data."));

	const FString PlatformDerivedDataKey = BuildDerivedDataKey(TargetPlatform);
	FSkeletalMeshRenderData* PlatformRenderData = HeadRenderData;
	while (PlatformRenderData && PlatformRenderData->DerivedDataKey != PlatformDerivedDataKey)
	{
		PlatformRenderData = PlatformRenderData->NextCachedRenderData.Get();
	}

	if (!PlatformRenderData)
	{
		// Prepare the mesh model for BuildLODModel (only needed on cache miss)
		PrepareMeshModel();

		PlatformRenderData = new FSkeletalMeshRenderData();

		FSkinnedAssetBuildContext Context;
		Context.bIsSerializeSaving = bIsSerializeSaving;
		double CacheSeconds = 0.0;
		{
			FScopedDurationTimer Timer(CacheSeconds);
			PlatformRenderData->Cache(TargetPlatform, this, &Context);
		}

		check(PlatformRenderData->DerivedDataKey == PlatformDerivedDataKey);
		// Insert the new platform render data after the head, preserving the head invariant.
		Swap(PlatformRenderData->NextCachedRenderData, HeadRenderData->NextCachedRenderData);
		HeadRenderData->NextCachedRenderData = TUniquePtr<FSkeletalMeshRenderData>(PlatformRenderData);

		UE_LOGF(LogChaosClothAsset, Display,
			"[%ls] GetPlatformRenderData: built new for [%ls] in %.1f ms (key=%ls).",
			*GetName(),
			*TargetPlatform->PlatformName(),
			CacheSeconds * 1000.0,
			*FormatDDCKeyTail(PlatformDerivedDataKey));
	}
	else if (PlatformRenderData == HeadRenderData)
	{
		// Verbose: this is the no-op fast path that fires for every running-platform cook save; logging at Display would spam the cook log.
		UE_LOGF(LogChaosClothAsset, Verbose,
			"[%ls] GetPlatformRenderData: reusing running-platform head for [%ls].",
			*GetName(),
			*TargetPlatform->PlatformName());
	}
	else
	{
		UE_LOGF(LogChaosClothAsset, Display,
			"[%ls] GetPlatformRenderData: cached hit for [%ls] (key=%ls).",
			*GetName(),
			*TargetPlatform->PlatformName(),
			*FormatDDCKeyTail(PlatformDerivedDataKey));
	}

	// Free the temporary mesh model, only needed during Cache()
	MeshModel.Reset();

	return *PlatformRenderData;
}

void UChaosClothAssetBase::EstablishCookedHeadKey()
{
	using namespace UE::Chaos::ClothAssetBase::Private;
	check(GetResourceForRendering());
	// Maintain the linked-list head invariant: head holds the running-platform render data,
	// keyed so that GetPlatformSkeletalMeshRenderData()'s walk finds it for the running platform.
	if (ITargetPlatform* const RunningPlatform = GetTargetPlatformManagerRef().GetRunningTargetPlatform())
	{
		const FString Key = BuildDerivedDataKey(RunningPlatform);
		GetResourceForRendering()->DerivedDataKey = Key;
		UE_LOGF(LogChaosClothAsset, Display,
			"[%ls] Serialize: cooked load for [%ls] (key=%ls).",
			*GetName(), *RunningPlatform->PlatformName(),
			*FormatDDCKeyTail(Key));
	}
	else
	{
		UE_LOGF(LogChaosClothAsset, Display,
			"[%ls] Serialize: cooked load (no running platform).", *GetName());
	}
}

FSkeletalMeshRenderData* UChaosClothAssetBase::GetSerializeRenderData(FArchive& Ar)
{
	using namespace UE::Chaos::ClothAssetBase::Private;
	// Refuse to cook an unbuilt asset; fail the archive instead of silently writing empty render data
	if (!IsInitialBuildDone())
	{
		UE_LOGF(LogChaosClothAsset, Error,
			"[%ls] Serialize: refusing cooked save of unbuilt asset (no DDC entry, no source). The cook will fail for this asset.",
			*GetName());
		Ar.SetError();
		return nullptr;
	}

	// Use platform-specific render data when cooking
	constexpr bool bIsSerializeSaving = true;
	const ITargetPlatform* const ArchiveCookingTarget = Ar.CookingTarget();
	FSkeletalMeshRenderData* LocalRenderData;
	if (ArchiveCookingTarget)
	{
		LocalRenderData = &GetPlatformSkeletalMeshRenderData(ArchiveCookingTarget, bIsSerializeSaving);
	}
	else
	{
		const ITargetPlatform* const RunningPlatform = GetTargetPlatformManagerRef().GetRunningTargetPlatform();
		check(RunningPlatform);
		LocalRenderData = &GetPlatformSkeletalMeshRenderData(RunningPlatform, bIsSerializeSaving);
	}

	UE_LOGF(LogChaosClothAsset, Display,
		"[%ls] Serialize: cooked save for [%ls] (key=%ls).",
		*GetName(),
		ArchiveCookingTarget ? *ArchiveCookingTarget->PlatformName() : TEXT("(running)"),
		*FormatDDCKeyTail(LocalRenderData->DerivedDataKey));

	return LocalRenderData;
}

void UChaosClothAssetBase::BeginCacheForCookedPlatformData(const ITargetPlatform* TargetPlatform)
{
	// Intentionally empty, caching is done lazily in IsCachedCookedPlatformDataLoaded
}

bool UChaosClothAssetBase::IsCachedCookedPlatformDataLoaded(const ITargetPlatform* TargetPlatform)
{
	// Don't touch platform render data while an async build is in flight (mirror USkeletalMesh)
	if (IsCompiling())
	{
		UE_LOGF(LogChaosClothAsset, Verbose, "[%ls] IsCachedCookedPlatformDataLoaded: waiting on async build.", *GetName());
		return false;
	}
	constexpr bool bIsSerializeSaving = false;
	GetPlatformSkeletalMeshRenderData(TargetPlatform, bIsSerializeSaving);
	return true;
}

void UChaosClothAssetBase::ClearAllCachedCookedPlatformData()
{
	if (GetResourceForRendering())
	{
		GetResourceForRendering()->NextCachedRenderData.Reset();
	}
	MeshModel.Reset();
	UE_LOGF(LogChaosClothAsset, Display, "[%ls] Cleared cached cooked platform data.", *GetName());
}

bool UChaosClothAssetBase::ExportToSkeletalMesh(USkeletalMesh& SkeletalMesh) const
{
	using namespace UE::Chaos::ClothAsset;
	using namespace UE::Geometry;

	const TArray<IClothAssetSkeletalMeshConverterClassProvider*> ClassProviders =
		IModularFeatures::Get().GetModularFeatureImplementations<IClothAssetSkeletalMeshConverterClassProvider>(IClothAssetSkeletalMeshConverterClassProvider::FeatureName);
	if (const IClothAssetSkeletalMeshConverterClassProvider* const ClassProvider = ClassProviders.Num() ? ClassProviders[0] : nullptr)
	{
		if (const TSubclassOf<UClothAssetSkeletalMeshConverter> ClothAssetSkeletalMeshConverterClass = ClassProvider->GetClothAssetSkeletalMeshConverter())
		{
			if (UClothAssetSkeletalMeshConverter* const ClothAssetSkeletalMeshConverter = ClothAssetSkeletalMeshConverterClass->GetDefaultObject<UClothAssetSkeletalMeshConverter>())
			{
				return ClothAssetSkeletalMeshConverter->ExportToSkeletalMesh(*this, SkeletalMesh);
			}
		}
	}
	else
	{
		UE_LOGF(LogChaosClothAsset, Error, "The export to SkeletalMesh has failed: Cannot find a SkeletalMesh converter. Make sure to enable the ChaosClothAssetEditor plugin.");
	}
	return false;
}

UChaosClothAssetBase* UChaosClothAssetBase::CreateSimModelPreviewAsset(
	const UChaosClothAssetBase& SourceAsset,
	UObject* Outer,
	EObjectFlags Flags,
	UMaterialInterface* Material)
{
	using namespace UE::Chaos::ClothAsset;

	// Build render data from the source asset's simulation models
	TUniquePtr<FSkeletalMeshRenderData> SimRenderData = FClothEngineTools::BuildSimPreviewRenderData(SourceAsset);
	if (!SimRenderData)
	{
		return nullptr;
	}

	// Create a transient asset of the same concrete type as the source
	UChaosClothAssetBase* const Preview = NewObject<UChaosClothAssetBase>(Outer, SourceAsset.GetClass(), NAME_None, Flags);

	// Apply the preview material
	Preview->Materials.Reset(1);
	if (Material)
	{
		constexpr bool bEnableShadowCasting = true;
		constexpr bool bRecomputeTangent = false;
		Preview->Materials.Emplace(Material, bEnableShadowCasting, bRecomputeTangent, Material->GetFName());
	}

	// Copy the reference skeleton and set up a single LOD
	Preview->SetReferenceSkeleton(&SourceAsset.GetRefSkeleton());
	Preview->LODInfo.Reset(1);
	Preview->LODInfo.AddDefaulted();

	// Assign the render data and finalize GPU resources
	Preview->SetResourceForRendering(MoveTemp(SimRenderData));
	Preview->CalculateBounds();
	if (FApp::CanEverRender())
	{
		Preview->InitResources();
	}
	Preview->CalculateInvRefMatrices();
	return Preview;
}
#endif  // #if WITH_EDITOR

void UChaosClothAssetBase::OnPropertyChanged(const bool bReregisterComponents) const
{
	// Context will go out of scope, causing the components to be re-registered, but only when bReregisterComponents is true
	const FMultiComponentReregisterContext MultiComponentReregisterContext(bReregisterComponents ? GetDependentComponents() : TArray<UActorComponent*>());

	// Update cloth components properties
	for (TObjectIterator<UChaosClothComponent> ObjectIterator; ObjectIterator; ++ObjectIterator)
	{
		if (UChaosClothComponent* const Component = *ObjectIterator)
		{
			if (Component->GetAsset() == this)
			{
				Component->UpdateConfigProperties();
			}
		}
	}
	// Update skeletal mesh components properties
	for (TObjectIterator<USkeletalMeshComponent> ObjectIterator; ObjectIterator; ++ObjectIterator)
	{
		if (USkeletalMeshComponent* const Component = *ObjectIterator)
		{
			if (Component->GetSkeletalMeshAsset())
			{
				for (const TObjectPtr<UClothingAssetBase>& ClothingAsset : Component->GetSkeletalMeshAsset()->GetMeshClothingAssets())
				{
					if (const UChaosClothAssetSKMClothingAsset* const ClothAssetSKMClothingAsset = Cast<UChaosClothAssetSKMClothingAsset>(ClothingAsset))
					{
						if (ClothAssetSKMClothingAsset->GetAsset() == this)
						{
							for (const FClothingSimulationInstance& ClothingSimulationInstance : Component->GetClothingSimulationInstances())
							{
								if (ClothingSimulationInstance.GetClothingSimulationFactory() &&
									ClothingSimulationInstance.GetClothingSimulationFactory()->SupportsAsset(ClothAssetSKMClothingAsset))
								{
									if (UClothingSimulationInteractor* const ClothingSimulationInteractor = ClothingSimulationInstance.GetClothingSimulationInteractor())
									{
										ClothingSimulationInteractor->ClothConfigUpdated();
									}
								}
							}
						}
					}
				}
			}
		}
	}
}

void UChaosClothAssetBase::OnAssetChanged(const bool bReregisterComponents) const
{
	// Context will go out of scope, causing the components to be re-registered, but only when bReregisterComponents is true
	const FMultiComponentReregisterContext MultiComponentReregisterContext(bReregisterComponents ? GetDependentComponents() : TArray<UActorComponent*>());

#if WITH_EDITOR
	for (TObjectIterator<UChaosClothAssetSKMClothingAsset> ObjectIterator; ObjectIterator; ++ObjectIterator)
	{
		if (UChaosClothAssetSKMClothingAsset* const ClothingAsset = *ObjectIterator)
		{
			if (ClothingAsset->GetAsset() == this)
			{
				constexpr bool bReregisterComponentsFromclothingAsset = false;  // Reregistration is done at function scope instead
				ClothingAsset->OnAssetChanged(bReregisterComponentsFromclothingAsset);
			}
		}
	}

	// Notify listeners (e.g. UThumbnailManager) that the asset has changed so cached thumbnails get regenerated
	FPropertyChangedEvent EmptyEvent(nullptr);
	FCoreUObjectDelegates::OnObjectPropertyChanged.Broadcast(const_cast<UChaosClothAssetBase*>(this), EmptyEvent);
#endif
}

void UChaosClothAssetBase::ReregisterComponents() const
{
	// Context goes out of scope, causing the components to be re-registered
	FMultiComponentReregisterContext MultiComponentReregisterContext(GetDependentComponents());
}

TArray<UActorComponent*> UChaosClothAssetBase::GetDependentComponents() const
{
	TArray<UActorComponent*> DependentComponents;

	// Find cloth components
	for (TObjectIterator<UChaosClothComponent> ObjectIterator; ObjectIterator; ++ObjectIterator)
	{
		if (UChaosClothComponent* const Component = *ObjectIterator)
		{
			if (Component->GetAsset() == this)
			{
				DependentComponents.Emplace(Component);
			}
		}
	}
	// Find skeletal mesh components
	for (TObjectIterator<UChaosClothAssetSKMClothingAsset> ObjectIterator; ObjectIterator; ++ObjectIterator)
	{
		if (UChaosClothAssetSKMClothingAsset* const ClothingAsset = *ObjectIterator)
		{
			if (ClothingAsset->GetAsset() == this)
			{
				if (USkeletalMesh* const OwnerMesh = Cast<USkeletalMesh>(ClothingAsset->GetOuter()))
				{
					for (TObjectIterator<USkeletalMeshComponent> It; It; ++It)
					{
						if (USkeletalMeshComponent* const Component = *It)
						{
							if (Component->GetSkeletalMeshAsset() == OwnerMesh)
							{
								DependentComponents.AddUnique(Component);  // Using AddUnique here since multiple SKMClothingAssets can have the same owner asset
							}
						}
					}
				}
			}
		}
	}
	return DependentComponents;
}

void UChaosClothAssetBase::CalculateBounds()
{
	FBox BoundingBox(ForceInit);
	if (FSkeletalMeshRenderData* const RenderData = GetResourceForRendering())
	{
		for (const FSkeletalMeshLODRenderData& LOD : RenderData->LODRenderData)
		{
			const FPositionVertexBuffer& PositionVertexBuffer = LOD.StaticVertexBuffers.PositionVertexBuffer;
			for (uint32 VertexIndex = 0; VertexIndex < PositionVertexBuffer.GetNumVertices(); ++VertexIndex)
			{
				BoundingBox += FVector(PositionVertexBuffer.VertexPosition(VertexIndex));
			}
		}
	}
	Bounds = FBoxSphereBounds(BoundingBox);
}
