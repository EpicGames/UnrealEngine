// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "ChaosClothAsset/ClothAssetBuilder.h"
#include "ChaosClothAsset/ClothAsset.h"
#include "ChaosClothAsset/ClothSimulationModel.h"
#include "ChaosClothAsset/ClothCollectionGroup.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "ChaosClothAsset/ClothAssetPrivate.h"
#include "ChaosClothAsset/ClothEngineTools.h"
#include "Chaos/CollectionPropertyFacade.h"
#include "Engine/RendererSettings.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Misc/ScopedSlowTask.h"
#include "Modules/ModuleManager.h"
#include "Rendering/SkeletalMeshLODModel.h"
#include "Utils/ClothingMeshUtils.h"
#include "BoneWeights.h"
#include "MeshUtilities.h"
#include "PointWeightMap.h"
#include "Tasks/Task.h"
#include "Async/TaskGraphInterfaces.h"
#include "Async/Mutex.h"

#define LOCTEXT_NAMESPACE "ClothAssetBuilderEditor"

namespace UE::Chaos::ClothAsset::Private
{
	int32 GetNumVertices(const UChaosClothAsset& ClothAsset, int32 LodIndex)
	{
		const TSharedPtr<const FChaosClothSimulationModel> ClothSimulationModel = ClothAsset.GetClothSimulationModel();
		return ClothSimulationModel && ClothSimulationModel->IsValidLodIndex(LodIndex) ?
			ClothSimulationModel->GetNumVertices(LodIndex) :
			0;
	}

	TConstArrayView<FVector3f> GetSimPositions(const UChaosClothAsset& ClothAsset, int32 LodIndex)
	{
		const TSharedPtr<const FChaosClothSimulationModel> ClothSimulationModel = ClothAsset.GetClothSimulationModel();
		return ClothSimulationModel && ClothSimulationModel->IsValidLodIndex(LodIndex) ?
			ClothSimulationModel->GetPositions(LodIndex) :
			TConstArrayView<FVector3f>();
	}

	TConstArrayView<uint32> GetSimIndices(const UChaosClothAsset& ClothAsset, int32 LodIndex)
	{
		const TSharedPtr<const FChaosClothSimulationModel> ClothSimulationModel = ClothAsset.GetClothSimulationModel();
		return ClothSimulationModel && ClothSimulationModel->IsValidLodIndex(LodIndex) ?
			ClothSimulationModel->GetIndices(LodIndex) :
			TConstArrayView<uint32>();
	}

	int32 ConformSkinWeightsToMaxInfluences(
		const TArray<int32>& InBoneIndices,
		const TArray<float>& InBoneWeights,
		TArray<int32>& OutBoneIndices,
		TArray<float>& OutBoneWeights,
		int32 MaxInfluences)
	{
		check(InBoneIndices.Num() == InBoneWeights.Num());
		const int32 NumInfluences = InBoneIndices.Num();

		// Sort the influences by bone weight
		TArray<int32, TInlineAllocator<MAX_TOTAL_INFLUENCES>> SortedInfluences;
		SortedInfluences.Reserve(NumInfluences);
		for (int32 Index = 0; Index < NumInfluences; ++Index)
		{
			SortedInfluences.Add(Index);
		}
		SortedInfluences.Sort([&InBoneWeights](int32 Index0, int32 Index1) { return InBoneWeights[Index0] > InBoneWeights[Index1]; });

		// Copy the weights by order of the most influential to the less
		OutBoneIndices.Reset(MAX_TOTAL_INFLUENCES);
		OutBoneWeights.Reset(MAX_TOTAL_INFLUENCES);

		float BoneWeightsSum = 0.f;
		
		for (int32 Index = 0; Index < FMath::Min(NumInfluences, MaxInfluences); ++Index)
		{
			const float BoneWeight = InBoneWeights[SortedInfluences[Index]];

			if (!FMath::IsNearlyZero(BoneWeight))
			{
				OutBoneIndices.Add(InBoneIndices[SortedInfluences[Index]]);
				OutBoneWeights.Add(BoneWeight);
				BoneWeightsSum += BoneWeight;
			}
			else
			{
				break;  // Found zero weights, early exit
			}
		}
		check(OutBoneIndices.Num() == OutBoneWeights.Num());

		if (BoneWeightsSum != 0.f && !FMath::IsNearlyEqual(BoneWeightsSum, 1.f))
		{
			const float BoneWeightsSumRecip = 1.f / BoneWeightsSum;
			for (float& OutBoneWeight : OutBoneWeights)
			{
				OutBoneWeight *= BoneWeightsSumRecip;
			}
		}

		return OutBoneWeights.Num();
	}
}  // End namespace UE::Chaos::ClothAsset::Private

void UChaosClothAsset::FBuilder::BuildLod(FSkeletalMeshLODModel& LODModel, const UChaosClothAsset& ClothAsset, int32 LodIndex, const ITargetPlatform* TargetPlatform)
{
	using namespace UE::Chaos::ClothAsset;
	using namespace ::Chaos::Softs;

	check(TargetPlatform);

	// Start from an empty LODModel
	LODModel.Empty();

	// Clear the mesh infos, none are stored on this asset
	LODModel.MaxImportVertex = 0;

	// Set 1 texture coordinate
	LODModel.NumTexCoords = 1;

	// Init the size of the vertex buffer
	LODModel.NumVertices = 0;

	// Offset to remap the LOD materials to the asset materials
	int32 MaterialOffset = 0;
	for (int32 CollectionIndex = 0; CollectionIndex < LodIndex; ++CollectionIndex)
	{
		const TSharedRef<const FManagedArrayCollection> ClothCollection = ClothAsset.GetClothCollections()[CollectionIndex];
		const FCollectionClothConstFacade ClothFacade(ClothCollection);
		MaterialOffset += ClothFacade.GetNumRenderPatterns();
	}

	const TSharedRef<const FManagedArrayCollection> ClothCollection = ClothAsset.GetClothCollections()[LodIndex];

	const FCollectionClothConstFacade ClothFacade(ClothCollection);

	// Load the mesh utilities module used to optimized the index buffer
	IMeshUtilities& MeshUtilities = FModuleManager::Get().LoadModuleChecked<IMeshUtilities>("MeshUtilities");

	// Build the sim mesh descriptor for creation of the sections' mesh to mesh mapping data
	const ClothingMeshUtils::ClothMeshDesc SourceMesh(
		Private::GetSimPositions(ClothAsset, LodIndex),
		Private::GetSimIndices(ClothAsset, LodIndex));  // Let it calculate the averaged normals as to match the simulation data output
	const bool bIsValidSourceMesh = (SourceMesh.GetPositions().Num() > 0);

	const int32 NumLodSimVertices = Private::GetNumVertices(ClothAsset, LodIndex);

	// Retrieve MaxDistance information (weight map and Low/High values)
	FCollectionPropertyConstFacade Properties(ClothCollection);
	const FPointWeightMap MaxDistances = FClothEngineTools::GetMaxDistanceWeightMap(ClothFacade, Properties, NumLodSimVertices);

	const int32 NumRenderVertices = ClothFacade.GetNumRenderVertices();
	LODModel.MeshToImportVertexMap.Reserve(NumRenderVertices);

	// Populate this LOD's sections and the LOD index buffer
	const int32 NumSections = ClothFacade.GetNumRenderPatterns();  // Cloth Render Patterns == Skeletal Mesh Sections
	LODModel.Sections.SetNum(NumSections);

	FScopedSlowTask SlowTask(0.0f, LOCTEXT("ClothAssetBuildLOD", "Building Cloth Asset LOD sections..."));
	SlowTask.MakeDialogDelayed(1.f);

	const TConstArrayView<int32> RecomputeTangent = ClothFacade.GetUserDefinedAttribute<int32>(TEXT("RecomputeTangents"), ClothCollectionGroup::RenderPatterns);

	TArray<UE::Tasks::FTask> PendingTasks;
	PendingTasks.Reserve(NumSections);
	int32 BaseIndex = 0;

	TArray<TSet<FBoneIndexType>> ActiveBoneIndicesArray;
	// Require at least one active bone indices set to merge all batches set and include the exta bones indices at the end.
	ActiveBoneIndicesArray.SetNum(FMath::Max(NumSections, 1));

	// Struct to store FSkeletalMeshLODModel data by thread
	struct FSkeletalMeshLODModelData
	{
		TArray<uint32> IndexBuffer;
		TArray<int32> MeshToImportVertexMap;
		int32 MaxImportVertex;
		uint32 NumTexCoords;
	};

	TArray <FSkeletalMeshLODModelData> LODModelDatas;
	LODModelDatas.SetNum(NumSections);

	// Mutex to protect the source mesh which has mutable attribute which are modified
	UE::FMutex SourceMeshMutex;
	for (int32 SectionIndex = 0; SectionIndex < NumSections; ++SectionIndex)
	{
		// Keep track of the active bone indices for this LOD model
		TSet<FBoneIndexType>& ActiveBoneIndices = ActiveBoneIndicesArray[SectionIndex];
		ActiveBoneIndices.Reserve(ClothAsset.GetRefSkeleton().GetNum());

		FCollectionClothRenderPatternConstFacade RenderPatternFacade = ClothFacade.GetRenderPattern(SectionIndex);
		FSkelMeshSection& Section = LODModel.Sections[SectionIndex];
		Section.OriginalDataSectionIndex = SectionIndex;
		const int32 NumVertices = RenderPatternFacade.GetNumRenderVertices();
		const int32 NumFaces = RenderPatternFacade.GetNumRenderFaces();
		const int32 NumIndices = NumFaces * 3;

		Section.BaseIndex = (uint32)BaseIndex;
		BaseIndex += NumIndices;
		Section.BaseVertexIndex = LODModel.NumVertices;
		LODModel.NumVertices += (uint32)NumVertices;

		FSkeletalMeshLODModelData& LODModelData = LODModelDatas[SectionIndex];

		UE::Tasks::FTask PendingTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, 
			// Instance data per thread, data that are not shared through different threads
			[SectionIndex, &Section, NumFaces, NumIndices, &LODModelData, &ActiveBoneIndices, 
			// Const data that should not be modified or protected data by mutex.
			&ClothFacade, MaterialOffset, &ClothAsset, &RecomputeTangent, &MeshUtilities, bIsValidSourceMesh, &SourceMesh, &SourceMeshMutex, TargetPlatform, LodIndex, MaxDistances]() 
		{
			FCollectionClothRenderPatternConstFacade RenderPatternFacade = ClothFacade.GetRenderPattern(SectionIndex);

			const int32 MaterialIndex = MaterialOffset + SectionIndex;
			if (!ClothAsset.GetMaterials().IsValidIndex(MaterialIndex))
			{
				const FString MaterialPath = RenderPatternFacade.GetRenderMaterialSoftObjectPathName().ToString();
				UE_LOGF(LogChaosClothAsset, Warning, "Cloth Asset BuildLod: Pattern %d in LOD %d has invalid material index. Material index is %d, asset has %d materials. Expected material path for this pattern: %ls",
					SectionIndex, LodIndex, MaterialIndex, ClothAsset.GetMaterials().Num(), *MaterialPath);
			}

			// Build the section face data (indices)
			Section.MaterialIndex = (uint16)MaterialIndex;

			Section.bRecomputeTangent = RecomputeTangent.IsValidIndex(MaterialIndex) && RecomputeTangent[MaterialIndex];
			Section.RecomputeTangentsVertexMaskChannel = ESkinVertexColorChannel::None;

			Section.NumTriangles = (uint32)NumFaces;

			TArray<uint32> Indices;
			Indices.SetNumUninitialized(NumIndices);

			const TConstArrayView<FIntVector3> PatternRenderIndices = RenderPatternFacade.GetRenderIndices();

			for (int32 FaceIndex = 0; FaceIndex < NumFaces; ++FaceIndex)
			{
				const FIntVector3& RenderIndices = PatternRenderIndices[FaceIndex];
				for (int32 VertexIndex = 0; VertexIndex < 3; ++VertexIndex)
				{
					const int32 RenderIndex = RenderIndices[VertexIndex];
					Indices[FaceIndex * 3 + VertexIndex] = (uint32)RenderIndex;
				}
			}

			MeshUtilities.CacheOptimizeIndexBuffer(Indices);

			LODModelData.IndexBuffer.Append(MoveTemp(Indices));

			// Build the section vertex data 
			const int32 NumVertices = RenderPatternFacade.GetNumRenderVertices();

			Section.SoftVertices.SetNumUninitialized(NumVertices);
			Section.NumVertices = NumVertices;

			const int32 PatternVertexOffset = RenderPatternFacade.GetRenderVerticesOffset();
			checkSlow(Section.BaseVertexIndex == PatternVertexOffset); // Otherwise Section's Indices don't match RenderPattern indices

			// Map reference skeleton bone index to the index in the section's bone map
			TMap<FBoneIndexType, FBoneIndexType> ReferenceToSectionBoneMap;

			// Track how many bones we added to the section's bone map so far
			int CurSectionBoneMapNum = 0;

			const TConstArrayView<FVector3f> PatternRenderPosition = RenderPatternFacade.GetRenderPosition();
			const TConstArrayView<FVector3f> PatternRenderTangentU = RenderPatternFacade.GetRenderTangentU();
			const TConstArrayView<FVector3f> PatternRenderTangentV = RenderPatternFacade.GetRenderTangentV();
			const TConstArrayView<FVector3f> PatternRenderNormal = RenderPatternFacade.GetRenderNormal();
			const TConstArrayView<FLinearColor> PatternRenderColor = RenderPatternFacade.GetRenderColor();
			const TConstArrayView<TArray<FVector2f>> PatternRenderUVs = RenderPatternFacade.GetRenderUVs();
			const TConstArrayView<TArray<int32>> PatternRenderBoneIndices = RenderPatternFacade.GetRenderBoneIndices();
			const TConstArrayView<TArray<float>> PatternRenderBoneWeights = RenderPatternFacade.GetRenderBoneWeights();

			LODModelData.NumTexCoords = 1;
			for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
			{
				// Save the original indices for the newly added vertices
				LODModelData.MeshToImportVertexMap.Add(VertexIndex + PatternVertexOffset);
				LODModelData.MaxImportVertex = VertexIndex + PatternVertexOffset;

				FSoftSkinVertex& SoftVertex = Section.SoftVertices[VertexIndex];

				SoftVertex.Position = PatternRenderPosition[VertexIndex];
				SoftVertex.TangentX = PatternRenderTangentU[VertexIndex];
				SoftVertex.TangentY = PatternRenderTangentV[VertexIndex];
				SoftVertex.TangentZ = PatternRenderNormal[VertexIndex];

				constexpr bool bSRGB = false; // Avoid linear to srgb conversion
				SoftVertex.Color = PatternRenderColor[VertexIndex].ToFColor(bSRGB);

				const TArray<FVector2f>& RenderUVs = PatternRenderUVs[VertexIndex];
				FMemory::Memzero(SoftVertex.UVs, sizeof(SoftVertex.UVs));
				const int32 NumTexCoords = FMath::Min(RenderUVs.Num(), (int32)MAX_TEXCOORDS);
				for (int32 TexCoord = 0; TexCoord < NumTexCoords; ++TexCoord)
				{
					SoftVertex.UVs[TexCoord] = RenderUVs[TexCoord];
				}
				LODModelData.NumTexCoords = FMath::Max(LODModelData.NumTexCoords, (uint32)NumTexCoords);

				// Conform to the platform max number of influences
				const int32 MaxBoneInfluencesFromUnlimitedBoneInfluences = FGPUBaseSkinVertexFactory::GetUnlimitedBoneInfluences(TargetPlatform) ? MAX_TOTAL_INFLUENCES : EXTRA_BONE_INFLUENCES;
				const int32 MaxBoneInfluencesFromPlatformProjectSettings = FGPUBaseSkinVertexFactory::GetBoneInfluenceLimitForAsset(0, TargetPlatform);
				const int32 MaxNumInfluences = FMath::Min(MaxBoneInfluencesFromUnlimitedBoneInfluences, MaxBoneInfluencesFromPlatformProjectSettings);

				TArray<int32> BoneIndices;
				TArray<float> BoneWeights;
				const int32 NumInfluences = Private::ConformSkinWeightsToMaxInfluences(
					PatternRenderBoneIndices[VertexIndex],
					PatternRenderBoneWeights[VertexIndex],
					BoneIndices,
					BoneWeights,
					MaxNumInfluences);

				// Add all of the bones that have non-zero influence to the section's bone map and keep track of the order
				// that we added the reference bone via CurSectionBoneMapNum
				for (int32 Influence = 0; Influence < NumInfluences; ++Influence)
				{
					const FBoneIndexType InfluenceBone = (FBoneIndexType)BoneIndices[Influence];

					if (ReferenceToSectionBoneMap.Contains(InfluenceBone) == false)
					{
						ReferenceToSectionBoneMap.Add(InfluenceBone, CurSectionBoneMapNum);
						++CurSectionBoneMapNum;
					}
				}

				int32 Influence = 0;
				for (; Influence < NumInfluences; ++Influence)
				{
					const FBoneIndexType InfluenceBone = (FBoneIndexType)BoneIndices[Influence];
					const float InWeight = BoneWeights[Influence];
					const uint16 InfluenceWeight = static_cast<uint16>(InWeight * static_cast<float>(UE::AnimationCore::MaxRawBoneWeight) + 0.5f);

					// FSoftSkinVertex::InfluenceBones contain indices into the section's bone map and not the reference
					// skeleton, so we need to remap
					const FBoneIndexType* const MappedIndexPtr = ReferenceToSectionBoneMap.Find(InfluenceBone);

					// ReferenceToSectionBoneMap should always contain InfluenceBone since it was added above
					checkSlow(MappedIndexPtr);
					if (MappedIndexPtr != nullptr)
					{
						SoftVertex.InfluenceBones[Influence] = *MappedIndexPtr;
						SoftVertex.InfluenceWeights[Influence] = InfluenceWeight;
					}
				}

				for (; Influence < MAX_TOTAL_INFLUENCES; ++Influence)
				{
					SoftVertex.InfluenceBones[Influence] = 0;
					SoftVertex.InfluenceWeights[Influence] = 0;
				}
			}

			// Initialize the section bone map
			Section.BoneMap.SetNumUninitialized(ReferenceToSectionBoneMap.Num());
			for (const TPair<FBoneIndexType, FBoneIndexType>& Pair : ReferenceToSectionBoneMap)
			{
				Section.BoneMap[Pair.Value] = Pair.Key;
			}

			ActiveBoneIndices.Append(Section.BoneMap);

			// Update max bone influences
			Section.CalcMaxBoneInfluences();
			Section.CalcUse16BitBoneIndex();

			// Setup clothing data
			if (bIsValidSourceMesh)
			{
				Section.ClothMappingDataLODs.SetNum(1);  // TODO: LODBias maps for raytracing

				Section.ClothingData.AssetLodIndex = LodIndex;
				Section.ClothingData.AssetGuid = ClothAsset.AssetGuid;  // There is only one cloth asset,
				Section.CorrespondClothAssetIndex = 0;       // this one

				const int32 RenderDeformerNumInfluences = RenderPatternFacade.GetRenderDeformerNumInfluences();
				if (RenderDeformerNumInfluences > 0)
				{
					TArray<FMeshToMeshVertData>& MeshToMeshVertData = Section.ClothMappingDataLODs[0];
					MeshToMeshVertData.SetNumUninitialized(RenderDeformerNumInfluences * NumVertices);

					const TConstArrayView<TArray<FVector4f>> RenderDeformerPositionBaryCoordsAndDist = RenderPatternFacade.GetRenderDeformerPositionBaryCoordsAndDist();
					const TConstArrayView<TArray<FVector4f>> RenderDeformerNormalBaryCoordsAndDist = RenderPatternFacade.GetRenderDeformerNormalBaryCoordsAndDist();
					const TConstArrayView<TArray<FVector4f>> RenderDeformerTangentBaryCoordsAndDist = RenderPatternFacade.GetRenderDeformerTangentBaryCoordsAndDist();
					const TConstArrayView<TArray<FIntVector3>> RenderDeformerSimIndices3D = RenderPatternFacade.GetRenderDeformerSimIndices3D();
					const TConstArrayView<TArray<float>> RenderDeformerWeight = RenderPatternFacade.GetRenderDeformerWeight();
					const TConstArrayView<float> RenderDeformerSkinningBlend = RenderPatternFacade.GetRenderDeformerSkinningBlend();

					for (int32 Index = 0; Index < NumVertices; ++Index)
					{
						const uint16 SkinningBlend = (uint16)(FMath::Clamp(RenderDeformerSkinningBlend[Index], 0.f, 1.f) * (float)TNumericLimits<uint16>::Max());

						check(RenderDeformerPositionBaryCoordsAndDist[Index].Num() == RenderDeformerNumInfluences);
						check(RenderDeformerNormalBaryCoordsAndDist[Index].Num() == RenderDeformerNumInfluences);
						check(RenderDeformerTangentBaryCoordsAndDist[Index].Num() == RenderDeformerNumInfluences);
						check(RenderDeformerSimIndices3D[Index].Num() == RenderDeformerNumInfluences);
						check(RenderDeformerWeight[Index].Num() == RenderDeformerNumInfluences);

						for (int32 Influence = 0; Influence < RenderDeformerNumInfluences; ++Influence)
						{
							const int32 InfluenceIndex = Index * RenderDeformerNumInfluences + Influence;

							MeshToMeshVertData[InfluenceIndex].PositionBaryCoordsAndDist = RenderDeformerPositionBaryCoordsAndDist[Index][Influence];
							MeshToMeshVertData[InfluenceIndex].NormalBaryCoordsAndDist = RenderDeformerNormalBaryCoordsAndDist[Index][Influence];
							MeshToMeshVertData[InfluenceIndex].TangentBaryCoordsAndDist = RenderDeformerTangentBaryCoordsAndDist[Index][Influence];
							MeshToMeshVertData[InfluenceIndex].Weight = RenderDeformerWeight[Index][Influence];
							const FIntVector3& SimIndices3D = RenderDeformerSimIndices3D[Index][Influence];
							MeshToMeshVertData[InfluenceIndex].SourceMeshVertIndices[0] = (uint16)SimIndices3D[0];
							MeshToMeshVertData[InfluenceIndex].SourceMeshVertIndices[1] = (uint16)SimIndices3D[1];
							MeshToMeshVertData[InfluenceIndex].SourceMeshVertIndices[2] = (uint16)SimIndices3D[2];
							MeshToMeshVertData[InfluenceIndex].SourceMeshVertIndices[3] = SkinningBlend;
						}
					}
				}
				else
				{
					// No mapping found in the collection, create them from scratch
					TArray<FVector3f> SectionRenderPositions;
					TArray<FVector3f> SectionRenderNormals;
					TArray<FVector3f> SectionRenderTangents;
					SectionRenderPositions.Reserve(NumVertices);
					SectionRenderNormals.Reserve(NumVertices);
					SectionRenderTangents.Reserve(NumVertices);
					for (const FSoftSkinVertex& SoftVert : Section.SoftVertices)
					{
						SectionRenderPositions.Add(SoftVert.Position);
						SectionRenderNormals.Add(SoftVert.TangentZ);
						SectionRenderTangents.Add(SoftVert.TangentX);
					}
					TArray<uint32> SectionRenderIndices;
					SectionRenderIndices.Reserve(NumIndices);
					const TArrayView<uint32> SectionIndexBuffer(LODModelData.IndexBuffer.GetData(), NumIndices);
					for (const uint32 LodModelVertIndex : SectionIndexBuffer)
					{
						SectionRenderIndices.Add(LodModelVertIndex - Section.BaseVertexIndex);
					}

					const ClothingMeshUtils::ClothMeshDesc TargetMesh(
						SectionRenderPositions,
						SectionRenderNormals,
						SectionRenderTangents,
						SectionRenderIndices);

					// SourceMesh has mutable AAbbtree which is modified so this function cannot be multi threaded
					SourceMeshMutex.Lock();
					PRAGMA_DISABLE_DEPRECATION_WARNINGS
						ClothingMeshUtils::GenerateMeshToMeshVertData(
							Section.ClothMappingDataLODs[0],
							TargetMesh,
							SourceMesh,
							&MaxDistances,
							ClothAsset.bSmoothTransition_DEPRECATED,
							ClothAsset.bUseMultipleInfluences_DEPRECATED,
							ClothAsset.SkinningKernelRadius_DEPRECATED);
					PRAGMA_ENABLE_DEPRECATION_WARNINGS
					SourceMeshMutex.Unlock();
				}
			}

			// Compute the overlapping vertices map (inspired from MeshUtilities::BuildSkeletalMesh)
			const TArray<FSoftSkinVertex>& SoftVertices = Section.SoftVertices;

			typedef TPair<float, int32> FIndexAndZ;  // Acceleration structure, list of vertex Z / index pairs
			TArray<FIndexAndZ> IndexAndZs;
			IndexAndZs.Reserve(NumVertices);
			for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
			{
				const FVector3f& Position = SoftVertices[VertexIndex].Position;

				const float Z = 0.30f * Position.X + 0.33f * Position.Y + 0.37f * Position.Z;
				IndexAndZs.Emplace(Z, VertexIndex);
			}
			IndexAndZs.Sort([](const FIndexAndZ& A, const FIndexAndZ& B) { return A.Key < B.Key; });

			for (int32 Index0 = 0; Index0 < IndexAndZs.Num(); ++Index0)
			{
				const float Z0 = IndexAndZs[Index0].Key;
				const uint32 VertexIndex0 = IndexAndZs[Index0].Value;
				const FVector3f& Position0 = SoftVertices[VertexIndex0].Position;

				// Only need to search forward, since we add pairs both ways
				for (int32 Index1 = Index0 + 1; Index1 < IndexAndZs.Num() && FMath::Abs(IndexAndZs[Index1].Key - Z0) <= THRESH_POINTS_ARE_SAME; ++Index1)
				{
					const uint32 VertexIndex1 = IndexAndZs[Index1].Value;
					const FVector3f& Position1 = SoftVertices[VertexIndex1].Position;

					if (PointsEqual(Position0, Position1))
					{
						// Add to the overlapping map
						TArray<int32>& SrcValueArray = Section.OverlappingVertices.FindOrAdd(VertexIndex0);
						SrcValueArray.Add(VertexIndex1);

						TArray<int32>& IterValueArray = Section.OverlappingVertices.FindOrAdd(VertexIndex1);
						IterValueArray.Add(VertexIndex0);
					}
				}
			}
		});

		PendingTasks.Add(PendingTask);
	}
	UE::Tasks::Wait(PendingTasks);

	for (int32 SectionIndex = 0; SectionIndex < NumSections; ++SectionIndex)
	{
		FSkeletalMeshLODModelData& LODModelData = LODModelDatas[SectionIndex];
		LODModel.IndexBuffer.Append(MoveTemp(LODModelData.IndexBuffer));
		LODModel.NumTexCoords = FMath::Max(LODModel.NumTexCoords, LODModelData.NumTexCoords);
		LODModel.MeshToImportVertexMap.Append(MoveTemp(LODModelData.MeshToImportVertexMap));
		LODModel.MaxImportVertex = LODModelData.MaxImportVertex;
	
		FSkelMeshSection& Section = LODModel.Sections[SectionIndex];
		// Must be in single thread
		// Copy to user section data, otherwise the section data set above would get lost when the user section gets synced
		FSkelMeshSourceSectionUserData::GetSourceSectionUserData(LODModel.UserSectionsData, Section);

	}

	for (int32 SectionIndex = 1; SectionIndex < NumSections; ++SectionIndex)
	{
		ActiveBoneIndicesArray[0].Append(MoveTemp(ActiveBoneIndicesArray[SectionIndex]));
	}

		// Update the active bone indices on the LOD model
		LODModel.ActiveBoneIndices = ActiveBoneIndicesArray[0].Array();

	// Ensure parent exists with incoming active bone indices, and the result should be sorted
	ClothAsset.GetRefSkeleton().EnsureParentsExistAndSort(LODModel.ActiveBoneIndices);
	LODModel.ActiveBoneIndices.Shrink();

	// Add the extra simulation bones to the list of required bones if any
	LODModel.RequiredBones = LODModel.ActiveBoneIndices;

	if (const TSharedPtr<const FChaosClothSimulationModel> ClothSimulationModel = ClothAsset.GetClothSimulationModel())
	{
		if (ensure(ClothSimulationModel->IsValidLodIndex(LodIndex)))
		{
			ActiveBoneIndicesArray[0].Append(ClothSimulationModel->ClothSimulationLodModels[LodIndex].RequiredExtraBoneIndices);

			LODModel.RequiredBones = ActiveBoneIndicesArray[0].Array();

			ClothAsset.GetRefSkeleton().EnsureParentsExistAndSort(LODModel.RequiredBones);
			LODModel.RequiredBones.Shrink();
		}
	}
}

#undef LOCTEXT_NAMESPACE
#endif  // #if WITH_EDITOR
