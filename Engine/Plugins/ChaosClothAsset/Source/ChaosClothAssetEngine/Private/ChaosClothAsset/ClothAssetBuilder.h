// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "ChaosClothAsset/ClothAsset.h"

class FSkeletalMeshLODModel;

/** Builder utility struct, nested struct of UChaosClothAsset. */
struct UChaosClothAsset::FBuilder
{
	/** Build a FSkeletalMeshLODModel out of the cloth asset for the specified LOD index and specified platform. */
	static void BuildLod(FSkeletalMeshLODModel& LODModel, const UChaosClothAsset& ClothAsset, int32 LodIndex, const ITargetPlatform* TargetPlatform);
};

#endif  // #if WITH_EDITOR
