// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ClothAssetInteractor.h"
#include "ChaosClothAsset/ClothAssetInteractorDataAsset.h"
#include "Chaos/CollectionPropertyFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClothAssetInteractor)

namespace UE::Chaos::ClothAsset::Private
{
	// Put aliases in the same order as how the if/else works in FClothConstraints::Create[Type]Constraints to ensure
	// the property that is actually used by the solver is used here.

	// SimulationBendingConfigNode
	static const TArray<FName> BendingStiffnessWarpAliases =
	{
		TEXT("XPBDAnisoBendingStiffnessWarp"),
	};
	static const TArray<FName> BendingStiffnessWeftAliases =
	{
		TEXT("XPBDAnisoBendingStiffnessWeft"),
	};
	static const TArray<FName> BendingStiffnessBiasAliases =
	{
		TEXT("XPBDAnisoBendingStiffnessBias"),
	};
	static const TArray<FName> BendingDampingAliases =
	{
		TEXT("XPBDAnisoBendingDamping"),
		TEXT("XPBDBendingElementDamping"),
		TEXT("XPBDBendingSpringDamping"),
	};
	static const TArray<FName> BucklingRatioAliases =
	{
		TEXT("XPBDAnisoBucklingRatio"),
		TEXT("XPBDBucklingRatio"),
		TEXT("BucklingRatio"),
	};
	static const TArray<FName> BucklingStiffnessWarpAliases =
	{
		TEXT("XPBDAnisoBucklingStiffnessWarp"),
	};
	static const TArray<FName> BucklingStiffnessWeftAliases =
	{
		TEXT("XPBDAnisoBucklingStiffnessWeft"),
	};
	static const TArray<FName> BucklingStiffnessBiasAliases =
	{
		TEXT("XPBDAnisoBucklingStiffnessBias"),
	};
	static const TArray<FName> BendingStiffnessAliases =
	{
		TEXT("XPBDBendingElementStiffness"),
		TEXT("BendingElementStiffness"),
		TEXT("XPBDBendingSpringStiffness"),
		TEXT("BendingSpringStiffness"),
	};
	static const TArray<FName> BucklingStiffnessAliases =
	{
		TEXT("XPBDBucklingStiffness"),
		TEXT("BucklingStiffness"),
	};

	// SimulationStretchConfigNode
	static const TArray<FName> StretchStiffnessWarpAliases =
	{
		TEXT("XPBDAnisoStretchStiffnessWarp"),
		TEXT("XPBDAnisoSpringStiffnessWarp"),
	};
	static const TArray<FName> StretchStiffnessWeftAliases =
	{
		TEXT("XPBDAnisoStretchStiffnessWeft"),
		TEXT("XPBDAnisoSpringStiffnessWeft"),
	};
	static const TArray<FName> StretchStiffnessBiasAliases =
	{
		TEXT("XPBDAnisoStretchStiffnessBias"),
		TEXT("XPBDAnisoSpringStiffnessBias"),
	};
	static const TArray<FName> StretchDampingAliases =
	{
		TEXT("XPBDAnisoStretchDamping"),
		TEXT("XPBDEdgeSpringDamping"),
		TEXT("XPBDAnisoSpringDamping"),
	};
	static const TArray<FName> StretchStiffnessAliases =
	{
		TEXT("XPBDEdgeSpringStiffness"),
		TEXT("EdgeSpringStiffness"),
	};
	static const TArray<FName> StretchWarpScaleAliases =
	{
		TEXT("XPBDAnisoStretchWarpScale"),
		TEXT("XPBDAnisoSpringWarpScale"),
		TEXT("EdgeSpringWarpScale"),
		TEXT("AreaSpringWarpScale"),
	};
	static const TArray<FName> StretchWeftScaleAliases =
	{
		TEXT("XPBDAnisoStretchWeftScale"),
		TEXT("XPBDAnisoSpringWeftScale"),
		TEXT("EdgeSpringWeftScale"),
		TEXT("AreaSpringWeftScale"),
	};
	static const TArray<FName> AreaStiffnessAliases =
	{
		TEXT("XPBDAreaSpringStiffness"),
		TEXT("AreaSpringStiffness"),
	};

	// SimulationDampingConfigNode
	static const TArray<FName> LocalDampingCoefficientAliases = // For backwards compatibility--single LocalDampingCoefficient has been split into linear and angular parts.
	{
		TEXT("LocalDampingCoefficient"),
		TEXT("LocalDampingLinearCoefficient"),
		TEXT("LocalDampingAngularCoefficient"),
	};
	

	static const TMap<FName, TArray<FName>> Aliases =
	{
		{TEXT("BendingStiffnessWarp"), BendingStiffnessWarpAliases},
		{TEXT("BendingStiffnessWeft"), BendingStiffnessWeftAliases},
		{TEXT("BendingStiffnessBias"), BendingStiffnessBiasAliases},
		{TEXT("BendingDamping"), BendingDampingAliases},
		{TEXT("BucklingRatio"), BucklingRatioAliases},
		{TEXT("BucklingStiffnessWarp"), BucklingStiffnessWarpAliases},
		{TEXT("BucklingStiffnessWeft"), BucklingStiffnessWeftAliases},
		{TEXT("BucklingStiffnessBias"), BucklingStiffnessBiasAliases},
		{TEXT("BendingStiffness"), BendingStiffnessAliases},
		{TEXT("BucklingStiffness"), BucklingStiffnessAliases},
		{TEXT("StretchStiffnessWarp"), StretchStiffnessWarpAliases},
		{TEXT("StretchStiffnessWeft"), StretchStiffnessWeftAliases},
		{TEXT("StretchStiffnessBias"), StretchStiffnessBiasAliases},
		{TEXT("StretchDamping"), StretchDampingAliases},
		{TEXT("StretchStiffness"), StretchStiffnessAliases},
		{TEXT("StretchWarpScale"), StretchWarpScaleAliases},
		{TEXT("StretchWeftScale"), StretchWeftScaleAliases},
		{TEXT("AreaStiffness"), AreaStiffnessAliases},
		{TEXT("LocalDampingCoefficient"), LocalDampingCoefficientAliases},
	};


	template<typename T>
	static T GetValueWithAlias(TSharedPtr<::Chaos::Softs::FCollectionPropertyFacade>& PropertyFacade, const FName PropertyName, const T& DefaultValue, const TFunctionRef<T(int32 KeyIndex)>& GetValue)
	{
		check(PropertyFacade);
		{
			const int32 KeyIndex = PropertyFacade->GetKeyNameIndex(PropertyName);
			if (KeyIndex != INDEX_NONE)
			{
				return GetValue(KeyIndex);
			}
		}

		if (const TArray<FName>* FoundAliases = Aliases.Find(PropertyName))
		{
			for (const FName& FoundAlias : *FoundAliases)
			{
				const int32 KeyIndex = PropertyFacade->GetKeyNameIndex(FoundAlias);
				if (KeyIndex != INDEX_NONE)
				{
					return GetValue(KeyIndex);
				}
			}
		}

		return DefaultValue;
	}

	void SetValueWithAlias(TSharedPtr<::Chaos::Softs::FCollectionPropertyFacade>& PropertyFacade, const FName PropertyName, const TFunctionRef<void(const FName&)>& SetValue)
	{
		check(PropertyFacade);
		if (const TArray<FName>* FoundAliases = Aliases.Find(PropertyName))
		{
			for (const FName& FoundAlias : *FoundAliases)
			{
				SetValue(FoundAlias);
			}
		}
		else
		{
			SetValue(PropertyName);
		}
	}
}

void UChaosClothAssetInteractor::SetProperties(const TArray<TSharedPtr<::Chaos::Softs::FCollectionPropertyFacade>>& InCollectionPropertyFacades)
{
	CollectionPropertyFacades.Reset(InCollectionPropertyFacades.Num());
	for (const TSharedPtr<::Chaos::Softs::FCollectionPropertyFacade>& InFacade : InCollectionPropertyFacades)
	{
		CollectionPropertyFacades.Emplace(InFacade);
	}
}

void UChaosClothAssetInteractor::ResetProperties()
{
	CollectionPropertyFacades.Reset();
}

TArray<FName> UChaosClothAssetInteractor::GetAllPropertyNames(int32 LODIndex) const
{
	using namespace UE::Chaos::ClothAsset::Private;

	TSet<FName> Keys;
	if (LODIndex == INDEX_NONE)
	{
		for (int32 Index = 0; Index < CollectionPropertyFacades.Num(); ++Index)
		{
			if (TSharedPtr<::Chaos::Softs::FCollectionPropertyFacade> PropertyFacade = CollectionPropertyFacades[Index].Pin())
			{
				if (Keys.IsEmpty())
				{
					// This is the first non-empty LOD. We can add all keys without worrying about uniqueness.
					Keys.Reserve(PropertyFacade->Num() + Aliases.Num());
					for (int32 KeyIndex = 0; KeyIndex < PropertyFacade->Num(); ++KeyIndex)
					{
						Keys.Add(PropertyFacade->GetKeyName(KeyIndex));
					}
				}
				else
				{
					for (int32 KeyIndex = 0; KeyIndex < PropertyFacade->Num(); ++KeyIndex)
					{
						Keys.Add(PropertyFacade->GetKeyName(KeyIndex));
					}
				}
			}
		}
	}
	else
	{
		if (CollectionPropertyFacades.IsValidIndex(LODIndex))
		{
			if (TSharedPtr<::Chaos::Softs::FCollectionPropertyFacade> PropertyFacade = CollectionPropertyFacades[LODIndex].Pin())
			{
				Keys.Reserve(PropertyFacade->Num());
				for (int32 KeyIndex = 0; KeyIndex < PropertyFacade->Num(); ++KeyIndex)
				{
					Keys.Add(PropertyFacade->GetKeyName(KeyIndex));
				}
			}
		}
	}

	for (TMap<FName, TArray<FName>>::TConstIterator AliasIter = Aliases.CreateConstIterator(); AliasIter; ++AliasIter)
	{
		for (const FName& OtherName : AliasIter.Value())
		{
			if (Keys.Contains(OtherName))
			{
				Keys.Add(AliasIter.Key());
			}
		}
	}
	return Keys.Array();
}

TArray<FString> UChaosClothAssetInteractor::GetAllProperties(int32 LODIndex) const
{
	using namespace UE::Chaos::ClothAsset::Private;
	const TArray<FName> PropertyNames = GetAllPropertyNames(LODIndex);
	TArray<FString> Result;
	Result.Reserve(PropertyNames.Num());
	for (const FName& Name : PropertyNames)
	{
		Result.Emplace(Name.ToString());
	}
	return Result;
}

float UChaosClothAssetInteractor::GetFloatPropertyValue(const FName PropertyName, int32 LODIndex, float DefaultValue) const
{
	using namespace UE::Chaos::ClothAsset::Private;
	if (CollectionPropertyFacades.IsValidIndex(LODIndex))
	{
		if (TSharedPtr<::Chaos::Softs::FCollectionPropertyFacade> PropertyFacade = CollectionPropertyFacades[LODIndex].Pin())
		{
			return GetValueWithAlias<float>(PropertyFacade, PropertyName, DefaultValue,
				[&PropertyFacade](int32 KeyIndex)
				{
					return PropertyFacade->GetValue<float>(KeyIndex);
				});
		}
	}
	return DefaultValue;
}

float UChaosClothAssetInteractor::GetLowFloatPropertyValue(const FName PropertyName, int32 LODIndex, float DefaultValue) const
{
	using namespace UE::Chaos::ClothAsset::Private;
	if (CollectionPropertyFacades.IsValidIndex(LODIndex))
	{
		if (TSharedPtr<::Chaos::Softs::FCollectionPropertyFacade> PropertyFacade = CollectionPropertyFacades[LODIndex].Pin())
		{
			return GetValueWithAlias<float>(PropertyFacade, PropertyName, DefaultValue,
				[&PropertyFacade](int32 KeyIndex)
				{
					return PropertyFacade->GetLowValue<float>(KeyIndex);
				});
		}
	}
	return DefaultValue;
}

float UChaosClothAssetInteractor::GetHighFloatPropertyValue(const FName PropertyName, int32 LODIndex, float DefaultValue) const
{
	using namespace UE::Chaos::ClothAsset::Private;
	if (CollectionPropertyFacades.IsValidIndex(LODIndex))
	{
		if (TSharedPtr<::Chaos::Softs::FCollectionPropertyFacade> PropertyFacade = CollectionPropertyFacades[LODIndex].Pin())
		{
			return GetValueWithAlias<float>(PropertyFacade, PropertyName, DefaultValue,
				[&PropertyFacade](int32 KeyIndex)
				{
					return PropertyFacade->GetHighValue<float>(KeyIndex);
				});
		}
	}
	return DefaultValue;
}

FVector2D UChaosClothAssetInteractor::GetWeightedFloatPropertyValue(const FName PropertyName, int32 LODIndex, FVector2D DefaultValue) const
{
	using namespace UE::Chaos::ClothAsset::Private;
	if (CollectionPropertyFacades.IsValidIndex(LODIndex))
	{
		if (TSharedPtr<::Chaos::Softs::FCollectionPropertyFacade> PropertyFacade = CollectionPropertyFacades[LODIndex].Pin())
		{
			return GetValueWithAlias<FVector2D>(PropertyFacade, PropertyName, DefaultValue,
				[&PropertyFacade](int32 KeyIndex)
				{
					return FVector2D(PropertyFacade->GetWeightedFloatValue(KeyIndex));
				});
		}
	}
	return DefaultValue;
}

int32 UChaosClothAssetInteractor::GetIntPropertyValue(const FName PropertyName, int32 LODIndex, int32 DefaultValue) const
{
	using namespace UE::Chaos::ClothAsset::Private;
	if (CollectionPropertyFacades.IsValidIndex(LODIndex))
	{
		if (TSharedPtr<::Chaos::Softs::FCollectionPropertyFacade> PropertyFacade = CollectionPropertyFacades[LODIndex].Pin())
		{
			return GetValueWithAlias<int32>(PropertyFacade, PropertyName, DefaultValue,
				[&PropertyFacade](int32 KeyIndex)
				{
					return PropertyFacade->GetValue<int32>(KeyIndex);
				});
		}
	}
	return DefaultValue;
}

FVector UChaosClothAssetInteractor::GetVectorPropertyValue(const FName PropertyName, int32 LODIndex, FVector DefaultValue) const
{
	using namespace UE::Chaos::ClothAsset::Private;
	if (CollectionPropertyFacades.IsValidIndex(LODIndex))
	{
		if (TSharedPtr<::Chaos::Softs::FCollectionPropertyFacade> PropertyFacade = CollectionPropertyFacades[LODIndex].Pin())
		{
			return GetValueWithAlias<FVector>(PropertyFacade, PropertyName, DefaultValue,
				[&PropertyFacade](int32 KeyIndex)
				{
					return FVector(PropertyFacade->GetValue<FVector3f>(KeyIndex));
				});
		}
	}
	return DefaultValue;
}

FString UChaosClothAssetInteractor::GetStringPropertyValue(const FName PropertyName, int32 LODIndex, const FString& DefaultValue) const
{
	using namespace UE::Chaos::ClothAsset::Private;
	if (CollectionPropertyFacades.IsValidIndex(LODIndex))
	{
		if (TSharedPtr<::Chaos::Softs::FCollectionPropertyFacade> PropertyFacade = CollectionPropertyFacades[LODIndex].Pin())
		{
			return GetValueWithAlias<FString>(PropertyFacade, PropertyName, DefaultValue,
				[&PropertyFacade](int32 KeyIndex)
				{
					return PropertyFacade->GetStringValue(KeyIndex);
				});
		}
	}
	return DefaultValue;
}

void UChaosClothAssetInteractor::SetPropertySet(const FClothAssetInteractorPropertyBag& PropertySet, int32 LODIndex)
{
	if (const UPropertyBag* const CurrentBagStruct = PropertySet.GetPropertyBagStruct())
	{
		for (const FPropertyBagPropertyDesc& Desc : CurrentBagStruct->GetPropertyDescs())
		{
			if (Desc.IsNumericFloatType())
			{
				if (TValueOrError<float, EPropertyBagResult> Value = PropertySet.GetValueFloat(Desc); Value.IsValid())
				{
					SetFloatPropertyValue(Desc.Name, LODIndex, Value.GetValue());
				}
			}
			else if (Desc.IsNumericType())
			{
				if (TValueOrError<int32, EPropertyBagResult> Value = PropertySet.GetValueInt32(Desc); Value.IsValid())
				{
					SetIntPropertyValue(Desc.Name, LODIndex, Value.GetValue());
				}
			}
			else if (Desc.ValueType == EPropertyBagPropertyType::String)
			{
				if (TValueOrError<FString, EPropertyBagResult> Value = PropertySet.GetValueString(Desc); Value.IsValid())
				{
					SetStringPropertyValue(Desc.Name, LODIndex, Value.GetValue());
				}
			}
			else if (Desc.ValueType == EPropertyBagPropertyType::Struct)
			{
				if (TValueOrError<FStructView, EPropertyBagResult> VectorValue = PropertySet.GetValueStruct(Desc, TBaseStructure<FVector>::Get()); VectorValue.IsValid())
				{
					// FVector
					const FVector& Vector = VectorValue.GetValue().Get<FVector>();
					SetVectorPropertyValue(Desc.Name, LODIndex, Vector);
				}
				else if (TValueOrError<FStructView, EPropertyBagResult> Vector2DValue = PropertySet.GetValueStruct(Desc, TBaseStructure<FVector2D>::Get()); Vector2DValue.IsValid())
				{
					// FVector2D (WeightedFloat)
					const FVector2D& Vector = Vector2DValue.GetValue().Get<FVector2D>();
					SetWeightedFloatPropertyValue(Desc.Name, LODIndex, Vector);
				}
			}
		}
	}

}

void UChaosClothAssetInteractor::SetFloatPropertyValue(const FName PropertyName, int32 LODIndex, float Value)
{
	using namespace UE::Chaos::ClothAsset::Private;
	if (LODIndex == INDEX_NONE)
	{
		for (int32 Index = 0; Index < CollectionPropertyFacades.Num(); ++Index)
		{
			if (TSharedPtr<::Chaos::Softs::FCollectionPropertyFacade> PropertyFacade = CollectionPropertyFacades[Index].Pin())
			{
				SetValueWithAlias(PropertyFacade, PropertyName,
					[&PropertyFacade, Value](const FName& Name)
					{
						PropertyFacade->SetValue(Name, Value);
					});
			}
		}
	}
	else if (CollectionPropertyFacades.IsValidIndex(LODIndex))
	{
		if (TSharedPtr<::Chaos::Softs::FCollectionPropertyFacade> PropertyFacade = CollectionPropertyFacades[LODIndex].Pin())
		{
			SetValueWithAlias(PropertyFacade, PropertyName,
				[&PropertyFacade, Value](const FName& Name)
				{
					PropertyFacade->SetValue(Name, Value);
				});
		}
	}
}

void UChaosClothAssetInteractor::SetLowFloatPropertyValue(const FName PropertyName, int32 LODIndex, float Value)
{
	using namespace UE::Chaos::ClothAsset::Private;
	if (LODIndex == INDEX_NONE)
	{
		for (int32 Index = 0; Index < CollectionPropertyFacades.Num(); ++Index)
		{
			if (TSharedPtr<::Chaos::Softs::FCollectionPropertyFacade> PropertyFacade = CollectionPropertyFacades[Index].Pin())
			{
				SetValueWithAlias(PropertyFacade, PropertyName,
					[&PropertyFacade, Value](const FName& Name)
					{
						PropertyFacade->SetLowValue(Name, Value);
					});
			}
		}
	}
	else if (CollectionPropertyFacades.IsValidIndex(LODIndex))
	{
		if (TSharedPtr<::Chaos::Softs::FCollectionPropertyFacade> PropertyFacade = CollectionPropertyFacades[LODIndex].Pin())
		{
			SetValueWithAlias(PropertyFacade, PropertyName,
				[&PropertyFacade, Value](const FName& Name)
				{
					PropertyFacade->SetLowValue(Name, Value);
				});
		}
	}
}

void UChaosClothAssetInteractor::SetHighFloatPropertyValue(const FName PropertyName, int32 LODIndex, float Value)
{
	using namespace UE::Chaos::ClothAsset::Private;
	if (LODIndex == INDEX_NONE)
	{
		for (int32 Index = 0; Index < CollectionPropertyFacades.Num(); ++Index)
		{
			if (TSharedPtr<::Chaos::Softs::FCollectionPropertyFacade> PropertyFacade = CollectionPropertyFacades[Index].Pin())
			{
				SetValueWithAlias(PropertyFacade, PropertyName,
					[&PropertyFacade, Value](const FName& Name)
					{
						PropertyFacade->SetHighValue(Name, Value);
					});
			}
		}
	}
	else if (CollectionPropertyFacades.IsValidIndex(LODIndex))
	{
		if (TSharedPtr<::Chaos::Softs::FCollectionPropertyFacade> PropertyFacade = CollectionPropertyFacades[LODIndex].Pin())
		{
			SetValueWithAlias(PropertyFacade, PropertyName,
				[&PropertyFacade, Value](const FName& Name)
				{
					PropertyFacade->SetHighValue(Name, Value);
				});
		}
	}
}

void UChaosClothAssetInteractor::SetWeightedFloatPropertyValue(const FName PropertyName, int32 LODIndex, FVector2D Value)
{
	using namespace UE::Chaos::ClothAsset::Private;
	if (LODIndex == INDEX_NONE)
	{
		for (int32 Index = 0; Index < CollectionPropertyFacades.Num(); ++Index)
		{
			if (TSharedPtr<::Chaos::Softs::FCollectionPropertyFacade> PropertyFacade = CollectionPropertyFacades[Index].Pin())
			{
				SetValueWithAlias(PropertyFacade, PropertyName,
					[&PropertyFacade, &Value](const FName& Name)
					{
						PropertyFacade->SetWeightedFloatValue(Name, FVector2f(Value));
					});
			}
		}
	}
	else if (CollectionPropertyFacades.IsValidIndex(LODIndex))
	{
		if (TSharedPtr<::Chaos::Softs::FCollectionPropertyFacade> PropertyFacade = CollectionPropertyFacades[LODIndex].Pin())
		{
			SetValueWithAlias(PropertyFacade, PropertyName,
				[&PropertyFacade, &Value](const FName& Name)
				{
					PropertyFacade->SetWeightedFloatValue(Name, FVector2f(Value));
				});
		}
	}
}

void UChaosClothAssetInteractor::SetIntPropertyValue(const FName PropertyName, int32 LODIndex, int32 Value)
{
	using namespace UE::Chaos::ClothAsset::Private;
	if (LODIndex == INDEX_NONE)
	{
		for (int32 Index = 0; Index < CollectionPropertyFacades.Num(); ++Index)
		{
			if (TSharedPtr<::Chaos::Softs::FCollectionPropertyFacade> PropertyFacade = CollectionPropertyFacades[Index].Pin())
			{
				SetValueWithAlias(PropertyFacade, PropertyName,
					[&PropertyFacade, Value](const FName& Name)
					{
						PropertyFacade->SetValue(Name, Value);
					});
			}
		}
	}
	else if (CollectionPropertyFacades.IsValidIndex(LODIndex))
	{
		if (TSharedPtr<::Chaos::Softs::FCollectionPropertyFacade> PropertyFacade = CollectionPropertyFacades[LODIndex].Pin())
		{
			SetValueWithAlias(PropertyFacade, PropertyName,
				[&PropertyFacade, Value](const FName& Name)
				{
					PropertyFacade->SetValue(Name, Value);
				});
		}
	}
}

void UChaosClothAssetInteractor::SetVectorPropertyValue(const FName PropertyName, int32 LODIndex, FVector Value)
{
	using namespace UE::Chaos::ClothAsset::Private;
	if (LODIndex == INDEX_NONE)
	{
		for (int32 Index = 0; Index < CollectionPropertyFacades.Num(); ++Index)
		{
			if (TSharedPtr<::Chaos::Softs::FCollectionPropertyFacade> PropertyFacade = CollectionPropertyFacades[Index].Pin())
			{
				SetValueWithAlias(PropertyFacade, PropertyName,
					[&PropertyFacade, &Value](const FName& Name)
					{
						PropertyFacade->SetValue(Name, FVector3f(Value));
					});
			}
		}
	}
	else if (CollectionPropertyFacades.IsValidIndex(LODIndex))
	{
		if (TSharedPtr<::Chaos::Softs::FCollectionPropertyFacade> PropertyFacade = CollectionPropertyFacades[LODIndex].Pin())
		{
			SetValueWithAlias(PropertyFacade, PropertyName,
				[&PropertyFacade, &Value](const FName& Name)
				{
					PropertyFacade->SetValue(Name, FVector3f(Value));
				});
		}
	}
}

void UChaosClothAssetInteractor::SetStringPropertyValue(const FName PropertyName, int32 LODIndex, const FString& Value)
{
	using namespace UE::Chaos::ClothAsset::Private;
	if (LODIndex == INDEX_NONE)
	{
		for (int32 Index = 0; Index < CollectionPropertyFacades.Num(); ++Index)
		{
			if (TSharedPtr<::Chaos::Softs::FCollectionPropertyFacade> PropertyFacade = CollectionPropertyFacades[Index].Pin())
			{
				SetValueWithAlias(PropertyFacade, PropertyName,
					[&PropertyFacade, &Value](const FName& Name)
					{
						PropertyFacade->SetStringValue(Name, Value);
					});
			}
		}
	}
	else if (CollectionPropertyFacades.IsValidIndex(LODIndex))
	{
		if (TSharedPtr<::Chaos::Softs::FCollectionPropertyFacade> PropertyFacade = CollectionPropertyFacades[LODIndex].Pin())
		{
			SetValueWithAlias(PropertyFacade, PropertyName,
				[&PropertyFacade, &Value](const FName& Name)
				{
					PropertyFacade->SetStringValue(Name, Value);
				});
		}
	}
}
