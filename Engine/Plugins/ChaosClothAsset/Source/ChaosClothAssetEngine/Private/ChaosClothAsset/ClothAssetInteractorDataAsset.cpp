// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ClothAssetInteractorDataAsset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClothAssetInteractorDataAsset)

#if WITH_EDITOR
void UClothAssetInteractorDataAsset::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UClothAssetInteractorDataAsset, PropertySets) && (PropertyChangedEvent.ChangeType & EPropertyChangeType::ArrayAdd))
	{
		// Initialize new property set with default values
		if (FMapProperty* Map = CastField<FMapProperty>(PropertyChangedEvent.Property))
		{
			FScriptMapHelper MapHelper(Map, &PropertySets);
			const int32 ArrayIndex = PropertyChangedEvent.GetArrayIndex(GET_MEMBER_NAME_STRING_CHECKED(UClothAssetInteractorDataAsset, PropertySets));
			if (MapHelper.IsValidIndex(ArrayIndex))
			{
				FClothAssetInteractorPropertyBag* const ValuePtr = (FClothAssetInteractorPropertyBag*)MapHelper.GetValuePtr(ArrayIndex);
				ValuePtr->Reset();
				ValuePtr->MigrateToNewBagInstance(DefaultProperties);
			}
		}
	}
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif

void UClothAssetInteractorDataAsset::SynchronizePropertySets()
{
	if (const UPropertyBag* const SourceBagStruct = DefaultProperties.GetPropertyBagStruct())
	{
		for (TMap<FName, FClothAssetInteractorPropertyBag>::TIterator PropertyIter = PropertySets.CreateIterator(); PropertyIter; ++PropertyIter)
		{
			if (const UPropertyBag* const TargetBagStruct = PropertyIter.Value().GetPropertyBagStruct())
			{
				constexpr bool bCheckNamesTrue = true;
				if (FStructUtils::TheSameLayout(SourceBagStruct, TargetBagStruct, bCheckNamesTrue))
				{
					continue;
				}

				const FClothAssetInteractorPropertyBag OrigTarget = PropertyIter.Value();
				PropertyIter.Value().Reset();
				PropertyIter.Value().MigrateToNewBagInstance(DefaultProperties);
				PropertyIter.Value().CopyMatchingValuesByID(OrigTarget);
			}
		}
	}
}