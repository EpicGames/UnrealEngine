// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DataAsset.h"
#include "StructUtils/PropertyBag.h"

#include "ClothAssetInteractorDataAsset.generated.h"

#define UE_API CHAOSCLOTHASSETENGINE_API

USTRUCT(BlueprintType)
struct FClothAssetInteractorPropertyBag : public FInstancedPropertyBag
{
	GENERATED_BODY()
};

template<>
struct TStructOpsTypeTraits<FClothAssetInteractorPropertyBag> : public TStructOpsTypeTraits<FInstancedPropertyBag>
{
};

/**
 * Data asset for setting groups of properties on the Chaos Cloth Asset Interactor.
 */
UCLASS(BlueprintType)
class  UClothAssetInteractorDataAsset : public UDataAsset
{
	GENERATED_BODY()

public:

	/** Get the Property Set for a given Set Name. If Set Name does not exist, the Default Properties will be returned.*/
	UFUNCTION(BlueprintCallable, Category = ClothProperty)
	const FClothAssetInteractorPropertyBag& GetPropertySet(const FName SetName) const
	{
		if (const FClothAssetInteractorPropertyBag* const FoundSet = PropertySets.Find(SetName))
		{
			return *FoundSet;
		}
		return DefaultProperties;
	}

	/** Synchronize the Property Sets with the Default Properties by adding/removing properties. Existing property values will not be modified.*/
	UFUNCTION(CallInEditor, Category = "Interactor Properties", meta = (DisplayPriority = 2))
	void SynchronizePropertySets();

#if WITH_EDITOR
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
private:

	/** Default Property schema and default values. Property Names should match the Interactor Name to set using the Cloth Asset Interactor. 
	 * Only property types known to the interactor (float, double, int, string, Vector, Vector2D) will be used to set values. */
	UPROPERTY(EditAnywhere, Category = "Interactor Properties")
	FClothAssetInteractorPropertyBag DefaultProperties;

	/** Property sets with matching schema to Default Properties. */
	UPROPERTY(EditAnywhere, Category = "Interactor Properties", Meta = (FixedLayout))
	TMap<FName, FClothAssetInteractorPropertyBag> PropertySets;
};
#undef UE_API
