// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"
#include "Stats/Stats.h"

DECLARE_LOG_CATEGORY_EXTERN(LogChaosClothAsset, Log, All);
DECLARE_STATS_GROUP(TEXT("ChaosClothAsset"), STATGROUP_ChaosClothAsset, STATCAT_Advanced);
