// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ClothAssetSKMClothingAsset.h"
#include "ChaosClothAsset/ClothAssetBase.h"
#include "ChaosClothAsset/ClothAssetPrivate.h"
#include "ChaosClothAsset/ClothSimulationModel.h"

#include "Engine/SkeletalMesh.h"

#if WITH_EDITOR
#include "Chaos/CollectionPropertyFacade.h"
#include "ChaosClothAsset/ClothEngineTools.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "Components/SkeletalMeshComponent.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "Utils/ClothingMeshUtils.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "ClothingAsset.h"
#endif  // #if WITH_EDITOR

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClothAssetSKMClothingAsset)

#define LOCTEXT_NAMESPACE "ChaosClothAssetSKMClothingAsset"

namespace UE::Chaos::ClothAsset::Private
{
	static FString MakeClothSimulationModelIdString(const FName Name, const FGuid Guid)
	{
		return FString::Format(TEXT("{0}-{1}"), { Name.ToString(), Guid.ToString() });
	}

#if WITH_EDITOR
	// Modified version of FLODUtilities::UnbindClothingAndBackup to only unbind specified assets
	static void UnbindClothingAndBackup(USkeletalMesh& SkeletalMesh, const UChaosClothAssetSKMClothingAsset& ClothingAsset, TArray<ClothingAssetUtils::FClothingAssetMeshBinding>& ClothingBindings)
	{
		ClothingBindings.Reset();

		if (SkeletalMesh.GetImportedModel())
		{
			auto UnbindClothingLODAndBackup = [&SkeletalMesh, &ClothingAsset](const int32 LODIndex, TArray<ClothingAssetUtils::FClothingAssetMeshBinding>& ClothingLODBindings)
				{
					ClothingLODBindings.Reset();

					TIndirectArray<FSkeletalMeshLODModel>& LODModels = SkeletalMesh.GetImportedModel()->LODModels;
					if (LODModels.IsValidIndex(LODIndex))
					{
						FSkeletalMeshLODModel& LODModel = LODModels[LODIndex];

						// Store this LOD's bindings
						ClothingAssetUtils::GetAllLodMeshClothingAssetBindings(&SkeletalMesh, ClothingLODBindings, LODIndex);  // TODO: We only need the binding for the specified ClothingAsset

						// Unbind this Cloth's LOD
						for (ClothingAssetUtils::FClothingAssetMeshBinding& Binding : ClothingLODBindings)
						{
							if (Binding.LODIndex == LODIndex && Binding.Asset == &ClothingAsset)
							{
								const int32 OriginalDataSectionIndex = LODModel.Sections[Binding.SectionIndex].OriginalDataSectionIndex;
								Binding.Asset->UnbindFromSkeletalMesh(&SkeletalMesh, Binding.LODIndex, Binding.SectionIndex);
								Binding.SectionIndex = OriginalDataSectionIndex;

								FSkelMeshSourceSectionUserData& SectionUserData = LODModel.UserSectionsData.FindChecked(OriginalDataSectionIndex);
								SectionUserData.ClothingData.AssetGuid = FGuid();
								SectionUserData.ClothingData.AssetLodIndex = INDEX_NONE;
								SectionUserData.CorrespondClothAssetIndex = INDEX_NONE;
							}
						}
					}
				};

			for (int32 LODIndex = 0; LODIndex < SkeletalMesh.GetImportedModel()->LODModels.Num(); ++LODIndex)
			{
				TArray<ClothingAssetUtils::FClothingAssetMeshBinding> ClothingLODBindings;
				UnbindClothingLODAndBackup(LODIndex, ClothingLODBindings);
				ClothingBindings.Append(ClothingLODBindings);
			}
		}
	}

	// Modified version of FLODUtilities::RestoreClothingFromBackup to only rebind specified assets
	static void RestoreClothingFromBackup(USkeletalMesh& SkeletalMesh, const UChaosClothAssetSKMClothingAsset& ClothingAsset, TArray<ClothingAssetUtils::FClothingAssetMeshBinding>& ClothingBindings)
	{
		if (SkeletalMesh.GetImportedModel() && ClothingAsset.GetClothSimulationModelIndex() != INDEX_NONE)
		{
			auto RestoreClothingLODFromBackup = [&SkeletalMesh, &ClothingAsset](const int32 LODIndex, TArray<ClothingAssetUtils::FClothingAssetMeshBinding>& ClothingBindings)
				{
					TIndirectArray<FSkeletalMeshLODModel>& LODModels = SkeletalMesh.GetImportedModel()->LODModels;
					if (LODModels.IsValidIndex(LODIndex))
					{
						FSkeletalMeshLODModel& LODModel = SkeletalMesh.GetImportedModel()->LODModels[LODIndex];
						for (ClothingAssetUtils::FClothingAssetMeshBinding& Binding : ClothingBindings)
						{
							if (Binding.LODIndex == LODIndex && Binding.Asset == &ClothingAsset)
							{
								for (int32 SectionIndex = 0; SectionIndex < LODModel.Sections.Num(); ++SectionIndex)
								{
									if (LODModel.Sections[SectionIndex].OriginalDataSectionIndex == Binding.SectionIndex)
									{
										if (Binding.Asset->BindToSkeletalMesh(&SkeletalMesh, Binding.LODIndex, SectionIndex, Binding.AssetInternalLodIndex))
										{
											// If successfull set back the section user data
											FSkelMeshSourceSectionUserData& SectionUserData = LODModel.UserSectionsData.FindChecked(Binding.SectionIndex);
											SectionUserData.CorrespondClothAssetIndex = LODModel.Sections[SectionIndex].CorrespondClothAssetIndex;
											SectionUserData.ClothingData = LODModel.Sections[SectionIndex].ClothingData;
										}
										break;  // Found the section, next binding
									}
								}
							}
						}
					}
				};

			for (int32 LODIndex = 0; LODIndex < SkeletalMesh.GetImportedModel()->LODModels.Num(); ++LODIndex)
			{
				RestoreClothingLODFromBackup(LODIndex, ClothingBindings);
			}
		}
	}
#endif
}

UChaosClothAssetSKMClothingAsset::UChaosClothAssetSKMClothingAsset(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	if (!IsTemplate())
	{
		UClothingAssetBase::AssetGuid = FGuid::NewGuid();
	}
}

#if WITH_EDITOR
void UChaosClothAssetSKMClothingAsset::SetAsset(const UChaosClothAssetBase* InAsset)
{
	if (Asset != InAsset)
	{
		Asset = InAsset;
		OnAssetChanged();
	}
}

void UChaosClothAssetSKMClothingAsset::OnAssetChanged(const bool bReregisterComponents)
{
	// If the asset has changed, check whether the current specified model still exists in this new asset, otherwise update the model guid
	const FGuid ClothSimulationModelGuid = GetClothSimulationModelGuid();
	FGuid ModelGuid;
	int32 ModelIndex = INDEX_NONE;

	if (Asset)
	{
		if (const int32 NumClothSimulationModels = Asset->GetNumClothSimulationModels())
		{
			// Find a matching model GUID
			for (ModelIndex = 0; ModelIndex < NumClothSimulationModels; ++ModelIndex)
			{
				if (Asset->GetAssetGuid(ModelIndex) == ClothSimulationModelGuid)
				{
					break;
				}
			}

			if (ModelIndex == NumClothSimulationModels)
			{
				// Try locate a model with a similar name to update to a valid GUID if possible
				const FName ClothSimulationModelName = GetClothSimulationModelName();

				for (ModelIndex = 0; ModelIndex < Asset->GetNumClothSimulationModels(); ++ModelIndex)
				{
					if (Asset->GetClothSimulationModelName(ModelIndex) == ClothSimulationModelName)
					{
						break;
					}
				}

				if (ModelIndex == NumClothSimulationModels)
				{
					// Or reset to the first model if there isn't one
					ModelIndex = 0;
				}
			}

			if (ModelIndex != INDEX_NONE)
			{
				ModelGuid = Asset->GetAssetGuid(ModelIndex);
			}
		}
	}

	// Notify the model has changed
	if (ClothSimulationModelIndex != ModelIndex || ClothSimulationModelGuid != ModelGuid)
	{
		ClothSimulationModelIndex = ModelIndex;

		OnModelChanged();
	}

	if (const USkeletalMesh* const OwnerMesh = Cast<USkeletalMesh>(GetOuter()))
	{
		if (bReregisterComponents)
		{
			for (TObjectIterator<USkeletalMeshComponent> It; It; ++It)
			{
				if (USkeletalMeshComponent* const Component = *It)
				{
					if (Component->GetSkeletalMeshAsset() == OwnerMesh)
					{
						FComponentReregisterContext Context(Component);
						// Context goes out of scope, causing Component to be re-registered
					}
				}
			}
		}  // TODO: Move UClothingAssetCommon::ReregisterComponentsUsingClothing() to a common place (in Utils/ClothingMeshUtils?)

		// Warn asset model already in use
		for (const UClothingAssetBase* const& MeshClothingAsset : OwnerMesh->GetMeshClothingAssets())
		{
			if (const UChaosClothAssetSKMClothingAsset* const Other = Cast<const UChaosClothAssetSKMClothingAsset>(MeshClothingAsset))
			{
				if (Other != this &&
					Other->Asset == Asset &&
					Other->ClothSimulationModelIndex == ClothSimulationModelIndex)
				{
					FNotificationInfo NotificationInfo(LOCTEXT("WarningUsingDuplicateAssetModel",
						"This asset and simulation model is already set to another Clothing Data.\n"
						"For performance reason, prefer binding a single Clothing Data to multiple LOD sections instead."));
					NotificationInfo.ExpireDuration = 10.f;
					FSlateNotificationManager::Get().AddNotification(NotificationInfo);
					UE_LOGF(LogChaosClothAsset, Warning, "%ls", *NotificationInfo.Text.Get().ToString());
					break;
				}
			}
		}
	}
}

void UChaosClothAssetSKMClothingAsset::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UChaosClothAssetSKMClothingAsset, Asset))
	{
		OnAssetChanged();
	}
	else if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UChaosClothAssetSKMClothingAsset, ClothSimulationModelId))
	{
		OnModelChanged();
	}
}

USkinnedAsset* UChaosClothAssetSKMClothingAsset::GetSkinnedAssetDependency() const
{
	return const_cast<UChaosClothAssetBase*>(Asset.Get());
}

bool UChaosClothAssetSKMClothingAsset::BindToSkeletalMesh(USkeletalMesh* SkeletalMesh, const int32 MeshLodIndex, const int32 SectionIndex, const int32 AssetLodIndex)
{
	using namespace UE::Chaos::ClothAsset;

	if (!ensure(SkeletalMesh) ||
		!ensure(SkeletalMesh == GetOuter()) ||
		!ensure(SkeletalMesh->GetImportedModel()) ||
		!ensure(SkeletalMesh->GetImportedModel()->LODModels.IsValidIndex(MeshLodIndex)) ||
		!ensure(SkeletalMesh->GetImportedModel()->LODModels[MeshLodIndex].Sections.IsValidIndex(SectionIndex)) ||
		!ensure(MeshLodIndex == AssetLodIndex))
	{
		return false;
	}

	// When FScopedSkeletalMeshPostEditChange goes out of scope it causes the postedit change and components to be re-registered and the mesh to rebuild
	FScopedSkeletalMeshPostEditChange SkeletalMeshPostEditChange(SkeletalMesh);

	// Get the original render section
	FSkeletalMeshLODModel& SkeletalMeshLODModel = SkeletalMesh->GetImportedModel()->LODModels[MeshLodIndex];
	FSkelMeshSection& SkelMeshSection = SkeletalMeshLODModel.Sections[SectionIndex];

	// Detect a partial binding state
	// The GUID is already set but no mapping data was ever generated (e.g. if Asset was null on the first bind).
	// Assigning the same GUID will not change the DDC key, so an explicit invalidation is done at the end of this function.
	const bool bWasPartiallyBound = (SkelMeshSection.ClothingData.AssetGuid == AssetGuid) && !SkelMeshSection.HasClothingData();

	// Clear the proxy deformer mappings data now in case the cloth simulation model is invalid
	SkelMeshSection.ClothMappingDataLODs.Reset();

	// Set the asset guid
	SkelMeshSection.ClothingData.AssetGuid = AssetGuid;
	SkelMeshSection.ClothingData.AssetLodIndex = AssetLodIndex;

	// Set the asset index, used during rendering to pick the correct sim mesh buffer
	int32 AssetIndex = INDEX_NONE;
	verify(SkeletalMesh->GetMeshClothingAssets().Find(this, AssetIndex));
	SkelMeshSection.CorrespondClothAssetIndex = (int16)AssetIndex;

	// Retrieve the cloth simulation model
	if (!Asset)
	{
		const FText Text = FText::Format(
			LOCTEXT("NoAssetHasBeenSet", "Clothing Data [{0}]: no Asset has been set."),
			FText::FromString(GetName()));
		WarningNotification(Text);
		return true;  // Must return true, this is to avoid breaking the binding when the cloth simulation model is invalid
	}
	if (ClothSimulationModelIndex == INDEX_NONE)
	{
		const FText Text = FText::Format(
			LOCTEXT("NotAClothSimulationModelOfAsset", "Clothing Data [{0}]: [{1}] is not an existing Cloth Simulation Model of Asset [{2}]."),
			FText::FromString(GetName()),
			FText::FromName(GetClothSimulationModelName()),
			FText::FromName(Asset->GetFName()));
		WarningNotification(Text);
		return true;  // Must return true, this is to avoid breaking the binding when the cloth simulation model is invalid
	}
	const TSharedPtr<const FChaosClothSimulationModel> ClothSimulationModel = Asset->GetClothSimulationModel(ClothSimulationModelIndex);
	if (!ClothSimulationModel ||
		!ClothSimulationModel->GetNumVertices(AssetLodIndex) ||
		!ClothSimulationModel->GetNumTriangles(AssetLodIndex))
	{
		const FText Text = FText::Format(
			LOCTEXT("EmptyClothSimulationModel", "Clothing Data [{0}]: [{1}] is an empty Cloth Simulation Model of Asset [{2}]."),
			FText::FromString(GetName()),
			FText::FromName(GetClothSimulationModelName()),
			FText::FromName(Asset->GetFName()));
		WarningNotification(Text);
		return true;  // Must return true, this is to avoid breaking the binding when the cloth simulation model is invalid
	}
	if (!ClothSimulationModel->IsValidLodIndex(AssetLodIndex))
	{
		const FText Text = FText::Format(
			LOCTEXT("NoLODInClothSimulationModel", "Clothing Data [{0}]: Cloth Simulation Model [{1}] has no LOD{2} in Asset [{3}]."),
			FText::FromString(GetName()),
			FText::FromName(GetClothSimulationModelName()),
			AssetLodIndex,
			FText::FromName(Asset->GetFName()));
		WarningNotification(Text);
		return true;  // Must return true, this is to avoid breaking the binding when the cloth simulation model is invalid
	}

	// Set the new section bone map, adding the cloth asset bones
	TArray<FBoneIndexType> BoneMap = SkelMeshSection.BoneMap;
	for (const FName BoneName : ClothSimulationModel->UsedBoneNames)
	{
		const int32 BoneIndex = SkeletalMesh->GetRefSkeleton().FindBoneIndex(BoneName);
		if (BoneIndex != INDEX_NONE)
		{
			BoneMap.AddUnique(BoneIndex);
		}
	}

	SkelMeshSection.BoneMap = MoveTemp(BoneMap);

	bool bRequireBoneChange = false;
	for (FBoneIndexType& BoneIndex : SkelMeshSection.BoneMap)
	{
		if (!SkeletalMeshLODModel.RequiredBones.Contains(BoneIndex))
		{
			bRequireBoneChange = true;
			if (SkeletalMesh->GetRefSkeleton().IsValidIndex(BoneIndex))
			{
				SkeletalMeshLODModel.RequiredBones.Add(BoneIndex);
				SkeletalMeshLODModel.ActiveBoneIndices.AddUnique(BoneIndex);
			}
		}
	}
	if (bRequireBoneChange)
	{
		SkeletalMeshLODModel.RequiredBones.Sort();
		SkeletalMesh->GetRefSkeleton().EnsureParentsExistAndSort(SkeletalMeshLODModel.ActiveBoneIndices);
	}

	// Set the deformer mappings
	int32 MatchingDeformerSectionIndex = INDEX_NONE;
	if (const FSkeletalMeshRenderData* const AssetRenderData = Asset->GetResourceForRendering())
	{
		// Look for a suitable Cloth/Outfit Asset render section to copy the deformer mapping from
		if (AssetRenderData->LODRenderData.IsValidIndex(AssetLodIndex))
		{
			const FSkeletalMeshLODRenderData& AssetLODRenderData = AssetRenderData->LODRenderData[AssetLodIndex];
			const FPositionVertexBuffer& AssetPositionVertexBuffer = AssetLODRenderData.StaticVertexBuffers.PositionVertexBuffer;

			TArray<FSoftSkinVertex> SkelMeshVertices;
			SkeletalMeshLODModel.GetVertices(SkelMeshVertices);
			const int32 SkelMeshVertexOffset = SkelMeshSection.BaseVertexIndex;

			for (int32 DeformerSectionIndex = 0; DeformerSectionIndex < AssetLODRenderData.RenderSections.Num(); ++DeformerSectionIndex)
			{
				const FSkelMeshRenderSection& AssetRenderSection = AssetLODRenderData.RenderSections[DeformerSectionIndex];
				if (AssetRenderSection.CorrespondClothAssetIndex == ClothSimulationModelIndex &&  // The model index must match when binding with an Outfit Asset
					AssetRenderSection.HasClothingData() &&  // The section must have mapping to copy from
					AssetRenderSection.GetNumVertices() == SkelMeshSection.GetNumVertices())
				{
					const int32 AssetVertexOffset = AssetRenderSection.BaseVertexIndex;

					bool bVertexPositionsMatch = true;
					for (int32 VertexIndex = 0; VertexIndex < AssetRenderSection.GetNumVertices(); ++VertexIndex)
					{
						const int32 SkelMeshVertexIndex = SkelMeshVertexOffset + VertexIndex;
						const int32 AssetVertexIndex = AssetVertexOffset + VertexIndex;

						if (UE_SMALL_NUMBER < FVector3f::DistSquared(
							AssetPositionVertexBuffer.VertexPosition(AssetVertexIndex),
							SkelMeshVertices[SkelMeshVertexIndex].Position))
						{
							bVertexPositionsMatch = false;
							break;
						}
					}
					if (bVertexPositionsMatch)
					{
						SkelMeshSection.ClothMappingDataLODs.SetNum(1);
						SkelMeshSection.ClothMappingDataLODs[0] = AssetRenderSection.ClothMappingDataLODs[0];
						MatchingDeformerSectionIndex = DeformerSectionIndex;
						break;
					}
				}
			}
			if (MatchingDeformerSectionIndex == INDEX_NONE && AssetLODRenderData.RenderSections.Num())
			{
				const FText Text = FText::Format(
					LOCTEXT("NoMatchingRenderDeformerSection",
						"No matching render section found in Clothing Data [{0}] Asset [{1}] to copy render deformer data from, even though the Cloth/Outfit Asset contains render data."),
					FText::FromString(GetName()),
					FText::FromName(Asset->GetFName()));
				WarningNotification(Text);
			}
		}
	}
	if (MatchingDeformerSectionIndex == INDEX_NONE)
	{
		// Calculate the deformer mappings instead if no section was found
		const int32 NumIndices = SkelMeshSection.NumTriangles * 3;
		const int32 NumVertices = SkelMeshSection.SoftVertices.Num();
		const uint32 BaseIndex = SkelMeshSection.BaseIndex;
		const uint32 BaseVertexIndex = SkelMeshSection.BaseVertexIndex;

		TArray<FVector3f> RenderPositions;
		TArray<FVector3f> RenderNormals;
		TArray<FVector3f> RenderTangents;
		RenderPositions.Reserve(NumVertices);
		RenderNormals.Reserve(NumVertices);
		RenderTangents.Reserve(NumVertices);
		for (const FSoftSkinVertex& SoftSkinVertex : SkelMeshSection.SoftVertices)
		{
			RenderPositions.Add(SoftSkinVertex.Position);
			RenderNormals.Add(SoftSkinVertex.TangentZ);
			RenderTangents.Add(SoftSkinVertex.TangentX);
		}

		const TConstArrayView<uint32> SectionRenderIndices(SkeletalMeshLODModel.IndexBuffer.GetData() + BaseIndex, NumIndices);
		TArray<uint32> RenderIndices;
		RenderIndices.Reserve(SectionRenderIndices.Num());
		for (const uint32 SectionRenderIndex : SectionRenderIndices)
		{
			RenderIndices.Add(SectionRenderIndex - BaseVertexIndex);
		}

		const ClothingMeshUtils::ClothMeshDesc TargetMesh(RenderPositions, RenderNormals, RenderTangents, RenderIndices);
		const ClothingMeshUtils::ClothMeshDesc SourceMesh(
			ClothSimulationModel->GetPositions(AssetLodIndex),
			ClothSimulationModel->GetIndices(AssetLodIndex));  // Let it calculate the averaged normals as to match the simulation data output

		const TSharedRef<const FManagedArrayCollection>& Collection = Asset->GetCollections(ClothSimulationModelIndex)[AssetLodIndex];
		FCollectionClothConstFacade ClothFacade(Collection);
		::Chaos::Softs::FCollectionPropertyConstFacade PropertyFacade(Collection);
		const FPointWeightMap MaxDistances = FClothEngineTools::GetMaxDistanceWeightMap(ClothFacade, PropertyFacade, ClothSimulationModel->GetNumVertices(AssetLodIndex));

		constexpr bool bSmoothTransition = true;
		constexpr bool bUseMultipleInfluences = false;
		constexpr float SkinningKernelRadius = 100.f;
		TArray<FMeshToMeshVertData> MeshToMeshData;
		ClothingMeshUtils::GenerateMeshToMeshVertData(
			MeshToMeshData, TargetMesh, SourceMesh, &MaxDistances,
			bSmoothTransition, bUseMultipleInfluences, SkinningKernelRadius);

		SkelMeshSection.ClothMappingDataLODs.SetNum(1);
		SkelMeshSection.ClothMappingDataLODs[0] = MoveTemp(MeshToMeshData);
	}

	// Update the extra cloth deformer mapping LOD bias using this cloth entry
	ClothingAssetUtils::UpdateLODBiasMappings(*this, SkeletalMesh, MeshLodIndex, SectionIndex);

	// Invalidate the skeletal mesh for when the GUID was already set but the mapping data was missing
	// Game thread only to avoid clashing with the async build where the cloth bindings are unbound/rebound.
	// Partially bound sections can only be fixed on the GameThread and from the Persona Editor anyway.
	if (bWasPartiallyBound && IsInGameThread())
	{
		SkeletalMesh->InvalidateDeriveDataCacheGUID();
	}

	return true;
}

void UChaosClothAssetSKMClothingAsset::UnbindFromSkeletalMesh(USkeletalMesh* SkeletalMesh, const int32 MeshLodIndex, const int32 SectionIndex)
{
	if (SkeletalMesh)
	{
		ClothingAssetUtils::UnbindFromSkeletalMesh(*this, *SkeletalMesh, MeshLodIndex, SectionIndex);
	}
}

void UChaosClothAssetSKMClothingAsset::UpdateAllLODBiasMappings(USkeletalMesh* SkeletalMesh)
{
	check(SkeletalMesh);

	if (FSkeletalMeshModel* const SkeletalMeshModel = SkeletalMesh->GetImportedModel())
	{
		// Iterate through all source LODs with cloth that could lead to upper (raytraced) sections needing some additional mapping
		TIndirectArray<FSkeletalMeshLODModel>& LODModels = SkeletalMeshModel->LODModels;

		for (int32 LODIndex = LODModels.Num() - 1; LODIndex >= 0; --LODIndex)  // Iterate in reverse order to allow shrinking the mapping array first
		{
			// Go through all sections to find the one(s?) that uses this cloth asset and clear the existing bias mappings
			TArray<FSkelMeshSection>& Sections = LODModels[LODIndex].Sections;

			for (int32 SectionIndex = 0; SectionIndex < Sections.Num(); ++SectionIndex)
			{
				if (Sections[SectionIndex].ClothingData.AssetGuid == GetAssetGuid())
				{
					Sections[SectionIndex].ClothMappingDataLODs.SetNum(1);  // Only keep ClothMappingDataLODs[0] that is the same LOD mapping to remove LOD bias mappings

					if (SkeletalMesh->GetSupportRayTracing())  // This conditional is inside the loop so that the SetNum(1) can removed the unused bias mappings
					{
						// Updates the upper LODs mappings of the specified section from this LODIndex
						ClothingAssetUtils::UpdateLODBiasMappings(*this, SkeletalMesh, LODIndex, SectionIndex);
					}
				}
			}
		}
	}
}

void UChaosClothAssetSKMClothingAsset::GetSimulationMesh(const int32 InMeshLodIndex, TArray<FVector3f>& OutPositions, TArray<uint32>& OutIndices, TArray<float>& OutMaxDistances) const
{
	using namespace UE::Chaos::ClothAsset;

	OutPositions.Reset();
	OutIndices.Reset();
	OutMaxDistances.Reset();

	if (Asset && ClothSimulationModelIndex >= 0 && ClothSimulationModelIndex < Asset->GetNumClothSimulationModels())
	{
		const TSharedPtr<const FChaosClothSimulationModel> ClothSimulationModel = Asset->GetClothSimulationModel(ClothSimulationModelIndex);
		if (ClothSimulationModel && InMeshLodIndex >= 0 && InMeshLodIndex < ClothSimulationModel->GetNumLods())
		{
			OutPositions = ClothSimulationModel->GetPositions(InMeshLodIndex);
			OutIndices = ClothSimulationModel->GetIndices(InMeshLodIndex);

			const TArray<TSharedRef<const FManagedArrayCollection>>& Collections = Asset->GetCollections(ClothSimulationModelIndex);
			if (Collections.IsValidIndex(InMeshLodIndex))
			{
				const TSharedRef<const FManagedArrayCollection>& Collection = Collections[InMeshLodIndex];
				FCollectionClothConstFacade ClothFacade(Collection);
				::Chaos::Softs::FCollectionPropertyConstFacade PropertyFacade(Collection);

				FPointWeightMap MaxDistances = FClothEngineTools::GetMaxDistanceWeightMap(ClothFacade, PropertyFacade, ClothSimulationModel->GetNumVertices(InMeshLodIndex));
				OutMaxDistances = MoveTemp(MaxDistances.Values);
			}
		}
	}
}

TArray<TPair<FText, FText>> UChaosClothAssetSKMClothingAsset::GetStats() const
{
	TArray<TPair<FText, FText>> Stats;
	if (Asset && ClothSimulationModelIndex != INDEX_NONE)
	{
		if (const TSharedPtr<const FChaosClothSimulationModel> ClothSimulationModel = Asset->GetClothSimulationModel(ClothSimulationModelIndex))
		{
			for (int32 LodIndex = 0; LodIndex < ClothSimulationModel->GetNumLods(); ++LodIndex)
			{
				Stats.Emplace(FText::Format(LOCTEXT("LodIndexLabel", "LOD {0}"), FText::AsNumber(LodIndex)), FText{});
				Stats.Emplace(LOCTEXT("VertexCount", "Vertices"), FText::AsNumber(ClothSimulationModel->GetNumVertices(LodIndex)));
				Stats.Emplace(LOCTEXT("TriangleCount", "Triangles"), FText::AsNumber(ClothSimulationModel->GetNumTriangles(LodIndex)));
				const FChaosClothSimulationLodModel& ClothSimulationLodModel = ClothSimulationModel->ClothSimulationLodModels[LodIndex];
				Stats.Emplace(LOCTEXT("WeightMapCount", "Weight Maps"), FText::AsNumber(ClothSimulationLodModel.WeightMaps.Num()));
				Stats.Emplace(LOCTEXT("MaxBoneWeights", "Bone Weights"), FText::AsNumber(ClothSimulationLodModel.CalcMaxBoneInfluences()));
			}
		}
	}
	return Stats;
}

void UChaosClothAssetSKMClothingAsset::OnModelChanged()
{
	// Update the ClothSimulationModelId dropdown property
	using namespace UE::Chaos::ClothAsset::Private;
	ClothSimulationModelId = (ClothSimulationModelIndex != INDEX_NONE) ?
		MakeClothSimulationModelIdString(Asset->GetClothSimulationModelName(ClothSimulationModelIndex), Asset->GetAssetGuid(ClothSimulationModelIndex)) :
		FString();

	// Unbind/rebind clothing with the new GUID
	if (USkeletalMesh* const SkeletalMesh = Cast<USkeletalMesh>(GetOuter()))
	{
		FScopedSkeletalMeshPostEditChange ScopedPostEditChange(SkeletalMesh);  // Prevent the skeletal mesh from rebuilding until the unbind/rebind operation is complete

		TArray<ClothingAssetUtils::FClothingAssetMeshBinding> ClothingBindings;
		UnbindClothingAndBackup(*SkeletalMesh, *this, ClothingBindings);
		RestoreClothingFromBackup(*SkeletalMesh, *this, ClothingBindings);
		SkeletalMesh->InvalidateDeriveDataCacheGUID();  // The DDC key only changes with the ClothingAsset GUID, but a model change still requires a mesh rebuild
	}

	// Invalidate the transient simulation model.
	TransientSimulationModel.Reset();
}

FGuid UChaosClothAssetSKMClothingAsset::GetClothSimulationModelGuid() const
{
	int32 DashIndex;
	return ClothSimulationModelId.FindLastChar(TEXT('-'), DashIndex) ?
		FGuid(ClothSimulationModelId.RightChop(DashIndex + 1)) : FGuid();
}

FName UChaosClothAssetSKMClothingAsset::GetClothSimulationModelName() const
{
	int32 DashIndex;
	return ClothSimulationModelId.FindLastChar(TEXT('-'), DashIndex) ?
		FName(ClothSimulationModelId.LeftChop(ClothSimulationModelId.Len() - DashIndex)) : FName();
}
#endif  // #if WITH_EDITOR

TArray<FString> UChaosClothAssetSKMClothingAsset::GetClothSimulationModelIds() const
{
	using namespace UE::Chaos::ClothAsset::Private;

	TArray<FString> ClothSimulationModelIds;
	if (Asset)
	{
		const int32 NumSimulationModels = Asset->GetNumClothSimulationModels();
		ClothSimulationModelIds.Reserve(NumSimulationModels);
		for (int32 ModelIndex = 0; ModelIndex < NumSimulationModels; ++ModelIndex)
		{
			if (Asset->GetClothSimulationModel(ModelIndex).IsValid() && Asset->GetClothSimulationModel(ModelIndex)->GetNumLods())
			{
				ClothSimulationModelIds.Emplace(MakeClothSimulationModelIdString(
					Asset->GetClothSimulationModelName(ModelIndex),
					Asset->GetAssetGuid(ModelIndex)));
			}
		}
	}
	return ClothSimulationModelIds;
}

bool UChaosClothAssetSKMClothingAsset::HasAnySimulationMeshData(const int32 LODIndex) const
{
	if (Asset && ClothSimulationModelIndex >= 0 && ClothSimulationModelIndex < Asset->GetNumClothSimulationModels())
	{
		if (const TSharedPtr<const FChaosClothSimulationModel> ClothSimulationModel = Asset->GetClothSimulationModel(ClothSimulationModelIndex))
		{
			return
				ClothSimulationModel->IsValidLodIndex(LODIndex) &&
				ClothSimulationModel->GetNumVertices(LODIndex) > 0 &&
				ClothSimulationModel->GetNumTriangles(LODIndex) > 0;
		}
	}
	return false;
}

const TSharedPtr<const FChaosClothSimulationModel>& UChaosClothAssetSKMClothingAsset::GetTransientSimulationModel() const
{
	return TransientSimulationModel;
}

void UChaosClothAssetSKMClothingAsset::RefreshBoneMapping(USkeletalMesh* SkeletalMesh)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UChaosClothAssetSKMClothingAsset::RefreshBoneMapping);

	// Remove the transient simulation model. It is always recreated if a bone remap is needed.
	TransientSimulationModel.Reset();

	if (!SkeletalMesh)
	{
		return;
	}

	if (!(GetAsset() && IsValid()))
	{
		return;
	}

	TSharedPtr<const FChaosClothSimulationModel> ClothSimulationModel = GetAsset()->GetClothSimulationModel(ClothSimulationModelIndex);

	if (!ClothSimulationModel)
	{
		return;
	}

	const int32 NumUsedBones = ClothSimulationModel->UsedBoneNames.Num();

	TArray<int32> RemappedUsedBoneIndices;
	RemappedUsedBoneIndices.SetNumUninitialized(NumUsedBones);

	bool bNeedsBoneRemap = false;

	// Construct the bone remap using the provided SkeletalMesh and check if we actually need the remap.
	for (int32 UsedBoneNameIndex = 0; UsedBoneNameIndex < NumUsedBones; ++UsedBoneNameIndex)
	{
		const FName& BoneName = ClothSimulationModel->UsedBoneNames[UsedBoneNameIndex];
		int32 RefSkeletonBoneIndex = SkeletalMesh->GetRefSkeleton().FindBoneIndex(BoneName);

		bNeedsBoneRemap |= ClothSimulationModel->UsedBoneIndices[UsedBoneNameIndex] != RefSkeletonBoneIndex;

		RemappedUsedBoneIndices[UsedBoneNameIndex] = FMath::Max(0, RefSkeletonBoneIndex); // If not found, map to the root. 
	}

	const int32 NumLODModels = ClothSimulationModel->ClothSimulationLodModels.Num();

	TArray< TArray<uint16>, TInlineAllocator<16> > PerLODAdditionalRequiredBoneRemappedIndices;
	PerLODAdditionalRequiredBoneRemappedIndices.SetNum(NumLODModels);

	// Recreate RequiredExtraBoneIndices for each LOD model.
	// TODO: optimize if needed.
	for (int32 LODIndex = 0; LODIndex < NumLODModels; ++LODIndex)
	{
		const FChaosClothSimulationLodModel& LODModel = ClothSimulationModel->ClothSimulationLodModels[LODIndex];
		PerLODAdditionalRequiredBoneRemappedIndices[LODIndex].Reserve(LODModel.RequiredExtraBoneIndices.Num());

		for (uint16 BoneIndex : LODModel.RequiredExtraBoneIndices)
		{
			int32 BoneNameIndex = ClothSimulationModel->UsedBoneIndices.Find((int32)BoneIndex);
			if (BoneNameIndex != INDEX_NONE)
			{
				const FName& BoneName = ClothSimulationModel->UsedBoneNames[BoneNameIndex];
				int32 RefSkeletonBoneNameIndex = SkeletalMesh->GetRefSkeleton().FindBoneIndex(BoneName);

				// NOTE: This removes the bone from the required if not found. 
				if (RefSkeletonBoneNameIndex != INDEX_NONE)
				{
					PerLODAdditionalRequiredBoneRemappedIndices[LODIndex].Add((uint16)RefSkeletonBoneNameIndex);
				}
			}
		}

		bNeedsBoneRemap |= PerLODAdditionalRequiredBoneRemappedIndices[LODIndex] != LODModel.RequiredExtraBoneIndices;
	}


	// If no remap is needed, we can skip the creation of the transient simulation model.
	if (bNeedsBoneRemap)
	{	
		// Always create a new transient simulation model to prevent races with other references.
		TSharedPtr<FChaosClothSimulationModel> LocalMutTransientSimulationModel = MakeShared<FChaosClothSimulationModel>(*ClothSimulationModel);

		const int32 ReferenceBoneNameIndex = ClothSimulationModel->UsedBoneIndices.Find(ClothSimulationModel->ReferenceBoneIndex);

		LocalMutTransientSimulationModel->ReferenceBoneIndex = INDEX_NONE;
		if (ReferenceBoneNameIndex != INDEX_NONE)
		{
			const FName& ReferenceBoneName = ClothSimulationModel->UsedBoneNames[ReferenceBoneNameIndex];
			LocalMutTransientSimulationModel->ReferenceBoneIndex = SkeletalMesh->GetRefSkeleton().FindBoneIndex(ReferenceBoneName);
		}

		// Fallback to the root if the reference bone name could not be found to prevent issues downstream.
		if (LocalMutTransientSimulationModel->ReferenceBoneIndex == INDEX_NONE)
		{
			LocalMutTransientSimulationModel->ReferenceBoneIndex = 0;
		}
		
		check(RemappedUsedBoneIndices.Num() == LocalMutTransientSimulationModel->UsedBoneNames.Num());
		LocalMutTransientSimulationModel->UsedBoneIndices = MoveTemp(RemappedUsedBoneIndices);

		check(LocalMutTransientSimulationModel->ClothSimulationLodModels.Num() == NumLODModels);
		for (int32 LODIndex = 0; LODIndex < NumLODModels; ++LODIndex)
		{
			FChaosClothSimulationLodModel& LODModel = LocalMutTransientSimulationModel->ClothSimulationLodModels[LODIndex];
			LODModel.RequiredExtraBoneIndices = MoveTemp(PerLODAdditionalRequiredBoneRemappedIndices[LODIndex]);
		}

		// Publish the local mutable simulation model.
		// NOTE: This is not thread safe. There could be a race if a thread reads with GetTransientSimulationModel at this point.
		// the same applies when we reset the TransientSimulationModel pointer.
		TransientSimulationModel = LocalMutTransientSimulationModel;
	}

}

#undef LOCTEXT_NAMESPACE
