// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ClothAssetSKMClothingSimulation.h"
#include "ChaosClothAsset/ClothAssetSKMClothingSimulationInteractor.h"
#include "Chaos/CollectionPropertyFacade.h"
#include "ChaosCloth/ChaosClothingSimulationSolver.h"
#include "ChaosClothAsset/ClothAssetBase.h"
#include "ChaosClothAsset/ClothAssetSKMClothingAsset.h"
#include "ChaosClothAsset/ClothComponent.h"  // For FChaosClothSimulationProperties  TODO: Move FChaosClothSimulationProperties to its own header file?
#include "ChaosClothAsset/ClothSimulationModel.h"
#include "ChaosClothAsset/ClothSimulationProxy.h"
#include "ChaosClothAsset/CollisionSources.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "HAL/IConsoleManager.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "ClothCollisionData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClothAssetSKMClothingSimulation)

namespace UE::Chaos::ClothAsset
{
	FSKMClothingSimulation::FSKMClothingSimulation() = default;
	FSKMClothingSimulation::~FSKMClothingSimulation() = default;

	void FSKMClothingSimulation::ResetConfigProperties()
	{
		check(IsInGameThread());

		// Update collection, facade and outfit interactor
		for (FAssetClothSimulationProperties& Properties : AssetClothSimulationProperties)
		{
			if (const UChaosClothAssetSKMClothingAsset* const ClothAssetSKMClothingAsset = Properties.Key)
			{
				if (const int32 ClothSimulationModelIndex = ClothAssetSKMClothingAsset->GetClothSimulationModelIndex(); ClothSimulationModelIndex != INDEX_NONE)
				{
					if (const UChaosClothAssetBase* const Asset = ClothAssetSKMClothingAsset->GetAsset(); ensure(Asset))  // ClothSimulationModelIndex has already been checked, so the ClothAsset can't be null
					{
						Properties.Value.Initialize(Asset->GetCollections(ClothSimulationModelIndex));
					}
				}
			}
		}
	}

	void FSKMClothingSimulation::RecreateClothSimulationProxy()
	{
		ClothSimulationProxy = MakeShared<FClothSimulationProxy>(*this);
		if (ClothSimulationProxy.IsValid())
		{
			// Re-add all assets to the proxy
			ClothSimulationProxy->PostConstructor();
		}
	}

	UChaosClothAssetInteractor* FSKMClothingSimulation::GetPropertyInteractor(const UChaosClothAssetBase* Asset, int32 ModelIndex) const
	{
		const int32 SimulationGroupId = GetSimulationGroupId(Asset, ModelIndex);
		checkf(SimulationGroupId != INDEX_NONE, TEXT("Invalid arguments"));
		return AssetClothSimulationProperties[SimulationGroupId].Value.ClothOutfitInteractor;
	}

	void FSKMClothingSimulation::Initialize()
	{
		Shutdown();
		CollisionSources = MakeUnique<FCollisionSources>();
		ClothSimulationProxy = MakeShared<FClothSimulationProxy>(*this);
	}

	void FSKMClothingSimulation::Shutdown()
	{
		DestroyActors();
		OwnerComponent = nullptr;
		CollisionSources = nullptr;
		ClothSimulationProxy = nullptr;
	}

	void FSKMClothingSimulation::FillContextAndPrepareTick(const USkeletalMeshComponent* /*InComponent*/, float InDeltaTime, IClothingSimulationContext* InOutContext, bool bIsInitialization, bool bForceTeleportResetOnly)
	{
		if (FSKMClothingSimulationContext* const Context = static_cast<FSKMClothingSimulationContext*>(InOutContext))
		{
			Context->DeltaTime = InDeltaTime;
		}
		if (!bIsInitialization) // Actual context and cloths will be initialized as part of EndCreateActor when initializing
		{
			if (ClothSimulationProxy.IsValid())
			{
				if (bForceTeleportResetOnly)
				{
					ClothSimulationProxy->ForcePendingReset_GameThread();
				}
				else
				{
					ClothSimulationProxy->PreProcess_GameThread(InDeltaTime);
					ClothSimulationProxy->PreSimulate_GameThread(InDeltaTime);

					// Reset the bNeedsResetRestLengths at every pass
					bNeedsResetRestLengths = false;
				}
			}
		}

	}

	void FSKMClothingSimulation::CreateActor(USkeletalMeshComponent* InOwnerComponent, const UClothingAssetBase* ClothingAsset, int32 SimDataIndex)
	{
		check(InOwnerComponent);
		if (const UChaosClothAssetSKMClothingAsset* const ClothAssetSKMClothingAsset = Cast<UChaosClothAssetSKMClothingAsset>(ClothingAsset))
		{
			if (const int32 ClothSimulationModelIndex = ClothAssetSKMClothingAsset->GetClothSimulationModelIndex(); ClothSimulationModelIndex != INDEX_NONE)
			{
				const UChaosClothAssetBase* const ClothAsset = ClothAssetSKMClothingAsset->GetAsset();
				check(ClothAsset);

				const int32 NumClothSimulationModels = ClothAsset->GetNumClothSimulationModels();
				check(ClothSimulationModelIndex < NumClothSimulationModels);

				// Keep track of the component
				if (!OwnerComponent || OwnerComponent != InOwnerComponent)
				{
					checkf(!OwnerComponent, TEXT("IClothingSimulationInterface::CreateActor() can't be called from multiple components."));
					OwnerComponent = InOwnerComponent;

					// Set the collision sources when CreateActor is setting the component
					checkf(CollisionSources, TEXT("IClothingSimulationInterface::Initialize() must be called before calling IClothingSimulationInterface::CreateActor()."));
					CollisionSources->SetOwnerComponent(OwnerComponent);
					CollisionSources->SetCollideWithEnvironment(OwnerComponent->bCollideWithEnvironment);
				}

				// Add this asset to the asset list, matching its SimDataIndex position in the array
				const int32 PrevNum = AssetClothSimulationProperties.Num();
				AssetClothSimulationProperties.Reserve(SimDataIndex + 1);
				for (int32 Index = PrevNum; Index <= SimDataIndex; ++Index)
				{
					AssetClothSimulationProperties.Emplace(nullptr, FChaosClothSimulationProperties());
				}

				// Set the properties for this SimDataIndex, but only if the ClothAsset/OutfitPiece isn't already used (the same cloth asset can be used in multiple render sections)
				auto ClothAssetPredicate = [ClothAsset, ClothSimulationModelIndex](const FAssetClothSimulationProperties& Properties)
					{
						return Properties.Key &&
							Properties.Key->GetAsset() == ClothAsset &&
							Properties.Key->GetClothSimulationModelIndex() == ClothSimulationModelIndex;
					};

				const int32 FirstSimDataIndex = AssetClothSimulationProperties.IndexOfByPredicate(ClothAssetPredicate);
				if (FirstSimDataIndex == INDEX_NONE)
				{
					FChaosClothSimulationProperties& ClothSimulationProperties = AssetClothSimulationProperties[SimDataIndex].Value;

					ClothSimulationProperties.Initialize(ClothAsset->GetCollections(ClothSimulationModelIndex));
				}
				else if (!ensureMsgf(SimDataIndex > FirstSimDataIndex, TEXT("CreateActor should be called with ascending SimDataIndex")))
				{
					Swap(AssetClothSimulationProperties[SimDataIndex].Value, AssetClothSimulationProperties[FirstSimDataIndex].Value);  // Otherwise the properties must be swapped
				}

				AssetClothSimulationProperties[SimDataIndex].Key = ClothAssetSKMClothingAsset;
			}
		}
	}

	void FSKMClothingSimulation::EndCreateActor()
	{
		CalculateLODHasAnyRenderClothMappingData();

		if (ClothSimulationProxy.IsValid())
		{
			// Add all assets to the proxy
			ClothSimulationProxy->PostConstructor();
		}
	}

	void FSKMClothingSimulation::DestroyActors()
	{
		AssetClothSimulationProperties.Reset();
		LODHasAnyRenderClothMappingData.Reset();
	}

	bool FSKMClothingSimulation::ShouldSimulateLOD(int32 OwnerLODIndex) const
	{
		if (!OwnerComponent)
		{
			return false;
		}
		return LODHasAnyRenderClothMappingData.IsValidIndex(OwnerLODIndex) && LODHasAnyRenderClothMappingData[OwnerLODIndex] && HasAnySimulationMeshData(OwnerLODIndex);
	}

	void FSKMClothingSimulation::Simulate_AnyThread(const IClothingSimulationContext* Context)
	{
		if (const FSKMClothingSimulationContext* const SKMClothingSimulationContext = static_cast<const FSKMClothingSimulationContext*>(Context))
		{
			if (ClothSimulationProxy.IsValid())
			{
				ClothSimulationProxy->Tick();
			}
		}
	}
	
	void FSKMClothingSimulation::ForceClothNextUpdateTeleportAndReset_AnyThread()
	{
		// Do nothing here. This is handled in FillContextAndPrepareTick
	}

	void FSKMClothingSimulation::HardResetSimulation(const IClothingSimulationContext* InContext)
	{
		ResetConfigProperties();
		if (ClothSimulationProxy.IsValid())
		{
			ClothSimulationProxy->HardResetSimulation_GameThread();
		}
	}

	void FSKMClothingSimulation::AppendSimulationData(TMap<int32, FClothSimulData>& InOutData, const USkeletalMeshComponent* /*InOwnerComponent*/, const USkinnedMeshComponent* /*OverrideComponent*/) const
	{
		if (ClothSimulationProxy.IsValid())
		{
			ClothSimulationProxy->PostSimulate_GameThread(&InOutData);
			ClothSimulationProxy->PostProcess_GameThread(&InOutData);
		}
	}

	FBoxSphereBounds FSKMClothingSimulation::GetBounds(const USkeletalMeshComponent* /*InOwnerComponent*/) const
	{
		if (ClothSimulationProxy.IsValid())
		{
			return ClothSimulationProxy->CalculateBounds_AnyThread();
		}
		return FBoxSphereBounds(ForceInit);
	}

	void FSKMClothingSimulation::AddExternalCollisions(const FClothCollisionData& Data)
	{
		if (CollisionSources)
		{
			CollisionSources->AddExternalCollisions(Data);
		}
	}

	void FSKMClothingSimulation::ClearExternalCollisions()
	{
		if (CollisionSources)
		{
			CollisionSources->ClearExternalCollisions();
		}
	}

	void FSKMClothingSimulation::SetNumIterations(int32 NumIterations)
	{
		for (const TSharedPtr<::Chaos::Softs::FCollectionPropertyFacade>& SolverPropertyFacade : GetSolverPropertyFacades())
		{
			if (SolverPropertyFacade)
			{
				SolverPropertyFacade->SetValue(::Chaos::FClothingSimulationSolver::NumIterationsName, NumIterations);
			}
		}
	}

	void FSKMClothingSimulation::SetMaxNumIterations(int32 MaxNumIterations)
	{
		for (const TSharedPtr<::Chaos::Softs::FCollectionPropertyFacade>& SolverPropertyFacade : GetSolverPropertyFacades())
		{
			if (SolverPropertyFacade)
			{
				SolverPropertyFacade->SetValue(::Chaos::FClothingSimulationSolver::MaxNumIterationsName, MaxNumIterations);
			}
		}
	}

	void FSKMClothingSimulation::SetNumSubsteps(int32 NumSubsteps)
	{
		for (const TSharedPtr<::Chaos::Softs::FCollectionPropertyFacade>& SolverPropertyFacade : GetSolverPropertyFacades())
		{
			if (SolverPropertyFacade)
			{
				SolverPropertyFacade->SetValue(::Chaos::FClothingSimulationSolver::NumSubstepsName, NumSubsteps);
			}
		}
	}

	const USkinnedMeshComponent& FSKMClothingSimulation::GetOwnerComponent() const
	{
		checkf(OwnerComponent, TEXT("No actors, owner component not set. Call CreateActor first."));
		return *Cast<USkinnedMeshComponent>(OwnerComponent);
	}

	TArray<const UChaosClothAssetBase*> FSKMClothingSimulation::GetAssets() const
	{
		TArray<const UChaosClothAssetBase*> Assets;
		Assets.Reserve(AssetClothSimulationProperties.Num());
		for (const FAssetClothSimulationProperties& Properties : AssetClothSimulationProperties)
		{
			if (const UChaosClothAssetSKMClothingAsset* const ClothAssetSKMClothingAsset = Properties.Key)
			{
				if (const UChaosClothAssetBase* const Asset = ClothAssetSKMClothingAsset->GetAsset())
				{
					Assets.AddUnique(Asset);
				}
			}
		}
		return Assets;
	}

	TSharedPtr<const FChaosClothSimulationModel> FSKMClothingSimulation::GetActiveClothSimulationModel(const UChaosClothAssetBase* Asset, int32 ModelIndex) const
	{

		for (const FAssetClothSimulationProperties& SimProps : AssetClothSimulationProperties)
		{
			const UChaosClothAssetSKMClothingAsset* SKMClothingAsset = SimProps.Key;

			if (!SKMClothingAsset)
			{
				continue;
			}

			if (!(SKMClothingAsset->GetAsset() == Asset && SKMClothingAsset->GetClothSimulationModelIndex() == ModelIndex))
			{
				continue;
			}

			TSharedPtr<const FChaosClothSimulationModel> TransientModel = SKMClothingAsset->GetTransientSimulationModel();

			if (TransientModel)
			{
				return TransientModel;
			}
			
			break;
		}

		// If no transient model found, fallback to the default behaviour. 
		return IClothComponentAdapter::GetActiveClothSimulationModel(Asset, ModelIndex);
	}

	int32 FSKMClothingSimulation::GetSimulationGroupId(const UChaosClothAssetBase* Asset, int32 ModelIndex) const
	{
		for (int32 Index = 0; Index < AssetClothSimulationProperties.Num(); ++Index)
		{
			const FAssetClothSimulationProperties& Properties = AssetClothSimulationProperties[Index];
			if (const UChaosClothAssetSKMClothingAsset* const ClothAssetSKMClothingAsset = Properties.Key)
			{
				if (ClothAssetSKMClothingAsset->GetAsset() == Asset &&
					ClothAssetSKMClothingAsset->GetClothSimulationModelIndex() == ModelIndex)
				{
					return Index;
				}
			}
		}
		return INDEX_NONE;
	}

	int32 FSKMClothingSimulation::GetNextLinkedSimulationGroupId(int32 PrevSimulationGroupId) const
	{
		if (AssetClothSimulationProperties.IsValidIndex(PrevSimulationGroupId))
		{
			if (const UChaosClothAssetSKMClothingAsset* const ClothAssetSKMClothingAsset = AssetClothSimulationProperties[PrevSimulationGroupId].Key)
			{
				if (const UChaosClothAssetBase* const Asset = ClothAssetSKMClothingAsset->GetAsset())
				{
					const int32 ModelIndex = ClothAssetSKMClothingAsset->GetClothSimulationModelIndex();
					for (int32 Index = PrevSimulationGroupId + 1; Index < AssetClothSimulationProperties.Num(); ++Index)
					{
						if (const UChaosClothAssetSKMClothingAsset* const NextClothAssetSKMClothingAsset = AssetClothSimulationProperties[Index].Key)
						{
							if (NextClothAssetSKMClothingAsset->GetAsset() == Asset &&
								NextClothAssetSKMClothingAsset->GetClothSimulationModelIndex() == ModelIndex)
							{
								return Index;
							}
						}
					}
				}
			}
		}
		return INDEX_NONE;
	}

	const FReferenceSkeleton* FSKMClothingSimulation::GetReferenceSkeleton() const
	{
		checkf(OwnerComponent, TEXT("No actors, owner component not set. Call CreateActor first."));
		return OwnerComponent->GetSkinnedAsset() ? &OwnerComponent->GetSkinnedAsset()->GetRefSkeleton() : nullptr;
	}

	EClothingTeleportMode FSKMClothingSimulation::GetClothTeleportMode() const
	{
		checkf(OwnerComponent, TEXT("No actors, owner component not set. Call CreateActor first."));
		return OwnerComponent->ClothTeleportMode;
	}

	bool FSKMClothingSimulation::IsSimulationEnabled() const
	{
		checkf(OwnerComponent, TEXT("No actors, owner component not set. Call CreateActor first."));
		return OwnerComponent->CanSimulateClothing();
	}

	bool FSKMClothingSimulation::IsSimulationSuspended() const
	{
		checkf(OwnerComponent, TEXT("No actors, owner component not set. Call CreateActor first."));
		return OwnerComponent->IsClothingSimulationSuspended();
	}

	float FSKMClothingSimulation::GetClothGeometryScale() const
	{
		checkf(OwnerComponent, TEXT("No actors, owner component not set. Call CreateActor first."));
		return OwnerComponent->ClothGeometryScale;
	}

	const TArray<TSharedPtr<const FManagedArrayCollection>>& FSKMClothingSimulation::GetPropertyCollections(const UChaosClothAssetBase* Asset, int32 ModelIndex) const
	{
		const int32 SimulationGroupId = GetSimulationGroupId(Asset, ModelIndex);
		checkf(SimulationGroupId != INDEX_NONE, TEXT("Invalid arguments"));
		return AssetClothSimulationProperties[SimulationGroupId].Value.PropertyCollections;
	}

	const TArray<TSharedPtr<const FManagedArrayCollection>>& FSKMClothingSimulation::GetSolverPropertyCollections() const
	{
		if (const FChaosClothSimulationProperties* const SolverProperties = FSKMClothingSimulation::GetSolverProperties())
		{
			return SolverProperties->PropertyCollections;
		}
		static const TArray<TSharedPtr<const FManagedArrayCollection>> EmptyManagedArrayCollectionArray;
		return EmptyManagedArrayCollectionArray;
	}

	void FSKMClothingSimulation::ClearPropertyCollectionsDirtyFlags(int32 LodIndex) const
	{
		for (const FAssetClothSimulationProperties& Properties : AssetClothSimulationProperties)
		{
			if (const UChaosClothAssetSKMClothingAsset* const ClothAssetSKMClothingAsset = Properties.Key)
			{
				if (Properties.Value.CollectionPropertyFacades.IsValidIndex(LodIndex))
				{
					const TSharedPtr<::Chaos::Softs::FCollectionPropertyFacade>& CollectionPropertyFacade = Properties.Value.CollectionPropertyFacades[LodIndex];
					if (CollectionPropertyFacade.IsValid())
					{
						CollectionPropertyFacade->ClearDirtyFlags();
					}
				}
			}
		}
	}

	bool FSKMClothingSimulation::HasAnySimulationMeshData(int32 LodIndex) const
	{
		// Note that for ClothAsset-based simulation, OwnerLODIndex == ClothAssetLODIndex
		for (const FAssetClothSimulationProperties& Properties : AssetClothSimulationProperties)
		{
			if (const UChaosClothAssetSKMClothingAsset* const ClothAssetSKMClothingAsset = Properties.Key)
			{
				if (ClothAssetSKMClothingAsset->HasAnySimulationMeshData(LodIndex))
				{
					return true;
				}
			}
		}
		return false;
	}

	void FSKMClothingSimulation::AddReferencedObjects(FReferenceCollector& Collector)
	{
		for (FAssetClothSimulationProperties& Properties : AssetClothSimulationProperties)
		{
			if (Properties.Key && Properties.Value.ClothOutfitInteractor)
			{
				Collector.AddReferencedObject(Properties.Value.ClothOutfitInteractor);
			}
		}
	}

	FString FSKMClothingSimulation::GetReferencerName() const
	{
		return TEXT("UE::Chaos::ClothAsset::FSKMClothingSimulation");
	}

	const FChaosClothSimulationProperties* FSKMClothingSimulation::GetSolverProperties() const
	{
		for (const FAssetClothSimulationProperties& Properties : AssetClothSimulationProperties)
		{
			if (const UChaosClothAssetSKMClothingAsset* const ClothAssetSKMClothingAsset = Properties.Key)
			{
				if (ClothAssetSKMClothingAsset->GetAsset())
				{
					return &Properties.Value;  // Return the properties for the first valid asset for now, there might be way to improve this in the future
				}
			}
		}
		return nullptr;
	}

	const TArray<TSharedPtr<::Chaos::Softs::FCollectionPropertyFacade>>& FSKMClothingSimulation::GetSolverPropertyFacades() const
	{
		if (const FChaosClothSimulationProperties* const SolverProperties = FSKMClothingSimulation::GetSolverProperties())
		{
			return SolverProperties->CollectionPropertyFacades;
		}
		static const TArray<TSharedPtr<::Chaos::Softs::FCollectionPropertyFacade>> EmptyManagedArrayCollectionArray;
		return EmptyManagedArrayCollectionArray;
	}

	void FSKMClothingSimulation::CalculateLODHasAnyRenderClothMappingData()
	{
		LODHasAnyRenderClothMappingData.Reset();
		if (OwnerComponent)
		{
			if (const USkeletalMesh* const SkeletalMesh = OwnerComponent->GetSkeletalMeshAsset())
			{
				if (const FSkeletalMeshRenderData* const SkeletalMeshRenderData = SkeletalMesh->GetResourceForRendering())
				{
					LODHasAnyRenderClothMappingData.Init(false, SkeletalMeshRenderData->LODRenderData.Num());
					for (int32 LodIndex = 0; LodIndex < SkeletalMeshRenderData->LODRenderData.Num(); ++LodIndex)
					{
						for (const FSkelMeshRenderSection& SkelMeshRenderSection : SkeletalMeshRenderData->LODRenderData[LodIndex].RenderSections)
						{
							if (SkelMeshRenderSection.HasClothingData() &&
								AssetClothSimulationProperties.IsValidIndex(SkelMeshRenderSection.CorrespondClothAssetIndex))
							{
								if (const UChaosClothAssetSKMClothingAsset* const ClothAssetSKMClothingAsset = AssetClothSimulationProperties[SkelMeshRenderSection.CorrespondClothAssetIndex].Key)
								{
									if (SkelMeshRenderSection.ClothingData.AssetGuid == ClothAssetSKMClothingAsset->GetAssetGuid() &&
										SkelMeshRenderSection.ClothingData.AssetLodIndex == LodIndex)
									{
										LODHasAnyRenderClothMappingData[LodIndex] = true;
										continue;
									}
								}
							}
						}
					}
				}
			}
		}
	}
}  // namespace UE::Chaos::ClothAsset

UChaosClothAssetSKMClothingSimulationFactory::UChaosClothAssetSKMClothingSimulationFactory() = default;
UChaosClothAssetSKMClothingSimulationFactory::~UChaosClothAssetSKMClothingSimulationFactory() = default;

bool UChaosClothAssetSKMClothingSimulationFactory::SupportsAsset(const UClothingAssetBase* Asset) const
{
	return Cast<UChaosClothAssetSKMClothingAsset>(Asset) != nullptr;
}

UClothingSimulationInteractor* UChaosClothAssetSKMClothingSimulationFactory::CreateInteractor()
{
	return NewObject<UChaosClothAssetSKMClothingSimulationInteractor>(GetTransientPackage());
}
