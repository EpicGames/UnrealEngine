// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ClothAssetSKMClothingSimulationInteractor.h"
#include "ChaosClothAsset/ClothAssetInteractor.h"
#include "ChaosClothAsset/ClothAssetSKMClothingAsset.h"
#include "ChaosClothAsset/ClothAssetSKMClothingSimulation.h"
#include "ChaosClothAsset/ClothAssetPrivate.h"
#include "Chaos/PBDAnimDriveConstraint.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClothAssetSKMClothingSimulationInteractor)

void UChaosClothAssetSKMClothingInteractor::Initialize(const UChaosClothAssetSKMClothingAsset* InClothingAsset, int32 InClothingId)
{
	ClothingId = InClothingId;
	ClothingAsset = InClothingAsset;
}

void UChaosClothAssetSKMClothingSimulationInteractor::Sync(IClothingSimulationInterface* Simulation, const IClothingSimulationContext* Context)
{
	using namespace UE::Chaos::ClothAsset;

	check(Simulation);
	check(Context);

	FSKMClothingSimulation* const SKMClothingSimulation = static_cast<FSKMClothingSimulation*>(Simulation);

	// Sync interactors
	for (TPair<FName, TObjectPtr<UClothingInteractor>>& ClothingInteractor : ClothingInteractors)
	{
		if (const UChaosClothAssetSKMClothingInteractor* const DummyClothingInteractor = Cast<UChaosClothAssetSKMClothingInteractor>(ClothingInteractor.Value))
		{
			if (const UChaosClothAssetSKMClothingAsset* const ClothingAsset = DummyClothingInteractor->ClothingAsset)
			{
				if (const int32 ModelIndex = ClothingAsset->GetClothSimulationModelIndex(); ModelIndex != INDEX_NONE)
				{
					ClothingInteractor.Value = SKMClothingSimulation->GetPropertyInteractor(ClothingAsset->GetAsset(), ModelIndex);
				}
			}
		}
		// Note: The iteration could theorically stop at the first non dummy interactor, but since the creation is done outside this file it is better not
		//       to assume all of them are being created at the same time, especially as there will only be a couple of these interactors in most cases.
	}

	// Execute commands
	for (FCommand& Command : Commands)
	{
		Command.Execute(SKMClothingSimulation);
	}
	Commands.Reset();

	// Call base class' sync 
	UClothingSimulationInteractor::Sync(Simulation, Context);
}

void UChaosClothAssetSKMClothingSimulationInteractor::PhysicsAssetUpdated()
{
	using namespace UE::Chaos::ClothAsset;
	Commands.Add(FCommand::CreateLambda([](FSKMClothingSimulation* SKMClothingSimulation)
		{
			SKMClothingSimulation->RecreateClothSimulationProxy();
		}));
}

void UChaosClothAssetSKMClothingSimulationInteractor::ClothConfigUpdated()
{
	using namespace UE::Chaos::ClothAsset;
	Commands.Add(FCommand::CreateLambda([](FSKMClothingSimulation* SKMClothingSimulation)
		{
			SKMClothingSimulation->ResetConfigProperties();
		}));
}

void UChaosClothAssetSKMClothingSimulationInteractor::SetAnimDriveSpringStiffness(float Stiffness)
{
	for (TPair<FName, TObjectPtr<UClothingInteractor>>& ClothingInteractor : ClothingInteractors)
	{
		if (UChaosClothAssetInteractor* const ClothAssetInteractor = Cast<UChaosClothAssetInteractor>(ClothingInteractor.Value))
		{
			constexpr int32 AllLODIndices = -1;
			ClothAssetInteractor->SetFloatPropertyValue(::Chaos::Softs::FPBDAnimDriveConstraint::AnimDriveStiffnessName, AllLODIndices, Stiffness);
		}
	}
}

void UChaosClothAssetSKMClothingSimulationInteractor::EnableGravityOverride(const FVector& Gravity)
{
	UE_LOGF(LogChaosClothAsset, Warning, "EnableGravityOverride is not implemented for the Cloth Asset Clothing Simulation Interactor.");
}

void UChaosClothAssetSKMClothingSimulationInteractor::DisableGravityOverride()
{
	UE_LOGF(LogChaosClothAsset, Warning, "DisableGravityOverride is not implemented for the Cloth Asset Clothing Simulation Interactor.");
}

void UChaosClothAssetSKMClothingSimulationInteractor::SetNumIterations(int32 NumIterations)
{
	using namespace UE::Chaos::ClothAsset;
	Commands.Add(FCommand::CreateLambda([NumIterations](FSKMClothingSimulation* SKMClothingSimulation)
		{
			if (SKMClothingSimulation)
			{
				SKMClothingSimulation->SetNumIterations(NumIterations);
			}
		}));
}

void UChaosClothAssetSKMClothingSimulationInteractor::SetMaxNumIterations(int32 MaxNumIterations)
{
	using namespace UE::Chaos::ClothAsset;
	Commands.Add(FCommand::CreateLambda([MaxNumIterations](FSKMClothingSimulation* SKMClothingSimulation)
		{
			if (SKMClothingSimulation)
			{
				SKMClothingSimulation->SetMaxNumIterations(MaxNumIterations);
			}
		}));
}

void UChaosClothAssetSKMClothingSimulationInteractor::SetNumSubsteps(int32 NumSubsteps)
{
	using namespace UE::Chaos::ClothAsset;
	Commands.Add(FCommand::CreateLambda([NumSubsteps](FSKMClothingSimulation* SKMClothingSimulation)
		{
			if (SKMClothingSimulation)
			{
				SKMClothingSimulation->SetNumSubsteps(NumSubsteps);
			}
		}));
}

void UChaosClothAssetSKMClothingSimulationInteractor::CreateClothingInteractor(const UClothingAssetBase* ClothingAsset, int32 ClothingId)
{
	if (const UChaosClothAssetSKMClothingAsset* const SKMClothingAsset = Cast<const UChaosClothAssetSKMClothingAsset>(ClothingAsset))
	{
		UChaosClothAssetSKMClothingInteractor* const ClothingInteractor = NewObject<UChaosClothAssetSKMClothingInteractor>(this);
		check(ClothingInteractor);
		ClothingInteractor->Initialize(SKMClothingAsset, ClothingId);

		ClothingInteractors.Emplace(ClothingAsset->GetName(), ClothingInteractor);  // The actual interactors will be populated during the Sync call since it needs the simulation pointer
	}
}
