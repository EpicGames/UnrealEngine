// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ClothingSimulationInteractor.h"
#include "ClothAssetSKMClothingSimulationInteractor.generated.h"

namespace UE::Chaos::ClothAsset
{
	class FSKMClothingSimulation;
}
class UChaosClothAssetSKMClothingAsset;

/** Dummy interactor not exposed to Blueprint used during initializations, until the correct clothing interactor is known from the simualtion. */
UCLASS(Transient, NotBlueprintType, Hidden, HideDropdown, MinimalAPI)
class UChaosClothAssetSKMClothingInteractor final : public UClothingInteractor
{
	GENERATED_BODY()

private:
	friend UChaosClothAssetSKMClothingSimulationInteractor;

	void Initialize(const UChaosClothAssetSKMClothingAsset* InClothingAsset, int32 InClothingId);

	UPROPERTY()
	TObjectPtr<const UChaosClothAssetSKMClothingAsset> ClothingAsset;
};

/** Simulation interactor for Cloth Assets running through a Skeletal Mesh simulation. */
UCLASS(Transient, BlueprintType, MinimalAPI)
class UChaosClothAssetSKMClothingSimulationInteractor final : public UClothingSimulationInteractor
{
	GENERATED_BODY()

private:
	DECLARE_DELEGATE_OneParam(FCommand, UE::Chaos::ClothAsset::FSKMClothingSimulation*)

	//~ Begin UClothingSimulationInteractor Interface
	virtual void CreateClothingInteractor(const UClothingAssetBase* ClothingAsset, int32 ClothingId) override;
	virtual void Sync(IClothingSimulationInterface* InSimulation, const IClothingSimulationContext* InContext) override;

	virtual void PhysicsAssetUpdated() override;
	virtual void ClothConfigUpdated() override;
	virtual void SetAnimDriveSpringStiffness(float InStiffness) override;
	virtual void EnableGravityOverride(const FVector& InVector) override;
	virtual void DisableGravityOverride() override;

	virtual void SetNumIterations(int32 NumIterations) override;
	virtual void SetMaxNumIterations(int32 MaxNumIterations) override;
	virtual void SetNumSubsteps(int32 NumSubsteps) override;
	//~ End UClothingSimulationInteractor Interface

	TArray<FCommand> Commands;
};
