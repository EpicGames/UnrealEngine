// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ClothComponent.h"
#include "ChaosClothAsset/ClothAsset.h"
#include "ChaosClothAsset/ClothAssetInteractor.h"
#include "ChaosClothAsset/ClothAssetPrivate.h"
#include "ChaosClothAsset/ClothComponentInstanceData.h"
#include "ChaosClothAsset/CollisionSources.h"
#include "ChaosClothAsset/ClothSimulationModel.h"
#include "ChaosClothAsset/ClothSimulationProxy.h"
#include "Chaos/CollectionPropertyFacade.h"
#include "ClothingSimulationTeleportHelpers.h"
#include "Components/SkeletalMeshComponent.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "HAL/IConsoleManager.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "SkeletalRenderPublic.h"
#include "Dataflow/DataflowSimulationManager.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "Stats/Stats.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClothComponent)

CSV_DECLARE_CATEGORY_MODULE_EXTERN(ENGINE_API, Animation);

void FChaosClothSimulationProperties::Initialize(const TArray<TSharedRef<const FManagedArrayCollection>>& AssetCollections)
{
	using namespace ::Chaos::Softs;
	PropertyCollections.Reset(AssetCollections.Num());
	CollectionPropertyFacades.Reset(AssetCollections.Num());
	for (const TSharedRef<const FManagedArrayCollection>& AssetCollection : AssetCollections)
	{
		TSharedPtr<FManagedArrayCollection> PropertyCollection = MakeShared<FManagedArrayCollection>();
		TSharedRef<FCollectionPropertyMutableFacade> CollectionPropertyMutableFacade = MakeShared<FCollectionPropertyMutableFacade>(PropertyCollection);
		CollectionPropertyMutableFacade->Copy(*AssetCollection);  // This only copy properties from the asset's collection (in case it also contains other groups)
		CollectionPropertyFacades.Emplace(StaticCastSharedRef<FCollectionPropertyFacade>(CollectionPropertyMutableFacade));  // Mutable facades conveniently inherits from non mutable ones
		PropertyCollections.Emplace(MoveTemp(PropertyCollection));
	}
	if (!ClothOutfitInteractor)  // No need to recreate the interactor, it only needs new property facades, and it is also required for config updates on the SKM implementation to keep the same pointer
	{
		ClothOutfitInteractor = NewObject<UChaosClothAssetInteractor>();
	}
	ClothOutfitInteractor->SetProperties(CollectionPropertyFacades);
}

UChaosClothComponent::UChaosClothComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bUseAttachedParentAsPoseComponent(1)  // By default use the parent component as leader pose component
	, bWaitForParallelTask(0)
	, bEnableSimulation(1)
	, bSuspendSimulation(0)
	, bBindToLeaderComponent(0)
	, bTeleport(0)
	, bReset(1)
	, bCollideWithEnvironment(0)
#if WITH_EDITORONLY_DATA
	, bSimulateInEditor(0)
#endif
	, TeleportDistanceThreshold(300.f)
	, TeleportRotationThreshold(0.f)
	, ClothTeleportDistThresholdSquared(UE::ClothingSimulation::TeleportHelpers::ComputeTeleportDistanceThresholdSquared(TeleportDistanceThreshold))
	, ClothTeleportCosineThresholdInRad(UE::ClothingSimulation::TeleportHelpers::ComputeTeleportCosineRotationThreshold(TeleportRotationThreshold))
	, ClothTeleportMode(EClothingTeleportMode::None)
	, CollisionSources(MakeUnique<UE::Chaos::ClothAsset::FCollisionSources>(this, (bool)bCollideWithEnvironment))
{
	PrimaryComponentTick.EndTickGroup = TG_PostPhysics;

	// End tick runs after every other tick group to wait on the parallel cloth task without pumping the EOF queue.
	EndTickFunction.TickGroup = TG_LastDemotable;
	EndTickFunction.EndTickGroup = TG_LastDemotable;
	EndTickFunction.bCanEverTick = true;
	EndTickFunction.bStartWithTickEnabled = true;

	PrevRootBoneMatrix = GetBoneMatrix(0);
}

UChaosClothComponent::UChaosClothComponent(FVTableHelper& Helper)
	: Super(Helper)
{
}

UChaosClothComponent::~UChaosClothComponent() = default;

void UChaosClothComponent::SetAsset(UChaosClothAssetBase* InAsset)
{
	SetSkinnedAssetAndUpdate(InAsset);

#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Asset = InAsset;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
}

UChaosClothAssetBase* UChaosClothComponent::GetAsset() const
{
	return Cast<UChaosClothAssetBase>(GetSkinnedAsset());
}

void UChaosClothComponent::SetClothAsset(UChaosClothAsset* InClothAsset)
{
	SetAsset(InClothAsset);
}

UChaosClothAsset* UChaosClothComponent::GetClothAsset() const
{
	return Cast<UChaosClothAsset>(GetAsset());
}

bool UChaosClothComponent::IsSimulationSuspended() const
{
	return bSuspendSimulation || !IsSimulationEnabled();
}

bool UChaosClothComponent::IsSimulationEnabled() const
{
	static IConsoleVariable* const CVarClothPhysics = IConsoleManager::Get().FindConsoleVariable(TEXT("p.ClothPhysics"));
	// If the console variable doesn't exist, default to simulation enabled.
	return bEnableSimulation && ClothSimulationProxy.IsValid() && (!CVarClothPhysics || CVarClothPhysics->GetBool());
}

void UChaosClothComponent::ResetConfigProperties()
{
	ClothSimulationProperties.Reset();

	if (IsRegistered())
	{
		if (GetAsset())
		{
			const int32 NumClothSimulationModels = GetAsset()->GetNumClothSimulationModels();
			ClothSimulationProperties.SetNum(NumClothSimulationModels);

			for (int32 ModelIndex = 0; ModelIndex < NumClothSimulationModels; ++ModelIndex)
			{
				ClothSimulationProperties[ModelIndex].Initialize(GetAsset()->GetCollections(ModelIndex));
			}
		}
	}
	else
	{
		UE_LOGF(LogChaosClothAsset, Warning, "Chaos Cloth Component [%ls]: Trying to reset runtime config properties without being registered.", *GetName());
	}
}

void UChaosClothComponent::UpdateConfigProperties()
{
	using namespace ::Chaos::Softs;
	if (IsRegistered())
	{
		if (GetAsset() && GetAsset()->GetNumClothSimulationModels() == ClothSimulationProperties.Num())
		{
			for (int32 ModelIndex = 0; ModelIndex < ClothSimulationProperties.Num(); ++ModelIndex)
			{
				const TArray<TSharedRef<const FManagedArrayCollection>>& AssetCollections = GetAsset()->GetCollections(ModelIndex);
				TArray<TSharedPtr<const FManagedArrayCollection>>& PropertyCollections = ClothSimulationProperties[ModelIndex].PropertyCollections;
				if (AssetCollections.Num() == PropertyCollections.Num())
				{
					TArray<TSharedPtr<FCollectionPropertyFacade>>& CollectionPropertyFacades = ClothSimulationProperties[ModelIndex].CollectionPropertyFacades;
					check(CollectionPropertyFacades.Num() == AssetCollections.Num());
					for (int32 LODIndex = 0; LODIndex < AssetCollections.Num(); ++LODIndex)
					{
						CollectionPropertyFacades[LODIndex]->UpdateProperties(AssetCollections[LODIndex].ToSharedPtr());
					}
				}
			}
		}
	}
}

void UChaosClothComponent::WaitForExistingParallelClothSimulation_GameThread()
{
	// Should only kick new parallel cloth simulations from game thread, so should be safe to also wait for existing ones there.
	check(IsInGameThread());
	HandleExistingParallelSimulation();
}

void UChaosClothComponent::RecreateClothSimulationProxy()
{
	if (IsRegistered())
	{
		ClothSimulationProxy.Reset();

		if (GetAsset() && GetAsset()->HasValidClothSimulationModels())
		{
			// Create the simulation proxy (note CreateClothSimulationProxy() can be overloaded)
			CreateClothSimulationProxyImpl();
		}
	}
	else
	{
		UE_LOGF(LogChaosClothAsset, Warning, "Chaos Cloth Component [%ls]: Trying to recreate the simulation proxy without being registered.", *GetName());
	}
}

void UChaosClothComponent::PostLoad()
{
	Super::PostLoad();

	RefreshAsset();
}

void UChaosClothComponent::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);

	RefreshAsset();
}

void UChaosClothComponent::RefreshAsset()
{
#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Asset = GetAsset();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
}

#if WITH_EDITOR
void UChaosClothComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (const FProperty* const Property = PropertyChangedEvent.Property)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (Property->GetFName() == GET_MEMBER_NAME_CHECKED(UChaosClothComponent, Asset))
		{
			SetAsset(Asset);  // Do not remove, ImportText_Direct skips the UPROPERTY Setter
		}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		if (Property->GetFName() == GET_MEMBER_NAME_CHECKED(UChaosClothComponent, bSimulateInEditor))
		{
			SetSimulateInEditor(bSimulateInEditor);  // Do not remove, ImportText_Direct skips the UPROPERTY Setter
		}
		if (Property->GetFName() == GET_MEMBER_NAME_CHECKED(UChaosClothComponent, bCollideWithEnvironment))
		{
			CollisionSources->SetCollideWithEnvironment(bCollideWithEnvironment);
		}
		if (Property->GetFName() == GET_MEMBER_NAME_CHECKED(UChaosClothComponent, TeleportDistanceThreshold))
		{
			ClothTeleportDistThresholdSquared = UE::ClothingSimulation::TeleportHelpers::ComputeTeleportDistanceThresholdSquared(TeleportDistanceThreshold);
		}
		if (Property->GetFName() == GET_MEMBER_NAME_CHECKED(UChaosClothComponent, TeleportRotationThreshold))
		{
			ClothTeleportCosineThresholdInRad = UE::ClothingSimulation::TeleportHelpers::ComputeTeleportCosineRotationThreshold(TeleportRotationThreshold);
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UChaosClothComponent::PostEditImport()
{
	Super::PostEditImport();

	// Sync the alias after a duplicate or paste (PostDuplicate is not called on actor components)
	RefreshAsset();
}

bool UChaosClothComponent::CanEditChange(const FProperty* InProperty) const
{
	if (!Super::CanEditChange(InProperty))
	{
		return false;
	}

	const FName& Name = InProperty->GetFName();

	if (Name == GET_MEMBER_NAME_CHECKED(ThisClass, SimulationAsset))
	{
		static const auto CVarEnableSimulationDataflow = IConsoleManager::Get().FindConsoleVariable(TEXT("p.Dataflow.EnableSimulation"));
		return CVarEnableSimulationDataflow->GetBool();
	}

	return true;
}

TStructOnScope<FActorComponentInstanceData> UChaosClothComponent::GetComponentInstanceData() const
{
	return MakeStructOnScope<FActorComponentInstanceData, FChaosClothComponentInstanceData>(this);
}
#endif // WITH_EDITOR

void UChaosClothComponent::OnRegister()
{
	LLM_SCOPE(ELLMTag::Chaos);

	// Register the component first, otherwise calls to ResetConfigProperties and RecreateClothSimulationProxy wouldn't work
	Super::OnRegister();

	// Update the component bone transforms (for colliders) from the cloth asset until these are animated from a leader component
	UpdateComponentSpaceTransforms();

	// Fill up the property collection with the original cloth asset properties
	ResetConfigProperties();

	// Create the proxy to start the simulation
	RecreateClothSimulationProxy();

	// Update render visibility, so that an empty LODs doesn't unnecessarily go to render
	UpdateVisibility();

	// Update CollisionSources
	CollisionSources->SetCollideWithEnvironment(bCollideWithEnvironment);

	// Register the dataflow simulation interface
	UE::Dataflow::RegisterSimulationInterface(this);

	// Update teleport thresholds.
	ClothTeleportDistThresholdSquared = UE::ClothingSimulation::TeleportHelpers::ComputeTeleportDistanceThresholdSquared(TeleportDistanceThreshold);
	ClothTeleportCosineThresholdInRad = UE::ClothingSimulation::TeleportHelpers::ComputeTeleportCosineRotationThreshold(TeleportRotationThreshold);

	// Register the TG_LastDemotable wait tick when neither the in-tick wait nor the EOF-sync fallback is active.
	RegisterEndTick(ShouldUseEndTickWait() && !IsSimulationSuspended() && !ShouldWaitForParallelSimulationInTickComponent());
}

void UChaosClothComponent::SetComponentTickEnabled(bool bEnabled)
{
	Super::SetComponentTickEnabled(bEnabled);

	// Unregister the end tick when the component's tick is being disabled (it will be re-enabled on the next TickComponent / OnRegister once the component ticks again).
	if (!bEnabled)
	{
		RegisterEndTick(false);
	}
}

void UChaosClothComponent::OnUnregister()
{
	RegisterEndTick(false);

	Super::OnUnregister();

	// Release cloth simulation
	ClothSimulationProxy.Reset();

	// Release the runtime simulation interactors, collections, and facades
	ClothSimulationProperties.Reset();

	// Unregister the dataflow simulation interface
	UE::Dataflow::UnregisterSimulationInterface(this);
}

bool UChaosClothComponent::IsComponentTickEnabled() const
{
	bool bLeaderClothComponentTickEnabled = false;
	if (bBindToLeaderComponent)
	{
		if (UChaosClothComponent* const LeaderComponent = Cast<UChaosClothComponent>(LeaderPoseComponent.Get()))
		{
			bLeaderClothComponentTickEnabled = LeaderComponent->IsComponentTickEnabled();
		}
	}
	return (bEnableSimulation || bLeaderClothComponentTickEnabled) && Super::IsComponentTickEnabled();
}

void UChaosClothComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(Physics);
	QUICK_SCOPE_CYCLE_COUNTER(STAT_ClothComponentTick);

	// Re-evaluate the end-tick registration so p.ClothPhysics.WaitForParallelClothTask and p.ClothPhysics.UseEndTickWait toggles take effect at runtime (no-op when state matches).
	RegisterEndTick(ShouldUseEndTickWait() && !IsSimulationSuspended() && !ShouldWaitForParallelSimulationInTickComponent());

	// Tick USkinnedMeshComponent first so it will update the predicted lod
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);
	
	// Make sure that the previous frame simulation has completed
	HandleExistingParallelSimulation();

	if (IsSimulationEnabled())
	{
		UpdateClothTeleport();
	}
	
	if (!SimulationAsset.DataflowAsset)
	{
		// Update the proxy and start the simulation parallel task
		StartNewParallelSimulation(DeltaTime);

		// Wait in tick function for the simulation results if required
		if (ShouldWaitForParallelSimulationInTickComponent())
		{
			HandleExistingParallelSimulation();
		}
	}

#if WITH_EDITOR
	if (TickType == LEVELTICK_ViewportsOnly && bTickOnceInEditor && !bSimulateInEditor)
	{
		// Only tick once in editor when requested. This is used to update from caches by the Chaos Cache Manager.
		bTickInEditor = false;
		bTickOnceInEditor = false;
	}
#endif
	bResetRestLengthsFromMorphTarget = false;
}

bool UChaosClothComponent::RequiresPreEndOfFrameSync() const
{
	if (!ShouldUseEndTickWait() && !IsSimulationSuspended() && !ShouldWaitForParallelSimulationInTickComponent())
	{
		return true;
	}
	if (bBindToLeaderComponent)
	{
		if (UChaosClothComponent* const LeaderComponent = LeaderClothComponent.Get())
		{
			return LeaderComponent->RequiresPreEndOfFrameSync();
		}
	}
	return Super::RequiresPreEndOfFrameSync();
}

void UChaosClothComponent::OnPreEndOfFrameSync()
{
	Super::OnPreEndOfFrameSync();

	if (!ShouldUseEndTickWait())
	{
		HandleExistingParallelSimulation();
	}
}

bool UChaosClothComponent::ShouldUseEndTickWait() const
{
	// Defaults to true (the CVar default) when the CVar can't be located
	static IConsoleVariable* const CVarClothPhysicsUseEndTickWait = IConsoleManager::Get().FindConsoleVariable(TEXT("p.ClothPhysics.UseEndTickWait"));
	return !CVarClothPhysicsUseEndTickWait || CVarClothPhysicsUseEndTickWait->GetBool();
}

void FChaosClothComponentEndTickFunction::ExecuteTick(float DeltaTime, enum ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
{
	QUICK_SCOPE_CYCLE_COUNTER(FChaosClothComponentEndTickFunction_ExecuteTick);
	if (Target)
	{
		Target->HandleExistingParallelSimulation();
	}
}

FString FChaosClothComponentEndTickFunction::DiagnosticMessage()
{
	if (Target)
	{
		return Target->GetFullName() + TEXT("[ClothEndTick]");
	}
	return TEXT("<NULL>[ClothEndTick]");
}

FName FChaosClothComponentEndTickFunction::DiagnosticContext(bool bDetailed)
{
	return FName("ChaosClothComponentEndTick");
}

void UChaosClothComponent::RegisterEndTick(bool bRegister)
{
	if (bRegister != EndTickFunction.IsTickFunctionRegistered())
	{
		if (bRegister)
		{
			if (SetupActorComponentTickFunction(&EndTickFunction))
			{
				EndTickFunction.Target = this;
				// Cloth simulation is dispatched from TickComponent, so PrimaryComponentTick is the correct prerequisite. USkeletalMeshComponent has a separate ClothTickFunction and pairs its end tick with that one.
				EndTickFunction.AddPrerequisite(this, PrimaryComponentTick);
			}
		}
		else
		{
			EndTickFunction.UnRegisterTickFunction();
		}
	}
}

FBoxSphereBounds UChaosClothComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_CalcClothComponentBounds);

	FBoxSphereBounds NewBounds(ForceInitToZero);

	// Use cached local bounds if possible
	if (bCachedWorldSpaceBoundsUpToDate || bCachedLocalBoundsUpToDate)
	{
		NewBounds = bCachedLocalBoundsUpToDate ?
			CachedWorldOrLocalSpaceBounds.TransformBy(LocalToWorld) :
			CachedWorldOrLocalSpaceBounds.TransformBy(CachedWorldToLocalTransform * LocalToWorld.ToMatrixWithScale());
	
		if (bIncludeComponentLocationIntoBounds)
		{
			NewBounds = NewBounds + FBoxSphereBounds(GetComponentLocation(), FVector(1.0f), 1.0f);
		}
	}
	else  // Calculate new bounds
	{
		FVector RootBoneOffset(ForceInitToZero);

		// If attached to a skeletal mesh component that uses fixed bounds, add the root bone translation
		if (const USkeletalMeshComponent* const SkeletalMeshComponent = Cast<USkeletalMeshComponent>(LeaderPoseComponent.Get()))
		{
			if (SkeletalMeshComponent->GetSkinnedAsset() && SkeletalMeshComponent->bComponentUseFixedSkelBounds)
			{
				RootBoneOffset = SkeletalMeshComponent->RootBoneTranslation; // Adjust bounds by root bone translation
			}
		}

		static IConsoleVariable* const CVarCacheLocalSpaceBounds = IConsoleManager::Get().FindConsoleVariable(TEXT("a.CacheLocalSpaceBounds"));
		const bool bCacheLocalSpaceBounds = CVarCacheLocalSpaceBounds ? (CVarCacheLocalSpaceBounds->GetInt() != 0) : true;

		const FTransform CachedBoundsTransform = bCacheLocalSpaceBounds ? FTransform::Identity : LocalToWorld;

		// Add render mesh bounds
		constexpr bool bHasValidBodies = false;
		NewBounds = CalcMeshBound((FVector3f)RootBoneOffset, bHasValidBodies, CachedBoundsTransform);

		if (bIncludeComponentLocationIntoBounds)
		{
			const FVector ComponentLocation = GetComponentLocation();
			const FBoxSphereBounds ComponentLocationBounds(ComponentLocation, FVector(1.), 1.);
			if (bCacheLocalSpaceBounds)
			{
				NewBounds = NewBounds.TransformBy(LocalToWorld);
				NewBounds = NewBounds + ComponentLocationBounds;
				NewBounds = NewBounds.TransformBy(LocalToWorld.ToInverseMatrixWithScale());
			}
			else
			{
				NewBounds = NewBounds + ComponentLocationBounds;
			}
		}

		// Add sim mesh bounds
		FBoxSphereBounds SimulationBounds(ForceInit);
		if (IsSimulationEnabled())
		{
			SimulationBounds = ClothSimulationProxy->CalculateBounds_AnyThread();
		}
		else if (bBindToLeaderComponent)
		{
			if (const UChaosClothComponent* const LeaderComponent = LeaderClothComponent.Get())
			{
				if (LeaderComponent->IsSimulationEnabled())
				{
					const UE::Chaos::ClothAsset::FClothSimulationProxy* const LeaderProxy = LeaderComponent->GetClothSimulationProxy();
					check(LeaderProxy); // valid pointer is included in IsSimulationEnabled
					SimulationBounds = LeaderProxy->CalculateBounds_AnyThread();
				}
			}
		}
		if (SimulationBounds.SphereRadius > UE_SMALL_NUMBER && FMath::IsFinite(SimulationBounds.Origin.X) && FMath::IsFinite(SimulationBounds.Origin.Y) && FMath::IsFinite(SimulationBounds.Origin.Z) &&
			FMath::IsFinite(SimulationBounds.BoxExtent.X) && FMath::IsFinite(SimulationBounds.BoxExtent.Y) && FMath::IsFinite(SimulationBounds.BoxExtent.Z) &&
			FMath::IsFinite(SimulationBounds.SphereRadius))  // Don't add the simulation bounds if there are empty, otherwise it could unwillingly add the component's location
		{
			NewBounds = NewBounds + SimulationBounds.TransformBy(CachedBoundsTransform);
		}

		CachedWorldOrLocalSpaceBounds = NewBounds;
		bCachedLocalBoundsUpToDate = bCacheLocalSpaceBounds;
		bCachedWorldSpaceBoundsUpToDate = !bCacheLocalSpaceBounds;

		if (bCacheLocalSpaceBounds)
		{
			CachedWorldToLocalTransform.SetIdentity();
			NewBounds = NewBounds.TransformBy(LocalToWorld);
		}
		else
		{
			CachedWorldToLocalTransform = LocalToWorld.ToInverseMatrixWithScale();
		}
	}
	return NewBounds;
}

void UChaosClothComponent::OnAttachmentChanged()
{
	if (bUseAttachedParentAsPoseComponent)
	{
		USkinnedMeshComponent* const AttachParentComponent = Cast<USkinnedMeshComponent>(GetAttachParent());
		if (UChaosClothComponent* const AttachClothComponent = Cast<UChaosClothComponent>(AttachParentComponent))
		{
			LeaderClothComponent = AttachClothComponent;
		}

		SetLeaderPoseComponent(AttachParentComponent);  // If the cast fail, remove the current leader

		// When parented to a skeletal mesh, the anim setup needs re-initializing in order to use the follower's bones requirement
		if (USkeletalMeshComponent* const SkeletalMeshComponent = Cast<USkeletalMeshComponent>(AttachParentComponent))
		{
			SkeletalMeshComponent->RecalcRequiredBones(SkeletalMeshComponent->GetPredictedLODLevel());
		}
	}

	Super::OnAttachmentChanged();
}

bool UChaosClothComponent::IsVisible() const
{
	return bHasValidRenderDataForVisibility && Super::IsVisible();
}

void UChaosClothComponent::RefreshBoneTransforms(FActorComponentTickFunction* /*TickFunction*/)
{
	MarkRenderDynamicDataDirty();

	bNeedToFlipSpaceBaseBuffers = true;
	bHasValidBoneTransform = false;
	FlipEditableSpaceBases();
	bHasValidBoneTransform = true;
}

void UChaosClothComponent::GetUpdateClothSimulationData_AnyThread(TMap<int32, FClothSimulData>& OutClothSimulData, FMatrix& OutLocalToWorld, float& OutBlendWeight) const
{
	OutLocalToWorld = GetComponentToWorld().ToMatrixWithScale();

	const UChaosClothComponent* const LeaderPoseClothComponent = LeaderClothComponent.Get();
	if (LeaderPoseClothComponent && LeaderPoseClothComponent->ClothSimulationProxy && bBindToLeaderComponent)
	{
		OutBlendWeight = BlendWeight;
		OutClothSimulData = LeaderPoseClothComponent->ClothSimulationProxy->GetCurrentSimulationData_AnyThread();
	}
	else if (IsSimulationEnabled() && !bBindToLeaderComponent && ClothSimulationProxy)
	{
		OutBlendWeight = BlendWeight;
		OutClothSimulData = ClothSimulationProxy->GetCurrentSimulationData_AnyThread();
	}
	else
	{
		OutClothSimulData.Reset();
	}

	// Blend cloth out whenever the simulation data is invalid
	if (!OutClothSimulData.Num())
	{
		OutBlendWeight = 0.0f;
	}
}

void UChaosClothComponent::SetSkinnedAssetAndUpdate(USkinnedAsset* InSkinnedAsset, bool bReinitPose)
{
	if (InSkinnedAsset != GetSkinnedAsset())
	{
		// Note: It is not necessary to stop the current simulation here, since it will die off once the proxy is recreated

		// Change the skinned asset, dirty render states, ...etc.
		Super::SetSkinnedAssetAndUpdate(InSkinnedAsset, bReinitPose);

		if (IsRegistered())
		{
			// Update the component bone transforms (for colliders) from the new cloth asset
			UpdateComponentSpaceTransforms();

			// Fill up the property collection with the new cloth asset properties
			ResetConfigProperties();

			// Hard reset the simulation
			RecreateClothSimulationProxy();
		}

		// Update the component visibility in case the new render mesh has no valid LOD
		UpdateVisibility();
	}
}

void UChaosClothComponent::GetAdditionalRequiredBonesForLeader(int32 LeaderLODIndex, TArray<FBoneIndexType>& InOutRequiredBones) const
{
	TArray<FBoneIndexType> RequiredBones;

	// Add the follower's bones (including sim and render mesh bones, both stored in the LODRenderData RequiredBones array)
	if (const FSkeletalMeshRenderData* const SkeletalMeshRenderData = GetSkeletalMeshRenderData())
	{
		const int32 MinLODIndex = ComputeMinLOD();
		const int32 MaxLODIndex = FMath::Max(GetNumLODs() - 1, MinLODIndex);

		const int32 LODIndex = FMath::Clamp(LeaderLODIndex, MinLODIndex, MaxLODIndex);

		if (SkeletalMeshRenderData->LODRenderData.IsValidIndex(LODIndex))
		{
			RequiredBones.Reserve(SkeletalMeshRenderData->LODRenderData[LODIndex].RequiredBones.Num());

			for (const FBoneIndexType RequiredBone : SkeletalMeshRenderData->LODRenderData[LODIndex].RequiredBones)
			{
				if (GetLeaderBoneMap().IsValidIndex(RequiredBone))
				{
					const int32 FollowerRequiredLeaderBone = GetLeaderBoneMap()[RequiredBone];
					if (FollowerRequiredLeaderBone != INDEX_NONE)
					{
						RequiredBones.Add((FBoneIndexType)FollowerRequiredLeaderBone);
					}
				}
			}

			// Then sort array of required bones in hierarchy order
			RequiredBones.Sort();
		}
	}

	// Merge the physics asset bones (the leader's physics asset can be different to this component's cloth asset)
	if (GetAsset())
	{
		for (int32 ModelIndex = 0; ModelIndex < GetAsset()->GetNumClothSimulationModels(); ++ModelIndex)
		{
			if (const UPhysicsAsset* const PhysicsAsset = GetAsset()->GetPhysicsAssetForModel(ModelIndex))
			{
				if (const USkinnedAsset* const LeaderSkinnedAsset = ensure(LeaderPoseComponent.IsValid()) ? LeaderPoseComponent->GetSkinnedAsset() : nullptr)  // Needs the leader SkinnedAsset for the correct RefSkeleton
				{
					USkinnedMeshComponent::GetPhysicsRequiredBones(LeaderSkinnedAsset, PhysicsAsset, RequiredBones);
				}
			}
		}
	}

	if (RequiredBones.Num())
	{
		// Make sure all of these are in RequiredBones, note MergeInBoneIndexArrays requires the arrays to be sorted and bone must be unique
		MergeInBoneIndexArrays(InOutRequiredBones, RequiredBones);
	}
}

void UChaosClothComponent::FinalizeBoneTransform()
{
	Super::FinalizeBoneTransform();

	OnBoneTransformsFinalizedMC.Broadcast();
}

FDelegateHandle UChaosClothComponent::RegisterOnBoneTransformsFinalizedDelegate(const FOnBoneTransformsFinalizedMultiCast::FDelegate& Delegate)
{
	return OnBoneTransformsFinalizedMC.Add(Delegate);
}

void UChaosClothComponent::UnregisterOnBoneTransformsFinalizedDelegate(const FDelegateHandle& DelegateHandle)
{
	OnBoneTransformsFinalizedMC.Remove(DelegateHandle);
}

bool UChaosClothComponent::HasAnySimulationMeshData(int32 LODIndex) const
{
	if (const UChaosClothAssetBase* const AssetToSimulate = GetAsset())
	{
		const int32 NumSimulationModels = AssetToSimulate->GetNumClothSimulationModels();
		for (int32 ModelIndex = 0; ModelIndex < NumSimulationModels; ++ModelIndex)
		{
			if (const TSharedPtr<const FChaosClothSimulationModel> ClothSimulationModel = AssetToSimulate->GetClothSimulationModel(ModelIndex))
			{
				if (ClothSimulationModel->IsValidLodIndex(LODIndex)
					&& ClothSimulationModel->GetNumVertices(LODIndex) > 0)
				{
					return true;
				}
			}
		}
	}
	return false;
}

TSharedPtr<UE::Chaos::ClothAsset::FClothSimulationProxy> UChaosClothComponent::CreateClothSimulationProxy()
{
	using namespace UE::Chaos::ClothAsset;
	return MakeShared<FClothSimulationProxy>(*this);
}

void UChaosClothComponent::CreateClothSimulationProxyImpl()
{
	static IConsoleVariable* const CVarClothPhysics = IConsoleManager::Get().FindConsoleVariable(TEXT("p.ClothPhysics"));
	if (!CVarClothPhysics || CVarClothPhysics->GetBool())
	{
		// Note: CreateClothSimulationProxy and ClothSimulationProxy::PostConstructor are both virtual.
		ClothSimulationProxy = CreateClothSimulationProxy();
		if (ClothSimulationProxy)
		{
			ClothSimulationProxy->PostConstructor();
		}
	}
}

void UChaosClothComponent::AddCollisionSource(USkinnedMeshComponent* SourceComponent, const UPhysicsAsset* SourcePhysicsAsset, bool bUseSphylsOnly)
{
	CollisionSources->Add(SourceComponent, SourcePhysicsAsset, bUseSphylsOnly);
}

void UChaosClothComponent::RemoveCollisionSources(const USkinnedMeshComponent* SourceComponent)
{
	CollisionSources->Remove(SourceComponent);
}

void UChaosClothComponent::RemoveCollisionSource(const USkinnedMeshComponent* SourceComponent, const UPhysicsAsset* SourcePhysicsAsset)
{
	CollisionSources->Remove(SourceComponent, SourcePhysicsAsset);
}

void UChaosClothComponent::ResetCollisionSources()
{
	CollisionSources->Reset();
}

void UChaosClothComponent::SetCollideWithEnvironment(bool bCollide)
{
	bCollideWithEnvironment = bCollide;
	CollisionSources->SetCollideWithEnvironment(bCollideWithEnvironment);
}

void UChaosClothComponent::SetSimulateInEditor(const bool bInSimulateInEditor)
{
#if WITH_EDITOR
	bSimulateInEditor = bInSimulateInEditor;
	// Editor-only tick gate must follow bSimulateInEditor so the simulation actually runs when toggled on.
	bTickInEditor = bInSimulateInEditor;
#endif
}

void UChaosClothComponent::StartNewParallelSimulation(float DeltaTime)
{
	if (ClothSimulationProxy.IsValid())
	{
		CSV_SCOPED_TIMING_STAT(Animation, Cloth);
		ClothSimulationProxy->Tick_GameThread(DeltaTime);
	}
}

void UChaosClothComponent::ClearPropertyCollectionsDirtyFlags(int32 LODIndex) const
{
	using namespace ::Chaos::Softs;
	for (int32 ModelIndex = 0; ModelIndex < ClothSimulationProperties.Num(); ++ModelIndex)
	{
		const TArray<TSharedPtr<FCollectionPropertyFacade>>& CollectionPropertyFacades = ClothSimulationProperties[ModelIndex].CollectionPropertyFacades;
		if (CollectionPropertyFacades.IsValidIndex(LODIndex) && CollectionPropertyFacades[LODIndex].IsValid())
		{
			CollectionPropertyFacades[LODIndex]->ClearDirtyFlags();
		}
	}
}

void UChaosClothComponent::UpdateClothTeleport()
{
	const FMatrix CurRootBoneMat = GetBoneMatrix(0);
	ClothTeleportMode = (bTeleport || bTeleportOnce) ? ((bReset || bResetOnce) ? EClothingTeleportMode::TeleportAndReset : EClothingTeleportMode::Teleport) : EClothingTeleportMode::None;
	ClothTeleportMode = UE::ClothingSimulation::TeleportHelpers::CalculateClothingTeleport(ClothTeleportMode, CurRootBoneMat, PrevRootBoneMatrix, (bReset||bResetOnce), ClothTeleportDistThresholdSquared, ClothTeleportCosineThresholdInRad);
	PrevRootBoneMatrix = CurRootBoneMat;
	bTeleportOnce = false;
	bResetOnce = false;
}

void UChaosClothComponent::HandleExistingParallelSimulation()
{
	if (bBindToLeaderComponent)
	{
		if (UChaosClothComponent* const LeaderComponent = LeaderClothComponent.Get())
		{
			LeaderComponent->HandleExistingParallelSimulation();
		}
	}

	if (ClothSimulationProxy.IsValid() && ClothSimulationProxy->IsParallelSimulationTaskValid())
	{
		ClothSimulationProxy->CompleteParallelSimulation_GameThread();
		InvalidateCachedBounds();
	}
}

bool UChaosClothComponent::ShouldWaitForParallelSimulationInTickComponent() const
{
	static IConsoleVariable* const CVarClothPhysicsWaitForParallelClothTask = IConsoleManager::Get().FindConsoleVariable(TEXT("p.ClothPhysics.WaitForParallelClothTask"));

	return bWaitForParallelTask || (CVarClothPhysicsWaitForParallelClothTask && CVarClothPhysicsWaitForParallelClothTask->GetBool());
}

void UChaosClothComponent::UpdateComponentSpaceTransforms()
{
	check(IsRegistered());

	if (!LeaderPoseComponent.IsValid() && GetAsset() && GetAsset()->GetResourceForRendering())
	{
		FSkeletalMeshLODRenderData& LODData = GetAsset()->GetResourceForRendering()->LODRenderData[GetPredictedLODLevel()];
		GetAsset()->FillComponentSpaceTransforms(GetAsset()->GetRefSkeleton().GetRefBonePose(), LODData.RequiredBones, GetEditableComponentSpaceTransforms());

		bNeedToFlipSpaceBaseBuffers = true; // Have updated space bases so need to flip
		FlipEditableSpaceBases();
		bHasValidBoneTransform = true;
	}
}

void UChaosClothComponent::UpdateVisibility()
{
	if (GetAsset() && GetAsset()->GetResourceForRendering())
	{
		const FSkeletalMeshRenderData* const SkeletalMeshRenderData = GetAsset()->GetResourceForRendering();
		const int32 FirstValidLODIdx = SkeletalMeshRenderData ? SkeletalMeshRenderData->GetFirstValidLODIdx(0) : INDEX_NONE;
		bHasValidRenderDataForVisibility = FirstValidLODIdx != INDEX_NONE;
	}
	else
	{
		bHasValidRenderDataForVisibility = false;
	}
	SetVisibility(bHasValidRenderDataForVisibility);
}

UChaosClothAssetInteractor* UChaosClothComponent::GetClothOutfitInteractor(int32 ModelIndex, const FName ClothSimulationModelName)
{
	check(IsInGameThread());
	if (ClothSimulationModelName != NAME_None && GetAsset() && GetAsset()->GetNumClothSimulationModels() == ClothSimulationProperties.Num())
	{
		for (int32 Index = 0; Index < ClothSimulationProperties.Num(); ++Index)
		{
			if (GetAsset()->GetClothSimulationModelName(Index) == ClothSimulationModelName)
			{
				return ClothSimulationProperties[Index].ClothOutfitInteractor;
			}
		}
	}
	return ClothSimulationProperties.IsValidIndex(ModelIndex) ? ClothSimulationProperties[ModelIndex].ClothOutfitInteractor : nullptr;
}

void UChaosClothComponent::BuildSimulationProxy()
{
	RecreateClothSimulationProxy();
}

void UChaosClothComponent::ResetSimulationProxy()
{
	ClothSimulationProxy.Reset();
}

void UChaosClothComponent::PreProcessSimulation(const float DeltaTime)
{
	if (ClothSimulationProxy.IsValid())
	{
		constexpr bool bForceWaitForInitialization = true;
		ClothSimulationProxy->PreProcess_GameThread(DeltaTime, bForceWaitForInitialization);
	}
}

void UChaosClothComponent::WriteToSimulation(const float DeltaTime, const bool bAsyncTask)
{
	using namespace ::Chaos::Softs;
	if (ClothSimulationProxy.IsValid())
	{
		ClothSimulationProxy->PreSimulate_GameThread(DeltaTime);
	}
}

void UChaosClothComponent::ReadFromSimulation(const float /*DeltaTime*/, const bool /*bAsyncTask*/)
{
	if (ClothSimulationProxy.IsValid())
	{
		ClothSimulationProxy->PostSimulate_GameThread();
		InvalidateCachedBounds();
	}
}

void UChaosClothComponent::PostProcessSimulation(const float /*DeltaTime*/)
{
	if (ClothSimulationProxy.IsValid())
	{
		ClothSimulationProxy->PostProcess_GameThread();
	}
}

const FReferenceSkeleton* UChaosClothComponent::GetReferenceSkeleton() const
{
	return GetAsset() ? &GetAsset()->GetRefSkeleton() : nullptr;
}

int32 UChaosClothComponent::GetSimulationGroupId(const UChaosClothAssetBase* InAsset, int32 ModelIndex) const
{
	return
		InAsset == GetAsset() &&
		ModelIndex >= 0 &&
		ModelIndex < InAsset->GetNumClothSimulationModels() &&
		InAsset->GetClothSimulationModel(ModelIndex).IsValid() ? ModelIndex : INDEX_NONE;
}

FDataflowSimulationProxy* UChaosClothComponent::GetSimulationProxy()
{
	return ClothSimulationProxy.Get();
}

const FDataflowSimulationProxy* UChaosClothComponent::GetSimulationProxy() const
{
	return ClothSimulationProxy.Get();
}

float UChaosClothComponent::GetTeleportRotationThreshold() const
{
	return TeleportRotationThreshold;
}

void UChaosClothComponent::SetTeleportRotationThreshold(float Threshold)
{
	TeleportRotationThreshold = Threshold;
	ClothTeleportCosineThresholdInRad = UE::ClothingSimulation::TeleportHelpers::ComputeTeleportCosineRotationThreshold(TeleportRotationThreshold);
}

float UChaosClothComponent::GetTeleportDistanceThreshold() const
{
	return TeleportDistanceThreshold;
}

void UChaosClothComponent::SetTeleportDistanceThreshold(float Threshold)
{
	TeleportDistanceThreshold = Threshold;
	ClothTeleportDistThresholdSquared = UE::ClothingSimulation::TeleportHelpers::ComputeTeleportDistanceThresholdSquared(TeleportDistanceThreshold);
}
