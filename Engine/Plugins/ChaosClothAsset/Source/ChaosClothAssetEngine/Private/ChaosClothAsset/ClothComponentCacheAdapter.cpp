// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ClothComponentCacheAdapter.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/ChaosCache.h"
#include "Chaos/CacheManagerActor.h"
#include "Chaos/PBDEvolution.h"
#include "ChaosCloth/ChaosClothingCacheSchema.h"
#include "ChaosCloth/ChaosClothingSimulation.h"
#include "ChaosCloth/ChaosClothingSimulationSolver.h"
#include "ChaosClothAsset/ClothComponent.h"
#include "ChaosClothAsset/ClothSimulationProxy.h"
#include "Components/SkeletalMeshComponent.h"

namespace UE::Chaos::ClothAsset
{
	::Chaos::FComponentCacheAdapter::SupportType FClothComponentCacheAdapter::SupportsComponentClass(UClass* InComponentClass) const
	{
		const UClass* Desired = GetDesiredClass();
		if(InComponentClass == Desired)
		{
			return ::Chaos::FComponentCacheAdapter::SupportType::Direct;
		}
		else if(InComponentClass->IsChildOf(Desired))
		{
			return ::Chaos::FComponentCacheAdapter::SupportType::Derived;
		}

		return ::Chaos::FComponentCacheAdapter::SupportType::None;
	}

	UClass* FClothComponentCacheAdapter::GetDesiredClass() const
	{
		return UChaosClothComponent::StaticClass();
	}

	uint8 FClothComponentCacheAdapter::GetPriority() const
	{
		return EngineAdapterPriorityBegin;
	}

	void FClothComponentCacheAdapter::Record_PostSolve(UPrimitiveComponent* InComponent, const FTransform& InRootTransform, FPendingFrameWrite& OutFrame, ::Chaos::FReal InTime) const
	{
		if (const ::Chaos::FClothingSimulationSolver* ClothSolver = GetClothSolver(InComponent))
		{
			::Chaos::FClothingCacheSchema::RecordPostSolve(*ClothSolver, OutFrame, InTime);
		}
	}

	void FClothComponentCacheAdapter::Playback_PreSolve(UPrimitiveComponent* InComponent, UChaosCache* InCache, ::Chaos::FReal InTime, FPlaybackTickRecord& TickRecord, TArray<::Chaos::TPBDRigidParticleHandle<::Chaos::FReal, 3>*>& OutUpdatedRigids) const
	{
		check(InCache);
		if (::Chaos::FClothingSimulationSolver* ClothSolver = GetClothSolver(InComponent))
		{
			::Chaos::FClothingCacheSchema::PlaybackPreSolve(*InCache, InTime, TickRecord, *ClothSolver);
		}
	}

	FGuid FClothComponentCacheAdapter::GetGuid() const
	{
		FGuid NewGuid;
		checkSlow(FGuid::Parse(TEXT("C704F4F536A34CD4973ABBB7BFEEE432"), NewGuid));
		return NewGuid;
	}

	bool FClothComponentCacheAdapter::ValidForPlayback(UPrimitiveComponent* InComponent, UChaosCache* InCache) const
	{
		const UChaosClothComponent* ClothComp = GetClothComponent(InComponent);
		return ClothComp && ::Chaos::FClothingCacheSchema::CacheIsValidForPlayback(InCache);
	}

	::Chaos::FPhysicsSolverEvents* FClothComponentCacheAdapter::BuildEventsSolver(UPrimitiveComponent* InComponent) const
	{
		if (UChaosClothComponent* ClothComp = GetClothComponent(InComponent))
		{
			ClothComp->RecreateClothSimulationProxy();
			return ClothComp->ClothSimulationProxy->Solver.Get();
		}
		return nullptr;
	}
	
	::Chaos::FPhysicsSolver* FClothComponentCacheAdapter::GetComponentSolver(UPrimitiveComponent* InComponent) const
	{
		return nullptr;
	}
	
	void FClothComponentCacheAdapter::SetRestState(UPrimitiveComponent* InComponent, UChaosCache* InCache, const FTransform& InRootTransform, ::Chaos::FReal InTime) const
	{
		if (!InCache || InCache->GetDuration() == 0.0f)
		{
			return;
		}

		if (UChaosClothComponent* const ClothComponent = GetClothComponent(InComponent))
		{
			if (FClothSimulationProxy* const Proxy = ClothComponent->ClothSimulationProxy.Get())
			{
				FClothingSimulationCacheData CacheData;
				::Chaos::FClothingCacheSchema::LoadCacheData(InCache, InTime, CacheData);

				Proxy->CacheData = MakeUnique<FClothingSimulationCacheData>(MoveTemp(CacheData));
				Proxy->SolverMode = FClothSimulationProxy::ESolverMode::Default;
#if WITH_EDITOR
				ClothComponent->SetTickOnceInEditor();
#endif
			}
		}
	}

	bool FClothComponentCacheAdapter::InitializeForRecord(UPrimitiveComponent* InComponent, UChaosCache* InCache)
	{
		if (FClothSimulationProxy* const Proxy = GetProxy(InComponent))
		{
			Proxy->SolverMode = FClothSimulationProxy::ESolverMode::EnableSolverForSimulateRecord;
		}
		return true;
	}

	bool FClothComponentCacheAdapter::InitializeForPlayback(UPrimitiveComponent* InComponent, UChaosCache* InCache, float InTime)
	{
		::Chaos::EnsureIsInGameThreadContext();

		if (FClothSimulationProxy* const Proxy = GetProxy(InComponent))
		{
			Proxy->SolverMode = FClothSimulationProxy::ESolverMode::DisableSolverForPlayback;
		}
		return true;
	}

	void FClothComponentCacheAdapter::WaitForSolverTasks(UPrimitiveComponent* InComponent) const
	{
		::Chaos::EnsureIsInGameThreadContext();

		if (UChaosClothComponent* const ClothComponent = GetClothComponent(InComponent))
		{
			ClothComponent->WaitForExistingParallelClothSimulation_GameThread();
		}
	}

	UChaosClothComponent* FClothComponentCacheAdapter::GetClothComponent(UPrimitiveComponent* InComponent) const
	{
		if (InComponent)
		{
			if (UChaosClothComponent* ClothComp = Cast<UChaosClothComponent>(InComponent))
			{
				return ClothComp;
			}
			else if (USkeletalMeshComponent* SkelComp = Cast<USkeletalMeshComponent>(InComponent))
			{
				TArray<USceneComponent*> Children;
				SkelComp->GetChildrenComponents(true, Children);
				TArray<USceneComponent*> ClothComps = Children.FilterByPredicate([](USceneComponent* Child) { return Child->IsA<UChaosClothComponent>(); });
				if (ClothComps.Num() > 1)
				{
					ensureMsgf(false, TEXT("Found more than one cloth component attached to a skeletal mesh component. This is not yet supported."));
				}
				else if (ClothComps.Num() == 1)
				{
					return Cast<UChaosClothComponent>(ClothComps[0]);
				}
			}
		}
		return nullptr;
	}

	FClothSimulationProxy* FClothComponentCacheAdapter::GetProxy(UPrimitiveComponent* InComponent) const
	{
		if (UChaosClothComponent* const ClothComponent = GetClothComponent(InComponent))
		{
			return ClothComponent->ClothSimulationProxy.Get();
		}
		return nullptr;
	}

	::Chaos::FClothingSimulationSolver* FClothComponentCacheAdapter::GetClothSolver(UPrimitiveComponent* InComponent) const
	{
		if (FClothSimulationProxy* const Proxy = GetProxy(InComponent))
		{
			return Proxy->Solver.Get();
		}
		return nullptr;
	}

}    // namespace Chaos
