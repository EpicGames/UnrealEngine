// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ClothComponentInstanceData.h"

#include "ChaosClothAsset/ClothComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClothComponentInstanceData)

FChaosClothComponentInstanceData::FChaosClothComponentInstanceData(const UChaosClothComponent* SourceComponent)
	: FSceneComponentInstanceData(SourceComponent)
	, Asset(nullptr)
	, bSimulateInEditor(0)
	, bHasOverrides(0)
{
#if WITH_EDITOR
	// Cache all transient values that get reset by the construction script when component is Blueprint-created
	const bool bIsBlueprintCreatedComponent =
		SourceComponent->CreationMethod == EComponentCreationMethod::SimpleConstructionScript ||
		SourceComponent->CreationMethod == EComponentCreationMethod::UserConstructionScript;
	if (bIsBlueprintCreatedComponent)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		Asset = SourceComponent->Asset;  // Read the Asset field directly rather than relying on the getter that is order dependant on PostEditChangeProperty
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		bSimulateInEditor = SourceComponent->GetSimulateInEditor();
		bHasOverrides = 1;
	}
#endif
}

bool FChaosClothComponentInstanceData::ContainsData() const
{
	return Super::ContainsData() || bHasOverrides;
}

void FChaosClothComponentInstanceData::ApplyToComponent(UActorComponent* Component, const ECacheApplyPhase CacheApplyPhase)
{
	Super::ApplyToComponent(Component, CacheApplyPhase);

#if WITH_EDITOR
	if (bHasOverrides)
	{
		if (UChaosClothComponent* const ClothComponent = Cast<UChaosClothComponent>(Component))
		{
			ClothComponent->SetAsset(Asset);
			ClothComponent->SetSimulateInEditor((bool)bSimulateInEditor);
		}
	}
#endif
}
