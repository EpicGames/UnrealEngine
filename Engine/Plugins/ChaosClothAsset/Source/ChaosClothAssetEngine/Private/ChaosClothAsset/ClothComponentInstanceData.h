// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Components/SceneComponent.h"

#include "ClothComponentInstanceData.generated.h"

class UChaosClothAssetBase;
class UChaosClothComponent;

/**
 * Saves internal cloth component (transient) state that would otherwise be lost.
 * Useful for when Blueprint construction scripts rerun for Blueprint-created components.
 */
USTRUCT()
struct FChaosClothComponentInstanceData : public FSceneComponentInstanceData
{
	GENERATED_BODY()

	FChaosClothComponentInstanceData() = default;
	explicit FChaosClothComponentInstanceData(const UChaosClothComponent* SourceComponent);

	virtual bool ContainsData() const override;
	virtual void ApplyToComponent(UActorComponent* Component, const ECacheApplyPhase CacheApplyPhase) override;

private:
	UPROPERTY()
	TObjectPtr<UChaosClothAssetBase> Asset;
	uint8 bSimulateInEditor : 1;
	uint8 bHasOverrides : 1;
};
