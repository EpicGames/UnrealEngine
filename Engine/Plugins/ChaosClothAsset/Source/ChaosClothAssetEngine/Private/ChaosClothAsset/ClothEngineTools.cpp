// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ClothEngineTools.h"
#include "Chaos/CollectionEmbeddedSpringConstraintFacade.h"
#include "Chaos/CollectionPropertyFacade.h"
#include "ChaosClothAsset/ClothAssetBase.h"
#include "ChaosClothAsset/ClothAssetPrivate.h"
#include "ChaosClothAsset/ClothCollectionGroup.h"
#include "ChaosClothAsset/ClothGeometryTools.h"
#include "ChaosClothAsset/ClothSimulationModel.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "ChaosClothAsset/CollectionClothSelectionFacade.h"
#include "Engine/SkeletalMesh.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Materials/MaterialInterface.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "UObject/SoftObjectPath.h"
#include "ClothTetherData.h"
#include "ClothVertBoneData.h"
#include "ReferenceSkeleton.h"

#define LOCTEXT_NAMESPACE "ChaosClothAssetEngineTools"

namespace UE::Chaos::ClothAsset
{

namespace Private
{
#if WITH_EDITORONLY_DATA
static bool bClothCollectionOnlyCookRequiredFacades = true;

static FAutoConsoleVariableRef CVarClothCollectionOnlyCookRequiredFacades(
	TEXT("p.ClothCollectionOnlyCookRequiredFacades"),
	bClothCollectionOnlyCookRequiredFacades,
	TEXT("Default setting for culling managed arrays on the cloth collection during the cook. Default[true]"));
#endif

static void AppendTetherData(FCollectionClothFacade& ClothFacade, const FClothTetherData& TetherData)
{
	// Append new tethers
	TArrayView<TArray<int32>> TetherKinematicIndex = ClothFacade.GetTetherKinematicIndex();
	TArrayView<TArray<float>> TetherReferenceLength = ClothFacade.GetTetherReferenceLength();
	for (const TArray<TTuple<int32, int32, float>>& TetherBatch : TetherData.Tethers)
	{
		for (const TTuple<int32, int32, float>& Tether : TetherBatch)
		{
			// Tuple is Kinematic, Dynamic, RefLength
			const int32 DynamicIndex = Tether.Get<1>();
			TArray<int32>& KinematicIndex = TetherKinematicIndex[DynamicIndex];
			TArray<float>& ReferenceLength = TetherReferenceLength[DynamicIndex];
			check(KinematicIndex.Num() == ReferenceLength.Num());
			checkSlow(KinematicIndex.Find(Tether.Get<0>()) == INDEX_NONE);
			KinematicIndex.Add(Tether.Get<0>());
			ReferenceLength.Add(Tether.Get<2>());
		}
	}
}

static void RemapBoneIndices(TArrayView<TArray<int32>> BoneIndices, const TArray<int32>& Remap)
{
	for (TArray<int32>& Array : BoneIndices)
	{
		for (int32& Index : Array)
		{
			if (Index != INDEX_NONE && ensure(Remap.IsValidIndex(Index)))
			{
				Index = Remap[Index];
			}
		}
	}
}
}

void FClothEngineTools::GenerateTethers(const TSharedRef<FManagedArrayCollection>& ClothCollection, const FName& WeightMapName, const bool bGenerateGeodesicTethers, const FVector2f& MaxDistanceValue)
{
	FCollectionClothFacade ClothFacade(ClothCollection);
	FClothGeometryTools::DeleteTethers(ClothCollection);
	if (ClothFacade.HasWeightMap(WeightMapName))
	{
		FClothTetherData TetherData;
		TArray<uint32> SimIndices;
		SimIndices.Reserve(ClothFacade.GetNumSimFaces() * 3);
		for (const FIntVector3& Face : ClothFacade.GetSimIndices3D())
		{
			// Exclude degenerate faces
			if (Face[0] != INDEX_NONE &&
				Face[1] != INDEX_NONE &&
				Face[2] != INDEX_NONE &&

				Face[0] != Face[1] &&
				Face[0] != Face[2] &&
				Face[1] != Face[2])
			{
				SimIndices.Add(Face[0]);
				SimIndices.Add(Face[1]);
				SimIndices.Add(Face[2]);
			}
		}

		if (MaxDistanceValue.Equals(FVector2f(0.f, 1.f)))
		{
			TetherData.GenerateTethers(ClothFacade.GetSimPosition3D(), TConstArrayView<uint32>(SimIndices), ClothFacade.GetWeightMap(WeightMapName), bGenerateGeodesicTethers);
		}
		else
		{
			const TSet<int32> KinematicVertices = FClothGeometryTools::GenerateKinematicVertices3D(ClothCollection, WeightMapName, MaxDistanceValue, NAME_None);
			TetherData.GenerateTethers(ClothFacade.GetSimPosition3D(), TConstArrayView<uint32>(SimIndices), KinematicVertices, bGenerateGeodesicTethers);
		}

		Private::AppendTetherData(ClothFacade, TetherData);
	}
}

void FClothEngineTools::GenerateTethersFromSelectionSet(const TSharedRef<FManagedArrayCollection>& ClothCollection, const FName& FixedEndSet, const bool bGeodesicTethers)
{
	FClothGeometryTools::DeleteTethers(ClothCollection);
	FCollectionClothSelectionConstFacade SelectionFacade(ClothCollection);

	if (SelectionFacade.HasSelection(FixedEndSet) && SelectionFacade.GetSelectionGroup(FixedEndSet) == UE::Chaos::ClothAsset::ClothCollectionGroup::SimVertices3D)
	{
		FCollectionClothFacade ClothFacade(ClothCollection);
		FClothTetherData TetherData;
		TArray<uint32> SimIndices;
		SimIndices.Reserve(ClothFacade.GetNumSimFaces() * 3);
		for (const FIntVector3& Face : ClothFacade.GetSimIndices3D())
		{
			// Exclude degenerate faces
			if (Face[0] != INDEX_NONE &&
				Face[1] != INDEX_NONE &&
				Face[2] != INDEX_NONE &&

				Face[0] != Face[1] &&
				Face[0] != Face[2] &&
				Face[1] != Face[2])
			{
				SimIndices.Add(Face[0]);
				SimIndices.Add(Face[1]);
				SimIndices.Add(Face[2]);
			}
		}

		TetherData.GenerateTethers(ClothFacade.GetSimPosition3D(), TConstArrayView<uint32>(SimIndices), SelectionFacade.GetSelectionSet(FixedEndSet), bGeodesicTethers);
		Private::AppendTetherData(ClothFacade, TetherData);
	}
}

void FClothEngineTools::GenerateTethersFromCustomSelectionSets(const TSharedRef<FManagedArrayCollection>& ClothCollection, const FName& InFixedEndSet, const TArray<TPair<FName, FName>>& CustomTetherEndSets, const bool bGeodesicTethers)
{
	FClothGeometryTools::DeleteTethers(ClothCollection);
	FCollectionClothSelectionConstFacade SelectionFacade(ClothCollection);
	if (SelectionFacade.HasSelection(InFixedEndSet) && SelectionFacade.GetSelectionGroup(InFixedEndSet) == UE::Chaos::ClothAsset::ClothCollectionGroup::SimVertices3D)
	{

		FCollectionClothFacade ClothFacade(ClothCollection);
		TArray<uint32> SimIndices;
		SimIndices.Reserve(ClothFacade.GetNumSimFaces() * 3);
		for (const FIntVector3& Face : ClothFacade.GetSimIndices3D())
		{
			// Exclude degenerate faces
			if (Face[0] != INDEX_NONE &&
				Face[1] != INDEX_NONE &&
				Face[2] != INDEX_NONE &&

				Face[0] != Face[1] &&
				Face[0] != Face[2] &&
				Face[1] != Face[2])
			{
				SimIndices.Add(Face[0]);
				SimIndices.Add(Face[1]);
				SimIndices.Add(Face[2]);
			}
		}

		const TSet<int32>& FixedEndSet = SelectionFacade.GetSelectionSet(InFixedEndSet);

		for (const TPair<FName, FName>& TetherEnds : CustomTetherEndSets)
		{
			const FName& CustomDynamicEndSet = TetherEnds.Get<0>();
			const FName& CustomFixedEndSet = TetherEnds.Get<1>();
			if (SelectionFacade.HasSelection(CustomFixedEndSet) && SelectionFacade.GetSelectionGroup(CustomFixedEndSet) == UE::Chaos::ClothAsset::ClothCollectionGroup::SimVertices3D &&
				SelectionFacade.HasSelection(CustomDynamicEndSet) && SelectionFacade.GetSelectionGroup(CustomDynamicEndSet) == UE::Chaos::ClothAsset::ClothCollectionGroup::SimVertices3D)
			{
				FClothTetherData TetherData;
				TetherData.GenerateTethers(ClothFacade.GetSimPosition3D(), TConstArrayView<uint32>(SimIndices), FixedEndSet, SelectionFacade.GetSelectionSet(CustomFixedEndSet), SelectionFacade.GetSelectionSet(CustomDynamicEndSet), bGeodesicTethers);
				Private::AppendTetherData(ClothFacade, TetherData);
			}
		}
	}
}

FPointWeightMap FClothEngineTools::GetMaxDistanceWeightMap(const TSharedRef<const FManagedArrayCollection>& ClothCollection, const int32 NumLodSimVertices)
{
	FCollectionClothConstFacade ClothFacade(ClothCollection);
	::Chaos::Softs::FCollectionPropertyConstFacade PropertyFacade(ClothCollection);
	return GetMaxDistanceWeightMap(ClothFacade, PropertyFacade, NumLodSimVertices);
}

FPointWeightMap FClothEngineTools::GetMaxDistanceWeightMap(const FCollectionClothConstFacade& ClothFacade, const ::Chaos::Softs::FCollectionPropertyConstFacade& PropertyFacade, const int32 NumLodSimVertices)
{
	int32 MaxDistancePropertyKeyIndex;
	static const FName MaxDistanceName(TEXT("MaxDistance"), EFindName::FNAME_Find);
	check(MaxDistanceName != NAME_None);
	const FString MaxDistanceString = PropertyFacade.GetStringValue(MaxDistanceName, MaxDistanceName.ToString(), &MaxDistancePropertyKeyIndex);
	const bool bHasMaxDistanceProperty = (MaxDistancePropertyKeyIndex != INDEX_NONE);
	const float MaxDistanceOffset = bHasMaxDistanceProperty ? PropertyFacade.GetLowValue<float>(MaxDistancePropertyKeyIndex) : TNumericLimits<float>::Max();  // Uses infinite distance when no MaxDistance properties are set
	const float MaxDistanceScale = bHasMaxDistanceProperty ? PropertyFacade.GetHighValue<float>(MaxDistancePropertyKeyIndex) - MaxDistanceOffset : 0.f;
	const TConstArrayView<float> MaxDistanceWeightMap = ClothFacade.GetWeightMap(FName(MaxDistanceString));

	return (MaxDistanceWeightMap.Num() == NumLodSimVertices) ?
		FPointWeightMap(MaxDistanceWeightMap, MaxDistanceOffset, MaxDistanceScale) :
		FPointWeightMap(NumLodSimVertices, MaxDistanceOffset);
}

int32 FClothEngineTools::CalculateReferenceBoneIndex(const TArray<int32>& UsedBones, const FReferenceSkeleton& ReferenceSkeleton)
{
	// Starts at root
	int32 ReferenceBoneIndex = 0;

	// List of valid paths to the root bone from each weighted bone
	TArray<TArray<int32>> PathsToRoot;

	const int32 NumUsedBones = UsedBones.Num();
	PathsToRoot.Reserve(NumUsedBones);

	// Compute paths to the root bone
	for (int32 UsedBoneIndex = 0; UsedBoneIndex < NumUsedBones; ++UsedBoneIndex)
	{
		PathsToRoot.AddDefaulted();
		TArray<int32>& Path = PathsToRoot.Last();

		int32 CurrentBone = UsedBones[UsedBoneIndex];
		Path.Add(CurrentBone);

		while (CurrentBone != 0 && CurrentBone != INDEX_NONE)
		{
			CurrentBone = ReferenceSkeleton.GetParentIndex(CurrentBone);
			Path.Add(CurrentBone);
		}
	}

	// Paths are from leaf->root, we want the other way
	for (TArray<int32>& Path : PathsToRoot)
	{
		Algo::Reverse(Path);
	}

	// Verify the last common bone in all paths as the root of the sim space
	const int32 NumPaths = PathsToRoot.Num();
	if (NumPaths > 0)
	{
		TArray<int32>& FirstPath = PathsToRoot[0];

		const int32 FirstPathSize = FirstPath.Num();
		for (int32 PathEntryIndex = 0; PathEntryIndex < FirstPathSize; ++PathEntryIndex)
		{
			const int32 CurrentQueryIndex = FirstPath[PathEntryIndex];
			bool bValidRoot = true;

			for (int32 PathIndex = 1; PathIndex < NumPaths; ++PathIndex)
			{
				if (!PathsToRoot[PathIndex].Contains(CurrentQueryIndex))
				{
					bValidRoot = false;
					break;
				}
			}

			if (bValidRoot)
			{
				ReferenceBoneIndex = CurrentQueryIndex;
			}
			else
			{
				// Once we fail to find a valid root we're done.
				break;
			}
		}
	}
	else
	{
		// Just use the root
		ReferenceBoneIndex = 0;
	}
	return ReferenceBoneIndex;
}


int32 FClothEngineTools::CalculateReferenceBoneIndex(const TSharedRef<const FManagedArrayCollection>& ClothCollection, const FReferenceSkeleton& ReferenceSkeleton)
{
	FCollectionClothConstFacade ClothFacade(ClothCollection);
	TConstArrayView<TArray<int32>> SimBoneIndices = ClothFacade.GetSimBoneIndices();
	TConstArrayView<TArray<float>> SimBoneWeights = ClothFacade.GetSimBoneWeights();

	TSet<int32> UsedBones;
	UsedBones.Reserve(ReferenceSkeleton.GetRawBoneNum());
	for (int32 VertexIndex = 0; VertexIndex < SimBoneIndices.Num(); ++VertexIndex)
	{
		check(SimBoneIndices[VertexIndex].Num() == SimBoneWeights[VertexIndex].Num());
		for (int32 BoneIndex = 0; BoneIndex < SimBoneIndices[VertexIndex].Num(); ++BoneIndex)
		{
			if (SimBoneWeights[VertexIndex][BoneIndex] > UE_SMALL_NUMBER)
			{
				UsedBones.Add(SimBoneIndices[VertexIndex][BoneIndex]);
			}
		}
	}

	return CalculateReferenceBoneIndex(UsedBones.Array(), ReferenceSkeleton);
}

bool FClothEngineTools::CalculateRemappedBoneIndicesIfCompatible(
	const FCollectionClothConstFacade& Cloth1,
	const FCollectionClothConstFacade& Cloth2,
	FSoftObjectPath& OutMergedSkeletalMeshPath,
	TArray<int32>& OutBoneIndicesRemapCloth1,
	TArray<int32>& OutBoneIndicesRemapCloth2,
	FText* OutIncompatibleErrorDetails)
{
	const FSoftObjectPath& SkeletalMeshPathName1 = Cloth1.GetSkeletalMeshSoftObjectPathName();
	const FSoftObjectPath& SkeletalMeshPathName2 = Cloth2.GetSkeletalMeshSoftObjectPathName();
	if (SkeletalMeshPathName1.IsNull() || SkeletalMeshPathName2.IsNull() || SkeletalMeshPathName1 == SkeletalMeshPathName2)
	{
		OutMergedSkeletalMeshPath = SkeletalMeshPathName1.IsNull() ? SkeletalMeshPathName2 : SkeletalMeshPathName1;
		OutBoneIndicesRemapCloth1.Reset();
		OutBoneIndicesRemapCloth2.Reset();
		return true;
	}

	const USkeletalMesh* const SkeletalMesh1 = Cast<USkeletalMesh>(SkeletalMeshPathName1.TryLoad());
	const USkeletalMesh* const SkeletalMesh2 = Cast<USkeletalMesh>(SkeletalMeshPathName2.TryLoad());
	if (!SkeletalMesh1 || !SkeletalMesh2)
	{
		if (OutIncompatibleErrorDetails)
		{
			*OutIncompatibleErrorDetails = FText::Format(
				LOCTEXT(
					"IncompatibleSkeletalMeshesLoadFailureDetails",
					"Cloth collections failed to merge due to failing to load SkeletalMesh \"{0}\" to check compatibility."),
				!SkeletalMesh1 ? FText::FromString(SkeletalMeshPathName1.ToString()) : FText::FromString(SkeletalMeshPathName2.ToString()));
		}
		return false;
	}

	const FReferenceSkeleton& RefSkeleton1 = SkeletalMesh1->GetRefSkeleton();
	const FReferenceSkeleton& RefSkeleton2 = SkeletalMesh2->GetRefSkeleton();

	const FReferenceSkeleton& MergedRefSkeleton = RefSkeleton1.GetNum() >= RefSkeleton2.GetNum() ? RefSkeleton1 : RefSkeleton2;
	const FReferenceSkeleton& RemapRefSkeleton = RefSkeleton1.GetNum() >= RefSkeleton2.GetNum() ? RefSkeleton2 : RefSkeleton1;
	const FSoftObjectPath& MergedSkeletalMeshPath = RefSkeleton1.GetNum() >= RefSkeleton2.GetNum() ? SkeletalMeshPathName1 : SkeletalMeshPathName2;

	const TArray<FMeshBoneInfo>& RemapBoneInfo = RemapRefSkeleton.GetRefBoneInfo();
	const TArray<FTransform>& MergedBonePose = MergedRefSkeleton.GetRefBonePose();
	const TArray<FTransform>& RemapBonePose = RemapRefSkeleton.GetRefBonePose();
	TArray<int32> RemapIndices;
	RemapIndices.SetNumUninitialized(RemapRefSkeleton.GetNum());
	bool bAnyRemap = false;
	for (int32 BoneIndex = 0; BoneIndex < RemapRefSkeleton.GetNum(); ++BoneIndex)
	{
		const int32 MergedBoneIndex = MergedRefSkeleton.FindBoneIndex(RemapBoneInfo[BoneIndex].Name);
		if (MergedBoneIndex == INDEX_NONE)
		{
			if (OutIncompatibleErrorDetails)
			{
				*OutIncompatibleErrorDetails = FText::Format(
					LOCTEXT(
						"IncompatibleSkeletalMeshesRefBoneInfoDetails",
						"Cloth collections failed to merge due to incompatible Skeletal Meshes, \"{0}\" and \"{1}\". Could not find bone \"{2}\" in \"{3}\"."),
					FText::FromString(SkeletalMeshPathName1.ToString()),
					FText::FromString(SkeletalMeshPathName2.ToString()),
					FText::FromName(RemapBoneInfo[BoneIndex].Name),
					FText::FromString(MergedSkeletalMeshPath.ToString()));
			}

			return false;
		}
		if (!RemapBonePose[BoneIndex].Equals(MergedBonePose[MergedBoneIndex]))
		{
			if (OutIncompatibleErrorDetails)
			{
				*OutIncompatibleErrorDetails = FText::Format(
					LOCTEXT(
						"IncompatibleSkeletalMeshesRefBonePoseDetails",
						"Cloth collections failed to merge due to incompatible Skeletal Meshes, \"{0}\" and \"{1}\". RefBonePoses are mismatched for bone \"{2}\"."),
					FText::FromString(SkeletalMeshPathName1.ToString()),
					FText::FromString(SkeletalMeshPathName2.ToString()),
					FText::FromName(RemapBoneInfo[BoneIndex].Name));
			}

			return false;
		}
		RemapIndices[BoneIndex] = MergedBoneIndex;
		if (BoneIndex != MergedBoneIndex)
		{
			bAnyRemap = true;
		}
	}

	OutMergedSkeletalMeshPath = MergedSkeletalMeshPath;

	if (bAnyRemap)
	{
		if (&RemapRefSkeleton == &RefSkeleton1)
		{
			OutBoneIndicesRemapCloth1 = MoveTemp(RemapIndices);
			OutBoneIndicesRemapCloth2.Reset();
		}
		else
		{
			check(&RemapRefSkeleton == &RefSkeleton2);
			OutBoneIndicesRemapCloth1.Reset();
			OutBoneIndicesRemapCloth2 = MoveTemp(RemapIndices);
		}
	}
	else
	{
		OutBoneIndicesRemapCloth1.Reset();
		OutBoneIndicesRemapCloth2.Reset();
	}

	if (OutIncompatibleErrorDetails)
	{
		*OutIncompatibleErrorDetails = FText();
	}

	if (UE_LOG_ACTIVE(LogChaosClothAsset, VeryVerbose))
	{
		UE_LOGF(LogChaosClothAsset, VeryVerbose, "--------- Calculate Remapped Bone Indices ---------");
		const TArray<FMeshBoneInfo>& MeshBoneInfos1 = RefSkeleton1.GetRefBoneInfo();
		const FString BoneNames1 = FString::JoinBy(MeshBoneInfos1, TEXT(", "), [](const FMeshBoneInfo& MeshBoneInfo) { return MeshBoneInfo.Name.ToString(); });
		UE_LOGF(LogChaosClothAsset, VeryVerbose, "Skeleton 1: %ls\n  %ls", *SkeletalMeshPathName1.GetAssetPathString(), *BoneNames1);

		const TArray<FMeshBoneInfo>& MeshBoneInfos2 = RefSkeleton2.GetRefBoneInfo();
		const FString BoneNames2 = FString::JoinBy(MeshBoneInfos2, TEXT(", "), [](const FMeshBoneInfo& MeshBoneInfo) { return MeshBoneInfo.Name.ToString(); });
		UE_LOGF(LogChaosClothAsset, VeryVerbose, "Skeleton 2: %ls\n  %ls", *SkeletalMeshPathName2.GetAssetPathString(), *BoneNames2);

		UE_LOGF(LogChaosClothAsset, VeryVerbose, "Out Skeleton: %ls", *OutMergedSkeletalMeshPath.GetAssetPathString());

		const FString RemapIndices1 = FString::JoinBy(OutBoneIndicesRemapCloth1, TEXT(", "), [](const int32 Index) { return FString::FromInt(Index); });
		UE_LOGF(LogChaosClothAsset, VeryVerbose, "Remap Indices 1:\n  %ls", *RemapIndices1);

		const FString RemapIndices2 = FString::JoinBy(OutBoneIndicesRemapCloth2, TEXT(", "), [](const int32 Index) { return FString::FromInt(Index); });
		UE_LOGF(LogChaosClothAsset, VeryVerbose, "Remap Indices 2:\n  %ls", *RemapIndices2);
		UE_LOGF(LogChaosClothAsset, VeryVerbose, "---------------------------------------------------");
	}

	return true;
}

void FClothEngineTools::RemapBoneIndices(FCollectionClothFacade& Cloth, const TArray<int32>& BoneIndicesRemap, const int32 SimVertex3DOffset, const int32 RenderVertexOffset)
{
	Private::RemapBoneIndices(Cloth.GetSimBoneIndices().RightChop(SimVertex3DOffset), BoneIndicesRemap);
	Private::RemapBoneIndices(Cloth.GetRenderBoneIndices().RightChop(RenderVertexOffset), BoneIndicesRemap);
	for (int32 AccessoryMeshIndex = 0; AccessoryMeshIndex < Cloth.GetNumSimAccessoryMeshes(); ++AccessoryMeshIndex)
	{
		FCollectionClothSimAccessoryMeshFacade AccessoryMesh = Cloth.GetSimAccessoryMesh(AccessoryMeshIndex);
		Private::RemapBoneIndices(AccessoryMesh.GetSimAccessoryMeshBoneIndices().RightChop(SimVertex3DOffset), BoneIndicesRemap);
	}
}

int32 FClothEngineTools::CopySimMeshToSimAccessoryMesh(const FName& AccessoryMeshName, FCollectionClothFacade& ToCloth, const FCollectionClothConstFacade& FromCloth, bool bUseSimImportVertexID, FText* OutIncompatibleErrorDetails)
{
	auto GenerateUniqueAccessoryMeshName = [&AccessoryMeshName, &ToCloth]()
		{
			int32 MaxExistingNumber = -1;
			bool bFoundMatchWithIndex = false;
			for (const FName& Name : ToCloth.GetSimAccessoryMeshName())
			{
				if (Name.GetComparisonIndex() == AccessoryMeshName.GetComparisonIndex())
				{
					MaxExistingNumber = FMath::Max(MaxExistingNumber, Name.GetNumber());
				}
				bFoundMatchWithIndex = bFoundMatchWithIndex || Name == AccessoryMeshName;
			}

			if (!bFoundMatchWithIndex)
			{
				return AccessoryMeshName;
			}
			FName UniqueName = AccessoryMeshName;
			UniqueName.SetNumber(MaxExistingNumber + 1);
			return UniqueName;
		};

	FSoftObjectPath RemapSkeletalMeshPath;
	TArray<int32> ToClothBoneRemap, FromClothBoneRemap;
	if (!CalculateRemappedBoneIndicesIfCompatible(ToCloth, FromCloth, RemapSkeletalMeshPath, ToClothBoneRemap, FromClothBoneRemap, OutIncompatibleErrorDetails))
	{
		return INDEX_NONE;
	}

	ToCloth.SetSkeletalMeshSoftObjectPathName(RemapSkeletalMeshPath);  // The path may change even with no remaps (e.g. when the second skeleton is extended from the first one)
	if (!ToClothBoneRemap.IsEmpty())
	{
		RemapBoneIndices(ToCloth, ToClothBoneRemap);
	}

	if (bUseSimImportVertexID)
	{
		if (ToCloth.IsValid(EClothCollectionExtendedSchemas::Import) && FromCloth.IsValid(EClothCollectionExtendedSchemas::Import))
		{
			// Start with existing sim data
			TArray<FVector3f> Positions(ToCloth.GetSimPosition3D());
			TArray<FVector3f> Normals(ToCloth.GetSimNormal());
			TArray<TArray<int32>> BoneIndices(ToCloth.GetSimBoneIndices());
			TArray<TArray<float>> BoneWeights(ToCloth.GetSimBoneWeights());

			TConstArrayView<int32> ToImportVertex = ToCloth.GetSimImportVertexID();
			TConstArrayView<int32> FromImportVertex = FromCloth.GetSimImportVertexID();

			TMap<int32, TArray<int32>> ToImportToSim2DLookup;
			ToImportToSim2DLookup.Reserve(ToImportVertex.Num());
			for (int32 ToSim2DIndex = 0; ToSim2DIndex < ToImportVertex.Num(); ++ToSim2DIndex)
			{
				ToImportToSim2DLookup.FindOrAdd(ToImportVertex[ToSim2DIndex]).Add(ToSim2DIndex);
			}

			TConstArrayView<int32> ToSim3DLookup = ToCloth.GetSimVertex3DLookup();
			TConstArrayView<int32> FromSim3DLookup = FromCloth.GetSimVertex3DLookup();
			TConstArrayView<FVector3f> FromPositions = FromCloth.GetSimPosition3D();
			TConstArrayView<FVector3f> FromNormals = FromCloth.GetSimNormal();
			TConstArrayView<TArray<int32>> FromBoneIndices = FromCloth.GetSimBoneIndices();
			TConstArrayView<TArray<float>> FromBoneWeights = FromCloth.GetSimBoneWeights();
			for (int32 FromSim2DIndex = 0; FromSim2DIndex < FromImportVertex.Num(); ++FromSim2DIndex)
			{
				const int32 FromSim3DIndex = FromSim3DLookup[FromSim2DIndex];
				if (FromPositions.IsValidIndex(FromSim3DIndex))
				{
					if (const TArray<int32>* const ToSim2DIndices = ToImportToSim2DLookup.Find(FromImportVertex[FromSim2DIndex]))
					{
						// Found matching index on ToCloth.
						// Copy data between associated sim3d vertices.
						// NOTE: this will just last one wins for vertices that did not seam the same way between the two meshes. Hopefully that's good enough.
						for (const int32 ToSim2DIndex : *ToSim2DIndices)
						{
							const int32 ToSim3DIndex = ToSim3DLookup[ToSim2DIndex];
							if (Positions.IsValidIndex(ToSim3DIndex))
							{
								Positions[ToSim3DIndex] = FromPositions[FromSim3DIndex];
								Normals[ToSim3DIndex] = FromNormals[FromSim3DIndex];
								BoneIndices[ToSim3DIndex] = FromBoneIndices[FromSim3DIndex];
								if (!FromClothBoneRemap.IsEmpty())
								{
									Private::RemapBoneIndices(TArrayView<TArray<int32>>(&BoneIndices[ToSim3DIndex], 1), FromClothBoneRemap);
								}
								BoneWeights[ToSim3DIndex] = FromBoneWeights[FromSim3DIndex];
							}
						}
					}
				}
			}

			FCollectionClothSimAccessoryMeshFacade AccessoryFacade = ToCloth.AddGetSimAccessoryMesh();
			AccessoryFacade.Initialize(GenerateUniqueAccessoryMeshName(), TConstArrayView<FVector3f>(Positions), TConstArrayView<FVector3f>(Normals), TConstArrayView<TArray<int32>>(BoneIndices), TConstArrayView<TArray<float>>(BoneWeights));
			return AccessoryFacade.GetSimAccessoryMeshIndex();
		}

		// no valid import data
		return INDEX_NONE;
	}

	if (FromCloth.GetNumSimVertices3D() >= ToCloth.GetNumSimVertices3D())
	{
		// Can just initialize directly 
		const int32 NumToSimVertices3D = ToCloth.GetNumSimVertices3D();
		FCollectionClothSimAccessoryMeshFacade AccessoryFacade = ToCloth.AddGetSimAccessoryMesh();
		AccessoryFacade.Initialize(GenerateUniqueAccessoryMeshName(), FromCloth.GetSimPosition3D().Left(NumToSimVertices3D), FromCloth.GetSimNormal().Left(NumToSimVertices3D), FromCloth.GetSimBoneIndices().Left(NumToSimVertices3D), FromCloth.GetSimBoneWeights().Left(NumToSimVertices3D));
		return AccessoryFacade.GetSimAccessoryMeshIndex();
	}

	// Fill in tail data with ToCloth's sim mesh data.
	const int32 NumToSimVertices3D = ToCloth.GetNumSimVertices3D();
	TArray<FVector3f> Positions;
	TArray<FVector3f> Normals;
	TArray<TArray<int32>> BoneIndices;
	TArray<TArray<float>> BoneWeights;
	Positions.Reserve(NumToSimVertices3D);
	Normals.Reserve(NumToSimVertices3D);
	BoneIndices.Reserve(NumToSimVertices3D);
	BoneWeights.Reserve(NumToSimVertices3D);

	TConstArrayView<FVector3f> FromPositions = FromCloth.GetSimPosition3D();
	TConstArrayView<FVector3f> FromNormals = FromCloth.GetSimNormal();
	TConstArrayView<TArray<int32>> FromBoneIndices = FromCloth.GetSimBoneIndices();
	TConstArrayView<TArray<float>> FromBoneWeights = FromCloth.GetSimBoneWeights();
	for (int32 Index = 0; Index < FromCloth.GetNumSimVertices3D(); ++Index)
	{
		Positions.Emplace(FromPositions[Index]);
		Normals.Emplace(FromNormals[Index]);
		BoneIndices.Emplace(FromBoneIndices[Index]);
		BoneWeights.Emplace(FromBoneWeights[Index]);
	}
	TConstArrayView<FVector3f> ToPositions = ToCloth.GetSimPosition3D();
	TConstArrayView<FVector3f> ToNormals = ToCloth.GetSimNormal();
	TConstArrayView<TArray<int32>> ToBoneIndices = ToCloth.GetSimBoneIndices();
	TConstArrayView<TArray<float>> ToBoneWeights = ToCloth.GetSimBoneWeights();
	for (int32 Index = FromCloth.GetNumSimVertices3D(); Index < NumToSimVertices3D; ++Index)
	{
		Positions.Emplace(ToPositions[Index]);
		Normals.Emplace(ToNormals[Index]);
		BoneIndices.Emplace(ToBoneIndices[Index]);
		BoneWeights.Emplace(ToBoneWeights[Index]);
	}

	FCollectionClothSimAccessoryMeshFacade AccessoryFacade = ToCloth.AddGetSimAccessoryMesh();
	AccessoryFacade.Initialize(GenerateUniqueAccessoryMeshName(), TConstArrayView<FVector3f>(Positions), TConstArrayView<FVector3f>(Normals), TConstArrayView<TArray<int32>>(BoneIndices), TConstArrayView<TArray<float>>(BoneWeights));
	return AccessoryFacade.GetSimAccessoryMeshIndex();

}

#if WITH_EDITOR
TUniquePtr<FSkeletalMeshRenderData> FClothEngineTools::BuildSimPreviewRenderData(const UChaosClothAssetBase& Asset)
{
	const int32 NumModels = Asset.GetNumClothSimulationModels();
	if (!NumModels)
	{
		return nullptr;
	}

	TUniquePtr<FSkeletalMeshRenderData> RenderData = MakeUnique<FSkeletalMeshRenderData>();
	RenderData->LODRenderData.Add(new FSkeletalMeshLODRenderData());
	FSkeletalMeshLODRenderData& LODRenderData = RenderData->LODRenderData[0];

	TArray<FVector3f> AllPositions;
	TArray<FVector3f> AllNormals;
	TArray<uint32> AllIndices;
	TArray<FSkinWeightInfo> AllSkinWeights;

	for (int32 ModelIndex = 0; ModelIndex < NumModels; ++ModelIndex)
	{
		const TSharedPtr<const FChaosClothSimulationModel> SimModel = Asset.GetClothSimulationModel(ModelIndex);
		if (!SimModel || !SimModel->GetNumLods())
		{
			continue;
		}

		constexpr int32 PreviewLodIndex = 0;
		const TConstArrayView<FVector3f> Positions = SimModel->GetPositions(PreviewLodIndex);
		const TConstArrayView<FVector3f> Normals = SimModel->GetNormals(PreviewLodIndex);
		const TConstArrayView<uint32> Indices = SimModel->GetIndices(PreviewLodIndex);
		const TConstArrayView<FClothVertBoneData> BoneData = SimModel->GetBoneData(PreviewLodIndex);

		if (!Positions.Num() || !Indices.Num())
		{
			continue;
		}

		const uint32 VertexOffset = AllPositions.Num();

		AllPositions.Append(Positions.GetData(), Positions.Num());
		AllNormals.Append(Normals.GetData(), Normals.Num());

		const uint32 IndexOffset = AllIndices.Num();
		AllIndices.Reserve(AllIndices.Num() + Indices.Num());
		for (const uint32 Index : Indices)
		{
			AllIndices.Add(Index + VertexOffset);
		}

		// Collect unique bone indices for the section bone map
		TArray<FBoneIndexType> SectionBoneMap;
		TMap<FBoneIndexType, FBoneIndexType> BoneIndexRemap;

		for (const FClothVertBoneData& VertBoneData : BoneData)
		{
			const int32 NumInfluences = FMath::Min(VertBoneData.NumInfluences, (int32)MAX_TOTAL_INFLUENCES);
			for (int32 InfluenceIndex = 0; InfluenceIndex < NumInfluences; ++InfluenceIndex)
			{
				const FBoneIndexType GlobalIndex = static_cast<FBoneIndexType>(VertBoneData.BoneIndices[InfluenceIndex]);
				if (!BoneIndexRemap.Contains(GlobalIndex))
				{
					BoneIndexRemap.Add(GlobalIndex, static_cast<FBoneIndexType>(SectionBoneMap.Num()));
					SectionBoneMap.Add(GlobalIndex);
				}
			}
		}

		// Ensure at least one bone in the map (required by the skinning system)
		if (!SectionBoneMap.Num())
		{
			constexpr FBoneIndexType RootBoneIndex = 0;
			SectionBoneMap.Add(RootBoneIndex);
		}

		// Convert FClothVertBoneData to FSkinWeightInfo with section-local bone indices
		for (const FClothVertBoneData& VertBoneData : BoneData)
		{
			FSkinWeightInfo& SkinWeight = AllSkinWeights.AddDefaulted_GetRef();
			FMemory::Memset(SkinWeight.InfluenceBones, 0, sizeof(SkinWeight.InfluenceBones));
			FMemory::Memset(SkinWeight.InfluenceWeights, 0, sizeof(SkinWeight.InfluenceWeights));

			const int32 NumInfluences = FMath::Min(VertBoneData.NumInfluences, (int32)MAX_TOTAL_INFLUENCES);
			for (int32 InfluenceIndex = 0; InfluenceIndex < NumInfluences; ++InfluenceIndex)
			{
				const FBoneIndexType GlobalIndex = static_cast<FBoneIndexType>(VertBoneData.BoneIndices[InfluenceIndex]);
				SkinWeight.InfluenceBones[InfluenceIndex] = BoneIndexRemap[GlobalIndex];
				SkinWeight.InfluenceWeights[InfluenceIndex] =
					static_cast<uint16>(FMath::Clamp(VertBoneData.BoneWeights[InfluenceIndex] * 65535.f, 0.f, 65535.f));
			}
		}

		// Create render section for this model
		FSkelMeshRenderSection& Section = LODRenderData.RenderSections.AddDefaulted_GetRef();
		Section.MaterialIndex = 0;
		Section.BaseIndex = IndexOffset;
		Section.NumTriangles = (uint32)Indices.Num() / 3;
		Section.BaseVertexIndex = VertexOffset;
		Section.NumVertices = Positions.Num();
		Section.MaxBoneInfluences = MAX_TOTAL_INFLUENCES;
		Section.BoneMap = MoveTemp(SectionBoneMap);
		Section.ClothingData.AssetGuid = Asset.GetAssetGuid(ModelIndex);
		Section.DuplicatedVerticesBuffer.Init(0, TMap<int32, TArray<int32>>());
	}

	const uint32 NumVertices = AllPositions.Num();
	if (NumVertices)
	{
		// Position buffer
		LODRenderData.StaticVertexBuffers.PositionVertexBuffer.Init(NumVertices);
		for (uint32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
		{
			LODRenderData.StaticVertexBuffers.PositionVertexBuffer.VertexPosition(VertexIndex) = AllPositions[VertexIndex];
		}

		// Static mesh vertex buffer (tangents + UVs)
		LODRenderData.StaticVertexBuffers.StaticMeshVertexBuffer.Init(NumVertices, 1);
		for (uint32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
		{
			LODRenderData.StaticVertexBuffers.StaticMeshVertexBuffer.SetVertexTangents(
				VertexIndex, FVector3f::ZeroVector, FVector3f::ZeroVector, AllNormals[VertexIndex]);
			LODRenderData.StaticVertexBuffers.StaticMeshVertexBuffer.SetVertexUV(VertexIndex, 0, FVector2f::ZeroVector);
		}

		// Index buffer
		const uint8 DataTypeSize = (NumVertices < static_cast<uint32>(TNumericLimits<uint16>::Max())) ? sizeof(uint16) : sizeof(uint32);
		LODRenderData.MultiSizeIndexContainer.RebuildIndexBuffer(DataTypeSize, AllIndices);

		// Skin weight buffer
		LODRenderData.SkinWeightVertexBuffer.SetMaxBoneInfluences(MAX_TOTAL_INFLUENCES);
		LODRenderData.SkinWeightVertexBuffer = AllSkinWeights;

		// Skin weight profiles
		LODRenderData.SkinWeightProfilesData.Init(&LODRenderData.SkinWeightVertexBuffer);

		// Collect all referenced bone indices across all sections, including the full parent chain
		const FReferenceSkeleton& RefSkeleton = Asset.GetRefSkeleton();
		TSet<FBoneIndexType> AllBoneIndices;
		AllBoneIndices.Add(0);
		for (const FSkelMeshRenderSection& Section : LODRenderData.RenderSections)
		{
			for (const FBoneIndexType BoneIndex : Section.BoneMap)
			{
				int32 CurrentBone = BoneIndex;
				while (CurrentBone != INDEX_NONE && !AllBoneIndices.Contains(static_cast<FBoneIndexType>(CurrentBone)))
				{
					AllBoneIndices.Add(static_cast<FBoneIndexType>(CurrentBone));
					CurrentBone = RefSkeleton.GetParentIndex(CurrentBone);
				}
			}
		}
		LODRenderData.ActiveBoneIndices.Reset(AllBoneIndices.Num());
		LODRenderData.RequiredBones.Reset(AllBoneIndices.Num());
		for (const FBoneIndexType BoneIndex : AllBoneIndices)
		{
			LODRenderData.ActiveBoneIndices.Add(BoneIndex);
			LODRenderData.RequiredBones.Add(BoneIndex);
		}
		LODRenderData.ActiveBoneIndices.Sort();
		LODRenderData.RequiredBones.Sort();

		return RenderData;
	}

	return nullptr;
}

UMaterialInterface* FClothEngineTools::GetSimPreviewMaterial()
{
	static const FSoftObjectPath MaterialPath(
		TEXT("/Engine/EditorMaterials/Cloth/CameraLitDoubleSided.CameraLitDoubleSided"));
	return Cast<UMaterialInterface>(MaterialPath.TryLoad());
}
#endif  // #if WITH_EDITOR

#if WITH_EDITORONLY_DATA
TSharedRef<const FManagedArrayCollection> FClothEngineTools::TrimClothCollectionOnCook(
	const FString& AssetName,
	const TSharedRef<const FManagedArrayCollection>& ClothCollection,
	bool bLog)
{
	const uint64 InputSize = (uint64)ClothCollection->GetAllocatedSize();

	if (Private::bClothCollectionOnlyCookRequiredFacades)
	{
		// Properties
		TSharedRef<FManagedArrayCollection> PropertyCollection = MakeShared<FManagedArrayCollection>();
		::Chaos::Softs::FCollectionPropertyMutableFacade CollectionPropertyMutableFacade(PropertyCollection);
		CollectionPropertyMutableFacade.Copy(*ClothCollection);

		// Springs
		const ::Chaos::Softs::FEmbeddedSpringFacade InEmbeddedSpringFacade(ClothCollection.Get(), ClothCollectionGroup::SimVertices3D);
		if (InEmbeddedSpringFacade.IsValid())
		{
			::Chaos::Softs::FEmbeddedSpringFacade EmbeddedSpringFacade(*PropertyCollection, ClothCollectionGroup::SimVertices3D);
			EmbeddedSpringFacade.DefineSchema();
			constexpr int32 VertexOffset = 0;
			EmbeddedSpringFacade.Append(InEmbeddedSpringFacade, VertexOffset);
		}

		// Morph targets and accessory meshes
		const FCollectionClothConstFacade InClothFacade(ClothCollection);
		if (InClothFacade.IsValid(EClothCollectionExtendedSchemas::CookedOnly))
		{
			FCollectionClothFacade ClothFacade(PropertyCollection);
			ClothFacade.DefineSchema(EClothCollectionExtendedSchemas::CookedOnly);
			ClothFacade.InitializeCookedOnly(InClothFacade);
		}

		if (bLog)
		{
			const uint64 OutputSize = (uint64)PropertyCollection->GetAllocatedSize();
			UE_LOGF(LogChaosClothAsset, Display, "TrimOnCook[ON] %ls [%llu -> %llu, saved %llu bytes]",
				*AssetName, InputSize, OutputSize, InputSize - OutputSize);
		}
		return PropertyCollection;
	}
	if (bLog)
	{
		UE_LOGF(LogChaosClothAsset, Display, "TrimOnCook[OFF] %ls [%llu bytes]",
			*AssetName, InputSize);
	}
	return ClothCollection;
}

FString FClothEngineTools::GetCookedAssetPath(const FArchive& Ar)
{
	const FString ArchiveName = Ar.GetArchiveName();
	static const FString HarvesterPrefix(TEXT("PackageHarvester ("));
	if (ArchiveName.StartsWith(HarvesterPrefix) && ArchiveName.EndsWith(TEXT(")")))
	{
		return ArchiveName.Mid(HarvesterPrefix.Len(), ArchiveName.Len() - HarvesterPrefix.Len() - 1);
	}
	return ArchiveName;
}
#endif  // #if WITH_EDITORONLY_DATA

}  // End namespace UE::Chaos::ClothAsset

#undef LOCTEXT_NAMESPACE
