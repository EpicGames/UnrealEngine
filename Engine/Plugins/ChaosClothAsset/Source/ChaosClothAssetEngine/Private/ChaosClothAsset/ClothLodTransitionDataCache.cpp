// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ClothLodTransitionDataCache.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClothLodTransitionDataCache)

bool FChaosClothAssetLodTransitionDataCache::Serialize(FArchive& Ar)
{
	// Serialize normal tagged property data
	if (Ar.IsLoading() || Ar.IsSaving())
	{
		UScriptStruct* const Struct = FChaosClothAssetLodTransitionDataCache::StaticStruct();
		Struct->SerializeTaggedProperties(Ar, (uint8*)this, Struct, nullptr);
	}

	Ar << ModelHash;

	Ar << LODTransitionUpData;
	Ar << LODTransitionDownData;

	// Return true to confirm that serialization has already been taken care of
	return true;
}
