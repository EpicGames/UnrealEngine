// Copyright Epic Games, Inc. All Rights Reserved.
#include "ChaosClothAsset/ClothSimulationAccessoryMesh.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "ClothVertBoneData.h"

namespace UE::Chaos::ClothAsset
{
	FClothSimulationAccessoryMesh::FClothSimulationAccessoryMesh(const TSharedRef<FCollectionClothConstFacade>& InClothFacade, const ::Chaos::FClothingSimulationMesh& InMesh, const int32 InAccessoryMeshIndex)
		: ::Chaos::FClothingSimulationAccessoryMesh(InMesh, InClothFacade->GetSimAccessoryMeshName()[InAccessoryMeshIndex])
		, AccessoryMeshFacade(InClothFacade->GetSimAccessoryMesh(InAccessoryMeshIndex))
	{
		SetBoneData();
	}

	void FClothSimulationAccessoryMesh::SetBoneData()
	{
		TConstArrayView<TArray<int32>> SimBoneIndices = AccessoryMeshFacade.GetSimAccessoryMeshBoneIndices();
		TConstArrayView<TArray<float>> SimBoneWeights = AccessoryMeshFacade.GetSimAccessoryMeshBoneWeights();
		check(SimBoneIndices.Num() == SimBoneWeights.Num());
		BoneData.SetNum(SimBoneIndices.Num());
		for (int32 VertexIndex = 0; VertexIndex < SimBoneIndices.Num(); ++VertexIndex)
		{
			check(SimBoneIndices[VertexIndex].Num() == SimBoneWeights[VertexIndex].Num());
			check(SimBoneIndices[VertexIndex].Num() <= FClothVertBoneData::MaxTotalInfluences);
			
			BoneData[VertexIndex].NumInfluences = FMath::Min(SimBoneIndices[VertexIndex].Num(), FClothVertBoneData::MaxTotalInfluences);
			for (int32 BoneIndex = 0; BoneIndex < BoneData[VertexIndex].NumInfluences; ++BoneIndex)
			{
				BoneData[VertexIndex].BoneIndices[BoneIndex] = SimBoneIndices[VertexIndex][BoneIndex];
				BoneData[VertexIndex].BoneWeights[BoneIndex] = SimBoneWeights[VertexIndex][BoneIndex];
			}
		}
	}

	FClothSimulationAccessoryMesh::~FClothSimulationAccessoryMesh() = default;

	int32 FClothSimulationAccessoryMesh::GetNumPoints() const
	{
		return AccessoryMeshFacade.GetSimAccessoryMeshPosition3D().Num();
	}

	TConstArrayView<FVector3f> FClothSimulationAccessoryMesh::GetPositions() const
	{
		return AccessoryMeshFacade.GetSimAccessoryMeshPosition3D();
	}

	TConstArrayView<FVector3f> FClothSimulationAccessoryMesh::GetNormals() const
	{
		return AccessoryMeshFacade.GetSimAccessoryMeshNormal();
	}
}