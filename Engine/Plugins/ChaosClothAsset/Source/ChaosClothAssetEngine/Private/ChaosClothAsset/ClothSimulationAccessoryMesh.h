// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ChaosCloth/ChaosClothingSimulationAccessoryMesh.h"
#include "ChaosClothAsset/CollectionClothSimAccessoryMeshFacade.h"

namespace UE::Chaos::ClothAsset
{
	class FCollectionClothConstFacade;

	// Accessory mesh used as an input to a simulation. They are expected to share topology (e.g., Indices, PatternIndices) with their associated FClothingSimulationMesh
	class FClothSimulationAccessoryMesh : public ::Chaos::FClothingSimulationAccessoryMesh
	{
	public:
		FClothSimulationAccessoryMesh(const TSharedRef<FCollectionClothConstFacade>& InClothFacade, const ::Chaos::FClothingSimulationMesh& InMesh, const int32 InAccessoryMeshIndex);

		virtual ~FClothSimulationAccessoryMesh();

		/* Return the number of points. */
		virtual int32 GetNumPoints() const override;

		/* Return the number of pattern points (2d, unwelded). */
		virtual int32 GetNumPatternPoints() const override
		{
			return 0;
		}

		/* Return the source mesh positions (pre-skinning). */
		virtual TConstArrayView<FVector3f> GetPositions() const override;

		/* Return the source mesh 2d pattern positions. */
		virtual TConstArrayView<FVector2f> GetPatternPositions() const override
		{
			return TConstArrayView<FVector2f>();
		}

		/* Return the source mesh normals (pre-skinning). */
		virtual TConstArrayView<FVector3f> GetNormals() const override;

		/* Return the bone data containing bone weights and influences. */
		virtual TConstArrayView<FClothVertBoneData> GetBoneData() const override
		{
			return BoneData;
		}
		// ---- End of the Cloth interface ----

	private:
		void SetBoneData();

		const FCollectionClothSimAccessoryMeshConstFacade AccessoryMeshFacade;
		TArray<FClothVertBoneData> BoneData;
	};
} // namespace UE::Chaos::ClothAsset

