// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ClothSimulationContext.h"
#include "ChaosClothAsset/ClothAsset.h"
#include "ChaosClothAsset/ClothComponentAdapter.h"
#include "ChaosClothAsset/ClothSimulationModel.h"
#include "Components/SkinnedMeshComponent.h"
#include "Engine/World.h"
#include "HAL/IConsoleManager.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "ClothingSystemRuntimeTypes.h"
#include "SceneInterface.h"

namespace UE::Chaos::ClothAsset
{
	void FClothSimulationContext::Fill(const IClothComponentAdapter& ClothComponentAdapter, float InDeltaTime, float MaxDeltaTime, bool bIsInitialization, FClothingSimulationCacheData* InCacheData)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FClothSimulationContext_Fill);
		// Set the time
		DeltaTime = FMath::Min(InDeltaTime, MaxDeltaTime);

		// Set the current LOD index. Do this before exiting early if there is no simulation data.
		LodIndex = ClothComponentAdapter.GetOwnerComponent().GetPredictedLODLevel();

		if (!bIsInitialization && !ClothComponentAdapter.HasAnySimulationMeshData(LodIndex))
		{
			return;  // This Lod will not simulate, so no need to update the context.
		}

		// Retrieve the list of assets to simulate
		const TArray<const UChaosClothAssetBase*> Assets = ClothComponentAdapter.GetAssets();

		// Set the teleport mode
		static const IConsoleVariable* const CVarMaxDeltaTimeTeleportMultiplier = IConsoleManager::Get().FindConsoleVariable(TEXT("p.Cloth.MaxDeltaTimeTeleportMultiplier"));
		constexpr float MaxDeltaTimeTeleportMultiplierDefault = 1.5f;
		const float MaxDeltaTimeTeleportMultiplier = CVarMaxDeltaTimeTeleportMultiplier ? CVarMaxDeltaTimeTeleportMultiplier->GetFloat() : MaxDeltaTimeTeleportMultiplierDefault;

		const EClothingTeleportMode ComponentTeleportMode = ClothComponentAdapter.GetClothTeleportMode();

		bTeleport = (InDeltaTime > MaxDeltaTime * MaxDeltaTimeTeleportMultiplier) ? true : 
			(ComponentTeleportMode >= EClothingTeleportMode::Teleport);
		bReset = ComponentTeleportMode == EClothingTeleportMode::TeleportAndReset;
		
		bResetRestLengthsWithMorphTarget = ClothComponentAdapter.NeedsResetRestLengths();
		if (bResetRestLengthsWithMorphTarget)
		{
			ResetRestLengthsMorphTargetName = ClothComponentAdapter.GetRestLengthsMorphTargetName();
		}

		VelocityScale = (!bTeleport && !bReset && InDeltaTime > 0.f) ?
			FMath::Min(InDeltaTime, MaxDeltaTime) / InDeltaTime :
			bReset ? 1.f : 0.f;  // Set to 0 when teleporting and 1 when resetting to match the internal solver's behavior

		// Copy component transform
		ComponentTransform = ClothComponentAdapter.GetOwnerComponent().GetComponentTransform();

		// Copy solver geometry scale
		SolverGeometryScale = ClothComponentAdapter.GetClothGeometryScale();

		// Update bone transforms
		const FReferenceSkeleton* const ReferenceSkeleton = ClothComponentAdapter.GetReferenceSkeleton();
		int32 NumBones = ReferenceSkeleton ? ReferenceSkeleton->GetNum() : 0;

		if (USkinnedMeshComponent* const LeaderComponent = ClothComponentAdapter.GetOwnerComponent().LeaderPoseComponent.Get())
		{
			const TArray<int32>& LeaderBoneMap = ClothComponentAdapter.GetOwnerComponent().GetLeaderBoneMap();
			if (!LeaderBoneMap.Num())
			{
				// This case indicates an invalid leader pose component (e.g. no skeletal mesh)
				BoneTransforms.Empty(NumBones);
				BoneTransforms.AddDefaulted(NumBones);
			}
			else
			{
				NumBones = LeaderBoneMap.Num();
				BoneTransforms.Reset(NumBones);
				BoneTransforms.AddDefaulted(NumBones);

				if (!bIsInitialization)  // Initializations must be done in bind pose
				{
					const TArray<FTransform>& LeaderTransforms = LeaderComponent->GetComponentSpaceTransforms();
					for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
					{
						bool bFoundLeader = false;
						if (LeaderBoneMap.IsValidIndex(BoneIndex))
						{
							const int32 LeaderIndex = LeaderBoneMap[BoneIndex];
							if (LeaderIndex != INDEX_NONE && LeaderIndex < LeaderTransforms.Num())
							{
								BoneTransforms[BoneIndex] = LeaderTransforms[LeaderIndex];
								bFoundLeader = true;
							}
						}

						if (!bFoundLeader && ReferenceSkeleton)
						{
							const int32 ParentIndex = ReferenceSkeleton->GetParentIndex(BoneIndex);

							BoneTransforms[BoneIndex] =
								BoneTransforms.IsValidIndex(ParentIndex) && ParentIndex < BoneIndex ?
								BoneTransforms[ParentIndex] * ReferenceSkeleton->GetRefBonePose()[BoneIndex] :
								ReferenceSkeleton->GetRefBonePose()[BoneIndex];
						}
					}
				}
			}
		}
		else if (!bIsInitialization)  // Initializations must be done in bind pose
		{
			BoneTransforms = ClothComponentAdapter.GetOwnerComponent().GetComponentSpaceTransforms();
		}
		else
		{
			BoneTransforms.Reset(NumBones);
			BoneTransforms.AddDefaulted(NumBones);
		}

		// Update bone matrices
		RefToLocalMatrices.Reset(NumBones);
		RequiredExtraBones.Reset();

		bool bSetRefToLocalMatricesToIdentity = true;
		if (!bIsInitialization)
		{
			for (const UChaosClothAssetBase* const Asset : Assets)
			{
				if (Asset)
				{
					for (int32 ModelIndex = 0; ModelIndex < Asset->GetNumClothSimulationModels(); ++ModelIndex)
					{
						const TSharedPtr<const FChaosClothSimulationModel> ClothModel = ClothComponentAdapter.GetActiveClothSimulationModel(Asset, ModelIndex);
						if (ClothModel && ClothModel->IsValidLodIndex(LodIndex))
						{
							RequiredExtraBones.Append(ClothModel->ClothSimulationLodModels[LodIndex].RequiredExtraBoneIndices);
						}
					}
					bSetRefToLocalMatricesToIdentity = false;
				}
			}
		}

		if (bSetRefToLocalMatricesToIdentity)
		{
			RefToLocalMatrices.AddUninitialized(NumBones);

			for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
			{
				RefToLocalMatrices[BoneIndex] = FMatrix44f::Identity;
			}
		}
		else
		{
			ClothComponentAdapter.GetOwnerComponent().GetCurrentRefToLocalMatrices(RefToLocalMatrices, LodIndex, &RequiredExtraBones);
		}
		
		// Update gravity
		const UWorld* const World = ClothComponentAdapter.GetOwnerComponent().GetWorld();
		constexpr float EarthGravity = -981.f;
		WorldGravity = FVector(0.f, 0.f, World ? World->GetGravityZ() : EarthGravity);

		// Update wind velocity
		if (World && World->Scene)
		{
			const FVector Position = ClothComponentAdapter.GetOwnerComponent().GetComponentTransform().GetTranslation();

			float WindSpeed;
			float WindMinGust;
			float WindMaxGust;
			World->Scene->GetWindParameters_GameThread(Position, WindVelocity, WindSpeed, WindMinGust, WindMaxGust);

			WindVelocity *= WindSpeed;
		}
		else
		{
			WindVelocity = FVector::ZeroVector;
		}

		if (InCacheData)
		{
			CacheData = MoveTemp(*InCacheData);
		}
		else
		{
			CacheData.Reset();
		}
	}
}  // End namespace UE::Chaos::ClothAsset
