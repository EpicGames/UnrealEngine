// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Math/Transform.h"
#include "ClothingSimulationCacheData.h"

namespace UE::Chaos::ClothAsset
{
	class IClothComponentAdapter;

	/**
	 * Cloth simulation context.
	 * Class used to pass data between the cloth component and its simulation proxy.
	 * The context is set in the game thread before the simulation is started and should not
	 * be modified until the simulation is ended.
	 */
	struct FClothSimulationContext
	{
		/** Time step in seconds. */
		float DeltaTime;

		/** Current LOD index at the start of the simulation. */
		int32 LodIndex;

		/** Component to world transform. */
		FTransform ComponentTransform;

		/** Scale applied to solver. */
		float SolverGeometryScale;

		/** Component space bone transforms of the owning component. */
		TArray<FTransform> BoneTransforms;

		/** Bone matrices used for skinning. */
		TArray<FMatrix44f> RefToLocalMatrices;

		/** Teleport. */
		bool bTeleport = false;

		/** Reset. */
		bool bReset = false;

		/** Reset RestLengths (with morph target) */
		bool bResetRestLengthsWithMorphTarget = false;
		FString ResetRestLengthsMorphTargetName;

		/** Velocity scale to compensate for time differences when the MaxPhysicsDeltaTime kicks in. */
		float VelocityScale;

		/** Gravity extracted from the world. */
		FVector WorldGravity;

		/** Wind velocity at the component location. */
		FVector WindVelocity;

		/* Data used by Chaos Cache*/
		FClothingSimulationCacheData CacheData;

		/** Fill the context from data collected from the specified component. */
		void Fill(const IClothComponentAdapter& ClothComponentAdapter, float InDeltaTime, float MaxDeltaTime, bool bIsInitialization = false, FClothingSimulationCacheData* CacheData = nullptr);

	private:
		TArray<uint16> RequiredExtraBones;  // Avoids reallocations while calculating RefToLocalMatrices
	};
}
