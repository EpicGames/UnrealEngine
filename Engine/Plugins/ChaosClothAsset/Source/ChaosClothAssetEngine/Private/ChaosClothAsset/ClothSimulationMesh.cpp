// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ClothSimulationMesh.h"
#include "ChaosClothAsset/ClothSimulationAccessoryMesh.h"
#include "ChaosClothAsset/ClothAsset.h"
#include "ChaosClothAsset/ClothSimulationModel.h"
#include "ChaosClothAsset/ClothSimulationContext.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "ClothSimulationMesh.h"
#include "ReferenceSkeleton.h"

namespace UE::Chaos::ClothAsset
{
	FClothSimulationMesh::FClothSimulationMesh(
		const FChaosClothSimulationModel& InClothSimulationModel,
		const FClothSimulationContext& InClothSimulationContext,
		const TArray<TSharedRef<const FManagedArrayCollection>>& InManagedArrayCollections,
		const FString& DebugName)
		: ::Chaos::FClothingSimulationMesh(DebugName)
		, ClothSimulationModel(InClothSimulationModel)
		, ClothSimulationContext(InClothSimulationContext)
		, ManagedArrayCollections(InManagedArrayCollections)
	{
#if CHAOS_DEBUG_DRAW
		const int32 ReferenceBoneIndex = GetReferenceBoneIndex();
		const int32 UsedBoneNameIndex = ClothSimulationModel.UsedBoneIndices.Find(ReferenceBoneIndex);
		if (ClothSimulationModel.UsedBoneNames.IsValidIndex(UsedBoneNameIndex))
		{
			ReferenceBoneName = ClothSimulationModel.UsedBoneNames[UsedBoneNameIndex];
		}
#endif
		ClothFacades.Reserve(ManagedArrayCollections.Num());
		AccessoryMeshes.Reserve(ManagedArrayCollections.Num());
		for (const TSharedRef<const FManagedArrayCollection>& Collection : ManagedArrayCollections)
		{
			const TSharedRef<FCollectionClothConstFacade>& ClothFacade = ClothFacades.Emplace_GetRef(MakeShared<FCollectionClothConstFacade>(Collection));
			TMap<FName, TUniquePtr<FClothSimulationAccessoryMesh>>& LODAccessoryMeshes = AccessoryMeshes.AddDefaulted_GetRef();
			for (int32 AccessoryIndex = 0; AccessoryIndex < ClothFacade->GetNumSimAccessoryMeshes(); ++AccessoryIndex)
			{
				const FCollectionClothSimAccessoryMeshConstFacade AccessoryFacade = ClothFacade->GetSimAccessoryMesh(AccessoryIndex);
				LODAccessoryMeshes.Add(AccessoryFacade.GetSimAccessoryMeshName(), MakeUnique<FClothSimulationAccessoryMesh>(ClothFacade, *this, AccessoryIndex));
			}
		}
	}

	int32 FClothSimulationMesh::GetNumLODs() const
	{
		return ClothSimulationModel.GetNumLods();
	}

	int32 FClothSimulationMesh::GetLODIndex() const
	{
		return FMath::Min(ClothSimulationContext.LodIndex, GetNumLODs() - 1);
	}

	int32 FClothSimulationMesh::GetOwnerLODIndex(int32 LODIndex) const
	{
		return LODIndex;  // The component LOD currently matches the asset LOD
	}

	bool FClothSimulationMesh::IsValidLODIndex(int32 LODIndex) const
	{
		return LODIndex >= 0 && LODIndex < GetNumLODs();
	}

	int32 FClothSimulationMesh::GetNumPoints(int32 LODIndex) const
	{
		return ClothSimulationModel.GetPositions(LODIndex).Num();
	}

	int32 FClothSimulationMesh::GetNumPatternPoints(int32 LODIndex) const
	{
		return ClothSimulationModel.GetPatternPositions(LODIndex).Num();
	}

	TConstArrayView<FVector3f> FClothSimulationMesh::GetPositions(int32 LODIndex) const
	{
		return TConstArrayView<FVector3f>(ClothSimulationModel.GetPositions(LODIndex));
	}

	TConstArrayView<FVector2f> FClothSimulationMesh::GetPatternPositions(int32 LODIndex) const
	{
		return TConstArrayView<FVector2f>(ClothSimulationModel.GetPatternPositions(LODIndex));
	}

	TConstArrayView<FVector3f> FClothSimulationMesh::GetNormals(int32 LODIndex) const
	{
		return TConstArrayView<FVector3f>(ClothSimulationModel.GetNormals(LODIndex));
	}

	TConstArrayView<uint32> FClothSimulationMesh::GetIndices(int32 LODIndex) const
	{
		return TConstArrayView<uint32>(ClothSimulationModel.GetIndices(LODIndex));
	}

	TConstArrayView<uint32> FClothSimulationMesh::GetPatternIndices(int32 LODIndex) const
	{
		return TConstArrayView<uint32>(ClothSimulationModel.GetPatternIndices(LODIndex));
	}

	TConstArrayView<uint32> FClothSimulationMesh::GetPatternToWeldedIndices(int32 LODIndex) const
	{
		return TConstArrayView<uint32>(ClothSimulationModel.GetPatternToWeldedIndices(LODIndex));
	}

	TArray<FName> FClothSimulationMesh::GetWeightMapNames(int32 LODIndex) const
	{
		TArray<FName> WeightMapNames;
		if (ClothSimulationModel.ClothSimulationLodModels.IsValidIndex(LODIndex))
		{
			ClothSimulationModel.ClothSimulationLodModels[LODIndex].WeightMaps.GetKeys(WeightMapNames);
		}
		return WeightMapNames;
	}

	TMap<FString, int32> FClothSimulationMesh::GetWeightMapIndices(int32 LODIndex) const
	{
		TMap<FString, int32> WeightMapIndices;

		// Retrieve weight map names for this cloth
		const TArray<FName> WeightMapNames = GetWeightMapNames(LODIndex);
		WeightMapIndices.Reserve(WeightMapNames.Num());

		for (int32 WeightMapIndex = 0; WeightMapIndex < WeightMapNames.Num(); ++WeightMapIndex)
		{
			const FName& WeightMapName = WeightMapNames[WeightMapIndex];
			WeightMapIndices.Emplace(WeightMapName.ToString(), WeightMapIndex);
		}
		return WeightMapIndices;
	}

	TArray<TConstArrayView<::Chaos::FRealSingle>> FClothSimulationMesh::GetWeightMaps(int32 LODIndex) const
	{
		TArray<TConstArrayView<::Chaos::FRealSingle>> WeightMaps;

		// Retrieve weight map names for this cloth
		const TArray<FName> WeightMapNames = GetWeightMapNames(LODIndex);
		WeightMaps.Reserve(WeightMapNames.Num());

		for (int32 WeightMapIndex = 0; WeightMapIndex < WeightMapNames.Num(); ++WeightMapIndex)
		{
			const FName& WeightMapName = WeightMapNames[WeightMapIndex];

			const TArray<float>& WeightMap = ClothSimulationModel.ClothSimulationLodModels[LODIndex].WeightMaps.FindChecked(WeightMapName);
			static_assert(std::is_same_v<::Chaos::FRealSingle, float>, "FRealSingle must be same as float for the Array View to match.");

			WeightMaps.Emplace(TConstArrayView<::Chaos::FRealSingle>(WeightMap));
		}
		return WeightMaps;
	}

	TMap<FString, const TSet<int32>*> FClothSimulationMesh::GetVertexSets(int32 LODIndex) const
	{
		TMap<FString, const TSet<int32>*> VertexSets;
		if (ClothSimulationModel.ClothSimulationLodModels.IsValidIndex(LODIndex))
		{
			VertexSets.Reserve(ClothSimulationModel.ClothSimulationLodModels[LODIndex].VertexSets.Num());
			for (TMap<FName, TSet<int32>>::TConstIterator Iter = ClothSimulationModel.ClothSimulationLodModels[LODIndex].VertexSets.CreateConstIterator(); Iter; ++Iter)
			{
				VertexSets.Emplace(Iter.Key().ToString(), &Iter.Value());
			}
		}
		return VertexSets;
	}

	TMap<FString, const TSet<int32>*> FClothSimulationMesh::GetFaceSets(int32 LODIndex) const
	{
		TMap<FString, const TSet<int32>*> FaceSets;
		if (ClothSimulationModel.ClothSimulationLodModels.IsValidIndex(LODIndex))
		{
			FaceSets.Reserve(ClothSimulationModel.ClothSimulationLodModels[LODIndex].FaceSets.Num());
			for (TMap<FName, TSet<int32>>::TConstIterator Iter = ClothSimulationModel.ClothSimulationLodModels[LODIndex].FaceSets.CreateConstIterator(); Iter; ++Iter)
			{
				FaceSets.Emplace(Iter.Key().ToString(), &Iter.Value());
			}
		}
		return FaceSets;
	}

	TMap<FString, TConstArrayView<int32>> FClothSimulationMesh::GetFaceIntMaps(int32 LODIndex) const
	{
		TMap<FString, TConstArrayView<int32>> FaceIntMaps;
		if (ClothSimulationModel.ClothSimulationLodModels.IsValidIndex(LODIndex))
		{
			FaceIntMaps.Reserve(ClothSimulationModel.ClothSimulationLodModels[LODIndex].FaceIntMaps.Num());
			for (TMap<FName, TArray<int32>>::TConstIterator Iter = ClothSimulationModel.ClothSimulationLodModels[LODIndex].FaceIntMaps.CreateConstIterator(); Iter; ++Iter)
			{
				FaceIntMaps.Emplace(Iter.Key().ToString(), TConstArrayView<int32>(Iter.Value()));
			}
		}
		return FaceIntMaps;
	}

	TArray<TConstArrayView<TTuple<int32, int32, float>>> FClothSimulationMesh::GetTethers(int32 LODIndex, bool /*bUseGeodesicTethers*/) const
	{
		return ClothSimulationModel.GetTethers(LODIndex);
	}

	int32 FClothSimulationMesh::GetReferenceBoneIndex() const
	{
		return ClothSimulationModel.ReferenceBoneIndex;
	}

	FTransform FClothSimulationMesh::GetReferenceBoneTransform() const
	{
		// TODO: Leader pose component (see FClothingSimulationContextCommon::FillBoneTransforms)
		const TArray<FTransform>& BoneTransforms = ClothSimulationContext.BoneTransforms;

		if (BoneTransforms.IsValidIndex(ClothSimulationModel.ReferenceBoneIndex))
		{
			return BoneTransforms[ClothSimulationModel.ReferenceBoneIndex] * GetComponentToWorldTransform();
		}
		return GetComponentToWorldTransform();
	}

	const TArray<FTransform>& FClothSimulationMesh::GetBoneTransforms() const
	{
		// TODO: Leader pose component (see FClothingSimulationContextCommon::FillBoneTransforms)
		return ClothSimulationContext.BoneTransforms;
	}

	const FTransform& FClothSimulationMesh::GetComponentToWorldTransform() const
	{
		return ClothSimulationContext.ComponentTransform;
	}

	const TArray<FMatrix44f>& FClothSimulationMesh::GetRefToLocalMatrices() const
	{
		return ClothSimulationContext.RefToLocalMatrices;
	}

	TConstArrayView<int32> FClothSimulationMesh::GetBoneMap() const
	{
		return ClothSimulationModel.UsedBoneIndices;
	}

	TConstArrayView<FClothVertBoneData> FClothSimulationMesh::GetBoneData(int32 LODIndex) const
	{
		return ClothSimulationModel.GetBoneData(LODIndex);
	}

	TConstArrayView<FMeshToMeshVertData> FClothSimulationMesh::GetTransitionUpSkinData(int32 LODIndex) const
	{
		return ClothSimulationModel.ClothSimulationLodModels[LODIndex].LODTransitionUpData;
	}

	TConstArrayView<FMeshToMeshVertData> FClothSimulationMesh::GetTransitionDownSkinData(int32 LODIndex) const
	{
		return ClothSimulationModel.ClothSimulationLodModels[LODIndex].LODTransitionDownData;
	}

	TSharedPtr<const FManagedArrayCollection> FClothSimulationMesh::GetManagedArrayCollection(int32 LODIndex) const
	{
		return ManagedArrayCollections.IsValidIndex(LODIndex) ? ManagedArrayCollections[LODIndex] : TSharedPtr<const FManagedArrayCollection>();
	}

	int32 FClothSimulationMesh::FindMorphTargetByName(int32 LODIndex, const FString& Name) const
	{
		return ClothFacades.IsValidIndex(LODIndex) ? ClothFacades[LODIndex]->FindSimMorphTargetIndexByName(Name) : INDEX_NONE;
	}

	TConstArrayView<FString> FClothSimulationMesh::GetAllMorphTargetNames(int32 LODIndex) const
	{
		if (ClothFacades.IsValidIndex(LODIndex))
		{
			return ClothFacades[LODIndex]->GetSimMorphTargetName();
		}
		return TConstArrayView<FString>();
	}

	TConstArrayView<FVector3f> FClothSimulationMesh::GetMorphTargetPositionDeltas(int32 LODIndex, int32 MorphTargetIndex) const
	{
		if (ClothFacades.IsValidIndex(LODIndex) && (MorphTargetIndex >= 0 && MorphTargetIndex < ClothFacades[LODIndex]->GetNumSimMorphTargets()))
		{
			FCollectionClothSimMorphTargetConstFacade MorphTargetFacade = ClothFacades[LODIndex]->GetSimMorphTarget(MorphTargetIndex);
			return MorphTargetFacade.GetSimMorphTargetPositionDelta();
		}
		return TConstArrayView<FVector3f>();
	}

	TConstArrayView<FVector3f> FClothSimulationMesh::GetMorphTargetTangentZDeltas(int32 LODIndex, int32 MorphTargetIndex) const
	{
		if (ClothFacades.IsValidIndex(LODIndex) && (MorphTargetIndex >= 0 && MorphTargetIndex < ClothFacades[LODIndex]->GetNumSimMorphTargets()))
		{
			FCollectionClothSimMorphTargetConstFacade MorphTargetFacade = ClothFacades[LODIndex]->GetSimMorphTarget(MorphTargetIndex);
			return MorphTargetFacade.GetSimMorphTargetTangentZDelta();
		}
		return TConstArrayView<FVector3f>();
	}

	TConstArrayView<int32> FClothSimulationMesh::GetMorphTargetIndices(int32 LODIndex, int32 MorphTargetIndex) const
	{
		if (ClothFacades.IsValidIndex(LODIndex) && (MorphTargetIndex >= 0 && MorphTargetIndex < ClothFacades[LODIndex]->GetNumSimMorphTargets()))
		{
			FCollectionClothSimMorphTargetConstFacade MorphTargetFacade = ClothFacades[LODIndex]->GetSimMorphTarget(MorphTargetIndex);
			return MorphTargetFacade.GetSimMorphTargetSimVertex3DIndex();
		}
		return TConstArrayView<int32>();
	}

	TConstArrayView<FName> FClothSimulationMesh::GetAllAccessoryMeshNames(int32 LODIndex) const
	{
		if (ClothFacades.IsValidIndex(LODIndex))
		{
			return ClothFacades[LODIndex]->GetSimAccessoryMeshName();
		}
		return TConstArrayView<FName>();
	}

	const ::Chaos::FClothingSimulationAccessoryMesh* FClothSimulationMesh::GetAccessoryMesh(int32 LODIndex, const FName& AccessoryMeshName) const
	{
		if (AccessoryMeshes.IsValidIndex(LODIndex))
		{
			if (const TUniquePtr<FClothSimulationAccessoryMesh>* FoundMesh = AccessoryMeshes[LODIndex].Find(AccessoryMeshName))
			{
				return FoundMesh->Get();
			}
		}
		return nullptr;
	}
}
