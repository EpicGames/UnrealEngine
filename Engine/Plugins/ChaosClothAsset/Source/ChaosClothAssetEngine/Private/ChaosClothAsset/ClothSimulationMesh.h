// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ChaosCloth/ChaosClothingSimulationMesh.h"

class UChaosClothAsset;
struct FChaosClothSimulationModel;

namespace UE::Chaos::ClothAsset
{
	struct FClothSimulationContext;
	class FCollectionClothConstFacade;
	class FClothSimulationAccessoryMesh;

	class FClothSimulationMesh : public ::Chaos::FClothingSimulationMesh
	{
	public:
		FClothSimulationMesh(
			const FChaosClothSimulationModel& InClothSimulationModel,
			const FClothSimulationContext& InClothSimulationContext,
			const TArray<TSharedRef<const FManagedArrayCollection>>& InManagedArrayCollections,
			const FString& DebugName);

		virtual ~FClothSimulationMesh() override = default;

		FClothSimulationMesh(const FClothSimulationMesh&) = delete;
		FClothSimulationMesh(FClothSimulationMesh&&) = delete;
		FClothSimulationMesh& operator=(const FClothSimulationMesh&) = delete;
		FClothSimulationMesh& operator=(FClothSimulationMesh&&) = delete;

		//~ Begin FClothingSimulationMesh Interface
		virtual int32 GetNumLODs() const override;
		virtual int32 GetLODIndex() const override;
		virtual int32 GetOwnerLODIndex(int32 LODIndex) const override;
		virtual bool IsValidLODIndex(int32 LODIndex) const override;
		virtual int32 GetNumPoints(int32 LODIndex) const override;
		virtual int32 GetNumPatternPoints(int32 LODIndex) const override;
		virtual TConstArrayView<FVector3f> GetPositions(int32 LODIndex) const override;
		virtual TConstArrayView<FVector2f> GetPatternPositions(int32 LODIndex) const override;
		virtual TConstArrayView<FVector3f> GetNormals(int32 LODIndex) const override;
		virtual TConstArrayView<uint32> GetIndices(int32 LODIndex) const override;
		virtual TConstArrayView<uint32> GetPatternIndices(int32 LODIndex) const override;
		virtual TConstArrayView<uint32> GetPatternToWeldedIndices(int32 LODIndex) const override;
		virtual TArray<FName> GetWeightMapNames(int32 LODIndex) const override;
		virtual TMap<FString, int32> GetWeightMapIndices(int32 LODIndex) const override;
		virtual TArray<TConstArrayView<::Chaos::FRealSingle>> GetWeightMaps(int32 LODIndex) const override;
		virtual TMap<FString, const TSet<int32>*> GetVertexSets(int32 LODIndex) const override;
		virtual TMap<FString, const TSet<int32>*> GetFaceSets(int32 LODIndex) const override;
		virtual TMap<FString, TConstArrayView<int32>> GetFaceIntMaps(int32 LODIndex) const override;
		// Note: there is only one set of tethers stored on ClothSimulationMesh assets
		virtual TArray<TConstArrayView<TTuple<int32, int32, float>>> GetTethers(int32 LODIndex, bool /*bUseGeodesicTethers*/) const override;
		virtual int32 GetReferenceBoneIndex() const override;
		virtual FTransform GetReferenceBoneTransform() const override;
		virtual const TArray<FTransform>& GetBoneTransforms() const override;
		virtual const FTransform& GetComponentToWorldTransform() const override;
		virtual const TArray<FMatrix44f>& GetRefToLocalMatrices() const override;
		virtual TConstArrayView<int32> GetBoneMap() const override;
		virtual TConstArrayView<FClothVertBoneData> GetBoneData(int32 LODIndex) const override;
		virtual TConstArrayView<FMeshToMeshVertData> GetTransitionUpSkinData(int32 LODIndex) const override;
		virtual TConstArrayView<FMeshToMeshVertData> GetTransitionDownSkinData(int32 LODIndex) const override;
		virtual TSharedPtr<const FManagedArrayCollection> GetManagedArrayCollection(int32 LODIndex) const override;
		virtual int32 FindMorphTargetByName(int32 LODIndex, const FString& Name) const override;
		virtual TConstArrayView<FString> GetAllMorphTargetNames(int32 LODIndex) const override;
		virtual TConstArrayView<FVector3f> GetMorphTargetPositionDeltas(int32 LODIndex, int32 MorphTargetIndex) const override;
		virtual TConstArrayView<FVector3f> GetMorphTargetTangentZDeltas(int32 LODIndex, int32 MorphTargetIndex) const override;
		virtual TConstArrayView<int32> GetMorphTargetIndices(int32 LODIndex, int32 MorphTargetIndex) const override;
		virtual TConstArrayView<FName> GetAllAccessoryMeshNames(int32 LODIndex) const override;
		virtual const ::Chaos::FClothingSimulationAccessoryMesh* GetAccessoryMesh(int32 LODIndex, const FName& AccessoryMeshName) const override;
		//~ End FClothingSimulationMesh Interface

	private:
		const FChaosClothSimulationModel& ClothSimulationModel;
		const FClothSimulationContext& ClothSimulationContext;
		const TArray<TSharedRef<const FManagedArrayCollection>>& ManagedArrayCollections;
		TArray<TSharedRef<FCollectionClothConstFacade>> ClothFacades;
		TArray<TMap<FName,TUniquePtr<FClothSimulationAccessoryMesh>>> AccessoryMeshes;
	};
}
