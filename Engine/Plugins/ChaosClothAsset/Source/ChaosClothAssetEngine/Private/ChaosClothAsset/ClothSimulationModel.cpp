// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ClothSimulationModel.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "ChaosClothAsset/CollectionClothSelectionFacade.h"
#include "ChaosClothAsset/ClothAssetPrivate.h"
#include "ChaosClothAsset/ClothCollectionGroup.h"
#include "ChaosClothAsset/ClothGeometryTools.h"
#include "ChaosClothAsset/ClothEngineTools.h"
#include "ChaosClothAsset/ClothLodTransitionDataCache.h"
#include "Chaos/CollectionPropertyFacade.h"
#include "Utils/ClothingMeshUtils.h"
#include "ReferenceSkeleton.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClothSimulationModel)

namespace UE::Chaos::ClothAsset::Private
{
	// Memory counter archive for GetResourceSizeEx
	class FMemoryCounterArchive final : public FArchive
	{
	public:
		FMemoryCounterArchive()
			: Size(0)
		{
			SetIsSaving(true);
			SetIsPersistent(true);
			ArIsCountingMemory = true;
		}

		virtual void Serialize(void*, int64 Length) override { Size += Length; }
		virtual int64 TotalSize() override { return Size; }

	private:
		int64 Size;
	};

	// Find the root bone for this cloth asset (common bone for all used bones)
	static int32 CalculateReferenceBoneIndex(const TArray<FChaosClothSimulationLodModel> ClothSimulationLodModels, const FReferenceSkeleton& ReferenceSkeleton, const TArray<int32>& UsedBoneIndices)
	{
		// First build a list per used bone for it's path to root
		TArray<int32> WeightedBones;  // List of actually weighted (not just used) bones

		for (const FChaosClothSimulationLodModel& ClothSimulationLodModel : ClothSimulationLodModels)
		{
			for (const FClothVertBoneData& VertBoneData : ClothSimulationLodModel.BoneData)
			{
				for (int32 InfluenceIndex = 0; InfluenceIndex < VertBoneData.NumInfluences; ++InfluenceIndex)
				{
					if (VertBoneData.BoneWeights[InfluenceIndex] > SMALL_NUMBER)
					{
						const int32 UnmappedBoneIndex = VertBoneData.BoneIndices[InfluenceIndex];
						check(UsedBoneIndices.IsValidIndex(UnmappedBoneIndex));
						WeightedBones.AddUnique(UsedBoneIndices[UnmappedBoneIndex]);
					}
					else
					{
						// Hit the last weight (they're sorted)
						break;
					}
				}
			}
		}

		return FClothEngineTools::CalculateReferenceBoneIndex(WeightedBones, ReferenceSkeleton);
	}

	FMD5Hash CalculateTransitionDataHash(const FChaosClothSimulationLodModel& CurrModel)
	{
		FMD5 MD5;
		const TArray<FVector3f>& Positions = CurrModel.Positions;
		const int32 NumPositions = Positions.Num();
		MD5.Update(reinterpret_cast<const uint8*>(&NumPositions), sizeof(NumPositions));
		MD5.Update(reinterpret_cast<const uint8*>(Positions.GetData()),
			NumPositions * sizeof(FVector3f));

		const TArray<uint32>& Indices = CurrModel.Indices;
		const int32 NumIndices = Indices.Num();
		MD5.Update(reinterpret_cast<const uint8*>(&NumIndices), sizeof(NumIndices));
		MD5.Update(reinterpret_cast<const uint8*>(Indices.GetData()), NumIndices * sizeof(uint32));

		FMD5Hash MD5Hash;
		MD5Hash.Set(MD5);

		return MD5Hash;
	}
}  // End namespace UE::Chaos::ClothAsset

bool FChaosClothSimulationLodModel::Serialize(FArchive& Ar)
{
	using namespace UE::Chaos::ClothAsset;

	// Serialize normal tagged property data
	if (Ar.IsLoading() || Ar.IsSaving() || Ar.IsCountingMemory())
	{
		UScriptStruct* const Struct = FChaosClothSimulationLodModel::StaticStruct();
		Struct->SerializeTaggedProperties(Ar, (uint8*)this, Struct, nullptr);
	}

	bool bCooked = Ar.IsCooking();
	Ar << bCooked;

#if WITH_EDITORONLY_DATA
	if (bCooked && Ar.IsSaving())
	{
		const FString AssetPath = FClothEngineTools::GetCookedAssetPath(Ar);

		TArray<FName> WeightMapsToRemove;
		for (TPair<FName, TArray<float>>& WeightMap : WeightMaps)
		{
			if (WeightMap.Key.ToString().StartsWith(TEXT("_")))
			{
				WeightMapsToRemove.Add(WeightMap.Key);
			}
		}
		for (const FName Key : WeightMapsToRemove)
		{
			TArray<float>& Value = WeightMaps.FindChecked(Key);
			Value.Shrink();
			const uint64 Size = (uint64)Value.GetAllocatedSize();
			WeightMaps.Remove(Key);
			UE_LOGF(LogChaosClothAsset, Display, "TrimOnCook %ls WeightMap[%ls] [%llu -> 0, saved %llu bytes]",
				*AssetPath, *Key.ToString(), Size, Size);
		}

		TArray<FName> VertexSetsToRemove;
		for (TPair<FName, TSet<int32>>& VertexSet : VertexSets)
		{
			if (VertexSet.Key.ToString().StartsWith(TEXT("_")))
			{
				VertexSetsToRemove.Add(VertexSet.Key);
			}
		}
		for (const FName Key : VertexSetsToRemove)
		{
			TSet<int32>& Value = VertexSets.FindChecked(Key);
			Value.Shrink();
			const uint64 Size = (uint64)Value.GetAllocatedSize();
			VertexSets.Remove(Key);
			UE_LOGF(LogChaosClothAsset, Display, "TrimOnCook %ls VertexSet[%ls] [%llu -> 0, saved %llu bytes]",
				*AssetPath, *Key.ToString(), Size, Size);
		}

		TArray<FName> FaceSetsToRemove;
		for (TPair<FName, TSet<int32>>& FaceSet : FaceSets)
		{
			if (FaceSet.Key.ToString().StartsWith(TEXT("_")))
			{
				FaceSetsToRemove.Add(FaceSet.Key);
			}
		}
		for (const FName Key : FaceSetsToRemove)
		{
			TSet<int32>& Value = FaceSets.FindChecked(Key);
			Value.Shrink();
			const uint64 Size = (uint64)Value.GetAllocatedSize();
			FaceSets.Remove(Key);
			UE_LOGF(LogChaosClothAsset, Display, "TrimOnCook %ls FaceSet[%ls] [%llu -> 0, saved %llu bytes]",
				*AssetPath, *Key.ToString(), Size, Size);
		}
	}
#endif

	// Serialize weight maps (not a tagged property)
	Ar << WeightMaps;

	Ar << LODTransitionUpData;
	Ar << LODTransitionDownData;

	Ar << VertexSets;
	Ar << FaceIntMaps;
	Ar << FaceSets;


	// Return true to confirm that serialization has already been taken care of
	return true;
}

void FChaosClothSimulationLodModel::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	UE::Chaos::ClothAsset::Private::FMemoryCounterArchive Ar;
	Serialize(Ar);
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(Ar.TotalSize());
}

int32 FChaosClothSimulationLodModel::CalcMaxBoneInfluences() const
{
	int32 MaxBoneInfluences = 0;
	for (const FClothVertBoneData& ClothVertBoneData : BoneData)
	{
		MaxBoneInfluences = FMath::Max(MaxBoneInfluences, ClothVertBoneData.NumInfluences);
	}
	return MaxBoneInfluences;
}

FChaosClothSimulationModel::FChaosClothSimulationModel(const TArray<TSharedRef<const FManagedArrayCollection>>& ClothCollections, 
	const FReferenceSkeleton& ReferenceSkeleton,
	TArray<FChaosClothAssetLodTransitionDataCache>* InOutTransitionCache)
{
	using namespace UE::Chaos::ClothAsset;

	// Initialize LOD models
	const int32 NumLods = ClothCollections.Num();
	ClothSimulationLodModels.SetNum(NumLods);

	ReferenceBoneIndex = INDEX_NONE;
	for (int32 LodIndex = 0; LodIndex < NumLods; ++LodIndex)
	{
		const FCollectionClothConstFacade Cloth(ClothCollections[LodIndex]);

		if (LodIndex == 0)
		{
			// Get reference bone
			const FName ReferenceBoneName = Cloth.GetReferenceBoneName();

			ReferenceBoneIndex = ReferenceSkeleton.FindRawBoneIndex(ReferenceBoneName);
		}

		// Retrieve weigh map names
		const TArray<FName> WeightMapNames = Cloth.GetWeightMapNames();

		FChaosClothSimulationLodModel& LodModel = ClothSimulationLodModels[LodIndex];

		TArray<TArray<int32>> WeldedToPatternIndices;
		Cloth.BuildSimulationMesh(LodModel.Positions, LodModel.Normals, LodModel.Indices, LodModel.PatternPositions, LodModel.PatternIndices, LodModel.PatternToWeldedIndices, &WeldedToPatternIndices);

		const ::Chaos::Softs::FCollectionPropertyConstFacade Properties(ClothCollections[LodIndex]);
		const bool bCopyAllUserAttributes = Cloth.GetCopyAllUserAttributesToSimModel();

		// Copy weight maps
		LodModel.WeightMaps.Reserve(WeightMapNames.Num());
		for (const FName& WeightMapName : WeightMapNames)
		{
			// Keep all weight maps to allow runtime switching. This is legacy behavior even regardless of value of bCopyAllUserAttributes
			LodModel.WeightMaps.Add(WeightMapName) = Cloth.GetWeightMap(WeightMapName);
		}

		auto IsNameUsedInProperties = [&Properties](const FName& WeightMapName)->bool
			{
				for (int32 PropertyIndex = 0; PropertyIndex < Properties.Num(); ++PropertyIndex)
				{
					if (Properties.GetStringValue(PropertyIndex) == WeightMapName)
					{
						return true;
					}
				}
				return false;
			};

		auto IsNameReserved = [&Properties](const FName& WeightMapName)->bool
			{
				return WeightMapName.ToString().StartsWith(TEXT("_"));
			};

		// Copy vertex and face sets
		FCollectionClothSelectionConstFacade Selection(ClothCollections[LodIndex]);
		const TArray<FName> SelectionNames = Selection.GetNames();
		LodModel.VertexSets.Reserve(SelectionNames.Num());
		LodModel.FaceSets.Reserve(SelectionNames.Num());
		for (const FName& SelectionName : SelectionNames)
		{
			if (bCopyAllUserAttributes || IsNameUsedInProperties(SelectionName) || IsNameReserved(SelectionName))
			{
				const FName SelectionGroup = Selection.GetSelectionGroup(SelectionName);
				if (SelectionGroup == ClothCollectionGroup::SimVertices3D)
				{
					LodModel.VertexSets.Add(SelectionName) = Selection.GetSelectionSet(SelectionName);
				}
				else if (SelectionGroup == ClothCollectionGroup::SimFaces)
				{
					LodModel.FaceSets.Add(SelectionName) = Selection.GetSelectionSet(SelectionName);
				}
			}
		}

		// Copy face int maps
		const TArray<FName> FaceIntMapNames = Cloth.GetUserDefinedAttributeNames<int32>(ClothCollectionGroup::SimFaces);
		LodModel.FaceIntMaps.Reserve(FaceIntMapNames.Num());
		for (const FName& FaceIntMapName : FaceIntMapNames)
		{
			if (bCopyAllUserAttributes || IsNameUsedInProperties(FaceIntMapName) || IsNameReserved(FaceIntMapName))
			{
				LodModel.FaceIntMaps.Add(FaceIntMapName) = Cloth.GetUserDefinedAttribute<int32>(FaceIntMapName, ClothCollectionGroup::SimFaces);
			}
		}

		// Copy bone influences (and track all used sim bones) and gather tether data.
		const int32 NumSimVertices3D = Cloth.GetNumSimVertices3D();
		TConstArrayView<TArray<int32>> SimBoneIndices = Cloth.GetSimBoneIndices();
		TConstArrayView<TArray<float>> SimBoneWeights = Cloth.GetSimBoneWeights();
		TConstArrayView<TArray<int32>> TetherKinematicIndex = Cloth.GetTetherKinematicIndex();
		TConstArrayView<TArray<float>> TetherReferenceLength = Cloth.GetTetherReferenceLength();
		TArray<TArray<TPair<float, int32>>> MergedTetherData;
		LodModel.BoneData.SetNum(NumSimVertices3D);
		MergedTetherData.SetNum(NumSimVertices3D);
		TSet<FBoneIndexType> RequiredExtraSimBones;
		RequiredExtraSimBones.Reserve(ReferenceSkeleton.GetRawBoneNum());
		if (ReferenceBoneIndex != INDEX_NONE)
		{
			RequiredExtraSimBones.Add(ReferenceBoneIndex);
		}

		for (int32 VertexIndex = 0; VertexIndex < NumSimVertices3D; ++VertexIndex)
		{
			check(SimBoneIndices[VertexIndex].Num() == SimBoneWeights[VertexIndex].Num());
			check(SimBoneIndices[VertexIndex].Num() <= FClothVertBoneData::MaxTotalInfluences);
			LodModel.BoneData[VertexIndex].NumInfluences = SimBoneIndices[VertexIndex].Num();
			for (int32 BoneIndex = 0; BoneIndex < LodModel.BoneData[VertexIndex].NumInfluences; ++BoneIndex)
			{
				LodModel.BoneData[VertexIndex].BoneIndices[BoneIndex] = SimBoneIndices[VertexIndex][BoneIndex];
				LodModel.BoneData[VertexIndex].BoneWeights[BoneIndex] = SimBoneWeights[VertexIndex][BoneIndex];
				RequiredExtraSimBones.Add(LodModel.BoneData[VertexIndex].BoneIndices[BoneIndex]);
			}

			check(TetherKinematicIndex[VertexIndex].Num() == TetherReferenceLength[VertexIndex].Num());
			MergedTetherData[VertexIndex].Reserve(TetherKinematicIndex[VertexIndex].Num());
			for (int32 TetherIdx = 0; TetherIdx < TetherKinematicIndex[VertexIndex].Num(); ++TetherIdx)
			{
				MergedTetherData[VertexIndex].Emplace(TetherReferenceLength[VertexIndex][TetherIdx], TetherKinematicIndex[VertexIndex][TetherIdx]);
			}
		}

		// Add any accessory mesh required bones
		for (int32 AccessoryMeshIndex = 0; AccessoryMeshIndex < Cloth.GetNumSimAccessoryMeshes(); ++AccessoryMeshIndex)
		{
			FCollectionClothSimAccessoryMeshConstFacade AccessoryMesh = Cloth.GetSimAccessoryMesh(AccessoryMeshIndex);

			TConstArrayView<TArray<int32>> AccessoryBoneIndices = AccessoryMesh.GetSimAccessoryMeshBoneIndices();
			for (const TArray<int32>& BoneIndices : AccessoryBoneIndices)
			{
				for (const int32 BoneIndex : BoneIndices)
				{
					if (BoneIndex != INDEX_NONE)
					{
						RequiredExtraSimBones.Add(BoneIndex);
					}
				}
			}
		}

		// Batch tethers
		LodModel.TetherData.GenerateTethers(MoveTemp(MergedTetherData));

		// Find all of the used render bones. We need to store the used sim bones that aren't used for rendering.
		TSet<FBoneIndexType> UsedRenderBones;
		TConstArrayView<TArray<int32>> RenderBoneIndices = Cloth.GetRenderBoneIndices();
		for (const TArray<int32>& Indices : RenderBoneIndices)
		{
			for (const int32 Index : Indices)
			{
				UsedRenderBones.Add((FBoneIndexType)Index);
			}
		}

		LodModel.RequiredExtraBoneIndices = RequiredExtraSimBones.Difference(UsedRenderBones).Array();
	}

	// Populate used bone names and indices
	for (int32 Index = 0; Index < ReferenceSkeleton.GetRawBoneNum(); ++Index)
	{	
		UsedBoneNames.Add(ReferenceSkeleton.GetRawRefBoneInfo()[Index].Name);
		UsedBoneIndices.Add(Index);
	}

	// Initialize Reference bone index if not set in the cloth collection
	if (ReferenceBoneIndex == INDEX_NONE)
	{
		ReferenceBoneIndex = Private::CalculateReferenceBoneIndex(ClothSimulationLodModels, ReferenceSkeleton, UsedBoneIndices);
	}

	CalculateLODTransitionUpDownData(InOutTransitionCache);
}

TArray<TConstArrayView<TTuple<int32, int32, float>>> FChaosClothSimulationModel::GetTethers(int32 LODIndex) const
{
	TArray<TConstArrayView<TTuple<int32, int32, float>>> Tethers;
	if (IsValidLodIndex(LODIndex))
	{
		const FClothTetherData& ClothTetherData = ClothSimulationLodModels[LODIndex].TetherData;

		const int32 NumTetherBatches = ClothTetherData.Tethers.Num();
		Tethers.Reserve(NumTetherBatches);
		for (int32 Index = 0; Index < NumTetherBatches; ++Index)
		{
			Tethers.Emplace(TConstArrayView<TTuple<int32, int32, float>>(ClothTetherData.Tethers[Index]));
		}
	}
	return Tethers;
}

void FChaosClothSimulationModel::CalculateLODTransitionUpDownData(TArray<FChaosClothAssetLodTransitionDataCache>* InOutTransitionCache)
{
	// Parameters for GenerateMeshToMeshVertData
	const FPointWeightMap* const MaxDistances = nullptr;  // No need to update the vertex contribution on the transition maps
	constexpr bool bUseSmoothTransitions = false;         // Smooth transitions are only used at rendering for now and not during LOD transitions
	constexpr bool bUseMultipleInfluences = false;        // Multiple influences must not be used for LOD transitions
	constexpr float SkinningKernelRadius = 0.f;           // KernelRadius is only required when using multiple influences

	// Update cache hashes.
	TArray<bool> TransitionCacheHashValid;
	TransitionCacheHashValid.SetNumZeroed(ClothSimulationLodModels.Num());
	if (InOutTransitionCache)
	{
		InOutTransitionCache->SetNum(ClothSimulationLodModels.Num());

		for (int32 LODIndex = 0; LODIndex < ClothSimulationLodModels.Num(); ++LODIndex)
		{
			const FMD5Hash CurrCacheHash = UE::Chaos::ClothAsset::Private::CalculateTransitionDataHash(ClothSimulationLodModels[LODIndex]);
			TransitionCacheHashValid[LODIndex] = (CurrCacheHash == (*InOutTransitionCache)[LODIndex].ModelHash);
			(*InOutTransitionCache)[LODIndex].ModelHash = CurrCacheHash;
		}
	}

	for (int32 LODIndex = 0; LODIndex < ClothSimulationLodModels.Num(); ++LODIndex)
	{
		const int32 PrevLODIndex = LODIndex - 1;
		const int32 NextLODIndex = LODIndex + 1;
				
		FChaosClothSimulationLodModel& CurrModel = ClothSimulationLodModels[LODIndex];
		const ClothingMeshUtils::ClothMeshDesc CurrentMeshDesc(CurrModel.Positions, CurrModel.Indices);

		if (ClothSimulationLodModels.IsValidIndex(PrevLODIndex))
		{
			if (TransitionCacheHashValid[LODIndex] && TransitionCacheHashValid[PrevLODIndex])
			{
				CurrModel.LODTransitionUpData = (*InOutTransitionCache)[LODIndex].LODTransitionUpData;
			}
			else
			{
				const FChaosClothSimulationLodModel& PrevModel = ClothSimulationLodModels[PrevLODIndex];
				const ClothingMeshUtils::ClothMeshDesc PrevMeshDesc(PrevModel.Positions, PrevModel.Indices);

				ClothingMeshUtils::GenerateMeshToMeshVertData(CurrModel.LODTransitionUpData, CurrentMeshDesc, PrevMeshDesc,
					MaxDistances, bUseSmoothTransitions, bUseMultipleInfluences, SkinningKernelRadius);

				if (InOutTransitionCache)
				{
					(*InOutTransitionCache)[LODIndex].LODTransitionUpData = CurrModel.LODTransitionUpData;
				}
			}
		}

		if (ClothSimulationLodModels.IsValidIndex(NextLODIndex))
		{
			if (TransitionCacheHashValid[LODIndex] && TransitionCacheHashValid[NextLODIndex])
			{
				CurrModel.LODTransitionDownData = (*InOutTransitionCache)[LODIndex].LODTransitionDownData;
			}
			else
			{
				const FChaosClothSimulationLodModel& NextModel = ClothSimulationLodModels[NextLODIndex];
				const ClothingMeshUtils::ClothMeshDesc NextMeshDesc(NextModel.Positions, NextModel.Indices);

				ClothingMeshUtils::GenerateMeshToMeshVertData(CurrModel.LODTransitionDownData, CurrentMeshDesc, NextMeshDesc,
					MaxDistances, bUseSmoothTransitions, bUseMultipleInfluences, SkinningKernelRadius);

				if (InOutTransitionCache)
				{
					(*InOutTransitionCache)[LODIndex].LODTransitionDownData = CurrModel.LODTransitionDownData;
				}
			}
		}
	}
}

void FChaosClothSimulationModel::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	UE::Chaos::ClothAsset::Private::FMemoryCounterArchive Ar;

	UScriptStruct* const Struct = FChaosClothSimulationModel::StaticStruct();
	Struct->SerializeTaggedProperties(Ar, (uint8*)this, Struct, nullptr);

	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(Ar.TotalSize());
}

CHAOSCLOTHASSETENGINE_API void FChaosClothSimulationModel::RemapBoneIndices(const TArray<int32>& BoneMap)
{
	// Default to the root bone index whenever an invalid index is provided to avoid any issues further on
	constexpr int32 RootBoneIndex = 0;

	// Remap the reference bone if needed
	ReferenceBoneIndex = BoneMap.IsValidIndex(ReferenceBoneIndex) ? BoneMap[ReferenceBoneIndex] : RootBoneIndex;

	// UsedBoneIndices is used to reindex from the subset index to the actual skeleton bone index, so needs updating
	for (int32& UsedBoneIndex : UsedBoneIndices)
	{
		UsedBoneIndex = BoneMap.IsValidIndex(UsedBoneIndex) ? BoneMap[UsedBoneIndex] : RootBoneIndex;
	}
}
