// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ClothSimulationProxy.h"
#include "Algo/NoneOf.h"
#include "ChaosClothAsset/ClothAsset.h"
#include "ChaosClothAsset/ClothAssetPrivate.h"
#include "ChaosClothAsset/ClothComponent.h"
#include "ChaosClothAsset/ClothSimulationAccessoryMesh.h"
#include "ChaosClothAsset/ClothSimulationContext.h"
#include "ChaosClothAsset/ClothSimulationMesh.h"
#include "ChaosClothAsset/ClothSimulationModel.h"
#include "ChaosClothAsset/CollisionSources.h"
#include "ChaosCloth/ChaosClothingSimulationCloth.h"
#include "ChaosCloth/ChaosClothingSimulationCollider.h"
#include "ChaosCloth/ChaosClothingSimulationConfig.h"
#include "ChaosCloth/ChaosClothingSimulationSolver.h"
#include "ChaosCloth/ChaosClothVisualization.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Misc/ScopeExit.h"
#include "PhysicsEngine/PhysicsSettings.h"
#include "PhysicsField/PhysicsFieldComponent.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "ClothingSimulation.h"

#if INTEL_ISPC
#include "ClothSimulationProxy.ispc.generated.h"
#endif

DECLARE_CYCLE_STAT(TEXT("ClothSimulationProxy Tick Game"), STAT_ClothSimulationProxy_TickGame, STATGROUP_ChaosClothAsset);
DECLARE_CYCLE_STAT(TEXT("ClothSimulationProxy Tick Physics"), STAT_ClothSimulationProxy_TickPhysics, STATGROUP_ChaosClothAsset);
DECLARE_CYCLE_STAT(TEXT("ClothSimulationProxy Write Simulation Data"), STAT_ClothSimulationProxy_WriteSimulationData, STATGROUP_ChaosClothAsset);
DECLARE_CYCLE_STAT(TEXT("ClothSimulationProxy Calculate Bounds"), STAT_ClothSimulationProxy_CalculateBounds, STATGROUP_ChaosClothAsset);
DECLARE_CYCLE_STAT(TEXT("ClothSimulationProxy End Parallel Cloth Task"), STAT_ClothSimulationProxy_EndParallelClothTask, STATGROUP_ChaosClothAsset);

CSV_DECLARE_CATEGORY_MODULE_EXTERN(ENGINE_API, Animation);

namespace UE::Chaos::ClothAsset
{
#if INTEL_ISPC && !UE_BUILD_SHIPPING
	static_assert(sizeof(ispc::FVector3f) == sizeof(FVector3f), "sizeof(ispc::FVector3f) != sizeof(FVector3f)");
	static_assert(sizeof(ispc::FTransform) == sizeof(::Chaos::FRigidTransform3), "sizeof(ispc::FTransform) != sizeof(::Chaos::FRigidTransform3)");

	bool bTransformClothSimulData_ISPC_Enabled = CHAOS_TRANSFORM_CLOTH_SIMUL_DATA_ISPC_ENABLED_DEFAULT;
	FAutoConsoleVariableRef CVarTransformClothSimukDataISPCEnabled(TEXT("p.ChaosClothAsset.TransformClothSimulData.ISPC"), bTransformClothSimulData_ISPC_Enabled, TEXT("Whether to use ISPC optimizations when transforming simulation data back to reference bone space."));
#endif

	float DeltaTimeDecay = 0.03f;
	FAutoConsoleVariableRef CVarDeltaTimeDecay(TEXT("p.ChaosClothAsset.DeltaTimeDecay"), DeltaTimeDecay, TEXT("Delta Time smoothing decay (1 = no smoothing)"));

	bool bEnableAsyncClothInitialization = false;
	FAutoConsoleVariableRef CVarEnableAsyncClothInitialization(TEXT("p.ChaosClothAsset.EnableAsyncClothInitialization"), bEnableAsyncClothInitialization, TEXT("Enable asynchronous cloth proxy initialization"));
	bool bWaitForAsyncClothInitialization = true;
	FAutoConsoleVariableRef CVarWaitForAsyncClothInitialization(TEXT("p.ChaosClothAsset.WaitForAsyncClothInitialization"), bWaitForAsyncClothInitialization, TEXT("When asynchronous cloth proxy initialization is enabled, wait for initialization to complete to start up cloth simulation. Otherwise, cloth simulation will be disabled until initialization has completed."));

	bool bSimulationDataCheckFinite = true;
	FAutoConsoleVariableRef CVarSimulationDataCheckFinite(TEXT("p.ChaosClothAsset.SimulationDataCheckFinite"), bSimulationDataCheckFinite, TEXT("Check for nans/infs in cloth simulation data"));
	bool bSimulationDataResetOnNonFinite = true;
	FAutoConsoleVariableRef CVarSimulationDataResetOnNonFinite(TEXT("p.ChaosClothAsset.SimulationDataResetOnNonFinite"), bSimulationDataResetOnNonFinite, TEXT("Reset cloth simulation if a nan/inf is detected. Requires p.ChaosClothAsset.SimulationDataCheckFinite"));

	static FAutoConsoleTaskPriority CPrio_ClothSimulationProxyParallelTask(
		TEXT("TaskGraph.TaskPriorities.ClothSimulationProxyParallelTask"),
		TEXT("Task and thread priority for the cloth simulation proxy."),
		ENamedThreads::HighThreadPriority, // If we have high priority task threads, then use them...
		ENamedThreads::NormalTaskPriority, // .. at normal task priority
		ENamedThreads::HighTaskPriority);  // If we don't have high priority threads, then use normal priority threads at high task priority instead

	class FClothSimulationProxyParallelTask
	{
	public:
		enum struct EType : uint8
		{
			Tick = 0,
			Initialization
		};

		FClothSimulationProxyParallelTask(FClothSimulationProxy& InClothSimulationProxy, EType InType = EType::Tick)
			: ClothSimulationProxy(InClothSimulationProxy)
			, Type(InType)
		{
		}

		TStatId GetStatId() const
		{
			RETURN_QUICK_DECLARE_CYCLE_STAT(FClothSimulationProxyParallelTask, STATGROUP_TaskGraphTasks);
		}

		static ENamedThreads::Type GetDesiredThread()
		{
			static IConsoleVariable* const CVarClothPhysicsUseTaskThread = IConsoleManager::Get().FindConsoleVariable(TEXT("p.ClothPhysics.UseTaskThread"));

			if (CVarClothPhysicsUseTaskThread && CVarClothPhysicsUseTaskThread->GetBool())
			{
				return CPrio_ClothSimulationProxyParallelTask.Get();
			}
			return ENamedThreads::GameThread;
		}

		static ESubsequentsMode::Type GetSubsequentsMode()
		{
			return ESubsequentsMode::TrackSubsequents;
		}

		void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
		{
			FScopeCycleCounterUObject ContextScope(ClothSimulationProxy.ClothComponentAdapter.GetOwnerComponent().GetSkinnedAsset());
			SCOPE_CYCLE_COUNTER(STAT_ClothTotalTime);
			CSV_SCOPED_TIMING_STAT(Animation, Cloth);
			switch (Type)
			{
			case EType::Initialization:
				ClothSimulationProxy.ExecuteInitialization();
				break;
			case EType::Tick:
			default:
				ClothSimulationProxy.Tick();
				break;
			}
		}

	private:
		FClothSimulationProxy& ClothSimulationProxy;
		EType Type = EType::Tick;
	};

	FClothSimulationProxy::FClothSimulationProxy(const IClothComponentAdapter& ClothComponentAdapter)
		: ClothComponentAdapter(ClothComponentAdapter)
		, ClothSimulationContext(MakeUnique<FClothSimulationContext>())
		, CollisionSourcesProxy(MakeUnique<FCollisionSourcesProxy>(ClothComponentAdapter.GetCollisionSources()))
		, Solver(nullptr)
		, Visualization(nullptr)
		, MaxDeltaTime(UPhysicsSettings::Get()->MaxPhysicsDeltaTime)
	{
	}

	FClothSimulationProxy::~FClothSimulationProxy()
	{
		WaitForParallelInitialization_GameThread();
		CompleteParallelSimulation_GameThread();
	}

	void FClothSimulationProxy::PostConstructor()
	{
		PostConstructorInternal(bEnableAsyncClothInitialization);
	}

	void FClothSimulationProxy::PostConstructorInternal(bool bAsyncInitialization)
	{
		BeginInitialization_GameThread();
		if (bAsyncInitialization)
		{
			ParallelInitializationTask = TGraphTask<FClothSimulationProxyParallelTask>::CreateTask(nullptr, ENamedThreads::GameThread).ConstructAndDispatchWhenReady(*this, FClothSimulationProxyParallelTask::EType::Initialization);
			// Note: Complete Initialization will be handled when the parallel task completes.
		}
		else
		{
			ExecuteInitialization();
			CompleteInitialization_GameThread();
		}
	}

	void FClothSimulationProxy::BeginInitialization_GameThread()
	{
		LLM_SCOPE_BYNAME(TEXT("Physics/Cloth"));
		using namespace ::Chaos;

		check(IsInGameThread());

		// Reset all simulation arrays
		Configs.Reset();
		Meshes.Reset();
		Colliders.Reset();
		Cloths.Reset();
		ClothSimulationModels.Reset();

		// Create solver config simulation thread object first. Need to know which solver type we're creating.
		const int32 SolverConfigIndex = Configs.Emplace(MakeUnique<FClothingSimulationConfig>(ClothComponentAdapter.GetSolverPropertyCollections()));

		// Use new SoftsEvolution, not PBDEvolution.
		constexpr bool bUseLegacySolver = false;
		Solver = MakeUnique<::Chaos::FClothingSimulationSolver>(Configs[SolverConfigIndex].Get(), bUseLegacySolver);

		Visualization = MakeUnique<::Chaos::FClothVisualizationNoGC>(Solver.Get());

		// Need a valid context to initialize the mesh
		constexpr bool bIsInitialization = true;
		constexpr Softs::FSolverReal NoAdvanceDt = 0.f;
		ClothSimulationContext->Fill(ClothComponentAdapter, NoAdvanceDt, MaxDeltaTime, bIsInitialization);

		// Setup startup transforms
		constexpr bool bNeedsReset = true;
		const FReal LocalSpaceScale = 1. / FMath::Max(ClothSimulationContext->SolverGeometryScale, UE_SMALL_NUMBER);
		Solver->SetLocalSpaceScale(LocalSpaceScale, bNeedsReset);
		Solver->SetLocalSpaceLocation((FVec3)ClothSimulationContext->ComponentTransform.GetLocation(), bNeedsReset);
		Solver->SetLocalSpaceRotation((FQuat)ClothSimulationContext->ComponentTransform.GetRotation());

		// Create mesh simulation thread objects
		// HardResetSimulation_GameThread expects to find Cloths added in this order, so update it if you change Cloths initialization order here.
		for (const UChaosClothAssetBase* const Asset : ClothComponentAdapter.GetAssets())
		{
			const int32 NumClothSimulationModels = Asset ? Asset->GetNumClothSimulationModels() : 0;
			for (int32 ModelIndex = 0; ModelIndex < NumClothSimulationModels; ++ModelIndex)
			{
				const int32 SimulationGroupId = ClothComponentAdapter.GetSimulationGroupId(Asset, ModelIndex);
				if (SimulationGroupId != INDEX_NONE)  // INDEX_NONE means this model shouldn't be added to the simulation
				{
					checkf(Algo::NoneOf(Cloths, [&SimulationGroupId](const TUniquePtr<::Chaos::FClothingSimulationCloth>& Cloth)
						{
							return Cloth && Cloth->GetGroupId() == SimulationGroupId;
						}), TEXT("The ClothComponentAdapter cannot return the same SimulationGroupId for two different Asset/ModelIndex pairs (unless INDEX_NONE)."));

					const TSharedPtr<const FChaosClothSimulationModel> ClothSimulationModel = ClothComponentAdapter.GetActiveClothSimulationModel(Asset, ModelIndex);
					if (Algo::NoneOf(ClothSimulationModels, [&ClothSimulationModel](const TSharedPtr<const FChaosClothSimulationModel>& InClothSimulationModel)
						{
							return InClothSimulationModel.Get() == ClothSimulationModel.Get();
						}))
					{
						ClothSimulationModels.Emplace(ClothSimulationModel);  // Ownership has to transfer to this proxy when the model changes during the simulation
					}

					FString DebugName;
#if !UE_BUILD_SHIPPING
					DebugName = ClothComponentAdapter.GetOwnerComponent().GetOwner() ?
						FString::Format(TEXT("{0}|{1}|{2}|Model[{3}]"),
							{
								ClothComponentAdapter.GetOwnerComponent().GetOwner()->GetActorNameOrLabel(),
								ClothComponentAdapter.GetOwnerComponent().GetName(),
								Asset->GetName(),
								ModelIndex
							}) :
						FString::Format(TEXT("{0}|{1}|Model[{2}]"),
							{
								ClothComponentAdapter.GetOwnerComponent().GetName(),
								Asset->GetName(),
								ModelIndex
							});
#endif
					const int32 MeshIndex = Meshes.Emplace(MakeUnique<FClothSimulationMesh>(*ClothSimulationModel, *ClothSimulationContext, Asset->GetCollections(ModelIndex), DebugName));

					// Create collider simulation thread object
					const FReferenceSkeleton* const ContextReferenceSkeleton = ClothComponentAdapter.GetReferenceSkeleton();  // Match the reference skeleton the simulation context builds bone transforms with
					const FReferenceSkeleton* const ColliderReferenceSkeleton = ContextReferenceSkeleton ? ContextReferenceSkeleton : &Asset->GetRefSkeleton();
					const int32 ColliderIndex = Colliders.Emplace(MakeUnique<FClothingSimulationCollider>(Asset->GetPhysicsAssetForModel(ModelIndex), ColliderReferenceSkeleton));
					Colliders[ColliderIndex]->SetCollisionData(&CollisionSourcesProxy->GetCollisionData());

					// Create cloth config simulation thread object
					const int32 ClothConfigIndex = Configs.Emplace(MakeUnique<FClothingSimulationConfig>(ClothComponentAdapter.GetPropertyCollections(Asset, ModelIndex)));

					// Create cloth simulation thread object
					const int32 ClothIndex = Cloths.Emplace(MakeUnique<FClothingSimulationCloth>(
						Configs[ClothConfigIndex].Get(),
						Meshes[MeshIndex].Get(),
						ColliderIndex != INDEX_NONE ? TArray<FClothingSimulationCollider*>({ Colliders[ColliderIndex].Get() }) : TArray<FClothingSimulationCollider*>(),
						SimulationGroupId));
				}
			}
		}
	}

	void FClothSimulationProxy::ExecuteInitialization()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ClothSimulationProxy_ExecuteInitialization);
		LLM_SCOPE_BYNAME(TEXT("Physics/Cloth"));
		int32 LocalNumCloths = 0;
		int32 LocalNumKinematicParticles = 0;
		int32 LocalNumDynamicParticles = 0;
		for (TUniquePtr<::Chaos::FClothingSimulationCloth>& Cloth : Cloths)
		{
			Solver->AddCloth(Cloth.Get());
			Cloth->Reset();
			++LocalNumCloths;
			LocalNumKinematicParticles += Cloth->GetNumActiveKinematicParticles();
			LocalNumDynamicParticles += Cloth->GetNumActiveDynamicParticles();
		}

		// Update cloth stats
		NumCloths = LocalNumCloths;
		NumKinematicParticles = LocalNumKinematicParticles;
		NumDynamicParticles = LocalNumDynamicParticles;
	}

	void FClothSimulationProxy::WaitForParallelInitialization_GameThread()
	{
		check(IsInGameThread());
		using namespace ::Chaos;
		if (IsValidRef(ParallelInitializationTask))
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ClothSimulationProxy_CompleteInitialization);
			CSV_SCOPED_SET_WAIT_STAT(Cloth);

			FTaskGraphInterface::Get().WaitUntilTaskCompletes(ParallelInitializationTask, ENamedThreads::GameThread);

			// No longer need this task, it has completed
			ParallelInitializationTask.SafeRelease();
		}
	}

	void FClothSimulationProxy::CompleteInitialization_GameThread()
	{
		check(IsInGameThread());
		using namespace ::Chaos;

		WaitForParallelInitialization_GameThread();

		// Set start pose (update the context, then the solver without advancing the simulation)
		constexpr Softs::FSolverReal NoAdvanceDt = 0.f;
		ClothSimulationContext->Fill(ClothComponentAdapter, NoAdvanceDt, MaxDeltaTime);
		Solver->Update((Softs::FSolverReal)ClothSimulationContext->DeltaTime);
		bIsInitialized = true;
	}

	void FClothSimulationProxy::HardResetSimulation_GameThread()
	{
		// Reset solver, cloths, and configs, but keep existing meshes
		using namespace ::Chaos;

		check(IsInGameThread());

		// Need a valid context to initialize startup transforms
		constexpr bool bIsInitialization = true;
		constexpr Softs::FSolverReal NoAdvanceDt = 0.f;
		ClothSimulationContext->Fill(ClothComponentAdapter, NoAdvanceDt, MaxDeltaTime, bIsInitialization);

		// Setup startup transforms
		constexpr bool bNeedsReset = true;
		const FReal LocalSpaceScale = 1. / FMath::Max(ClothSimulationContext->SolverGeometryScale, UE_SMALL_NUMBER);
		Solver->SetLocalSpaceScale(LocalSpaceScale, bNeedsReset);
		Solver->SetLocalSpaceLocation((FVec3)ClothSimulationContext->ComponentTransform.GetLocation(), bNeedsReset);
		Solver->SetLocalSpaceRotation((FQuat)ClothSimulationContext->ComponentTransform.GetRotation());

		// Reset stats
		int32 LocalNumCloths = 0;
		int32 LocalNumKinematicParticles = 0;
		int32 LocalNumDynamicParticles = 0;

		// Clear all cloths from the solver
		Solver->RemoveCloths();

		// Recreate all cloths.
		// This loop matches the loop in BeginInitialization_GameThread and expects Cloths to have been added in that order (and for the assets to have not changed since last initialization).
		int32 ClothIndex = 0;
		for (const UChaosClothAssetBase* const Asset : ClothComponentAdapter.GetAssets())
		{
			const int32 NumClothSimulationModels = Asset ? Asset->GetNumClothSimulationModels() : 0;
			for (int32 ModelIndex = 0; ModelIndex < NumClothSimulationModels; ++ModelIndex)
			{
				const int32 SimulationGroupId = ClothComponentAdapter.GetSimulationGroupId(Asset, ModelIndex);
				if (SimulationGroupId != INDEX_NONE)  // INDEX_NONE means this model shouldn't be added to the simulation
				{
					check(Cloths.IsValidIndex(ClothIndex));

					TUniquePtr<FClothingSimulationCloth>& Cloth = Cloths[ClothIndex++];
					check(SimulationGroupId == Cloth->GetGroupId());

					FClothSimulationMesh* const Mesh = static_cast<FClothSimulationMesh*>(Cloth->GetMesh());
					TArray<FClothingSimulationCollider*> ClothColliders = Cloth->GetColliders();

					// Update cloth config runtime simulation object
					FClothingSimulationConfig* const Config = Cloth->GetConfig();
					Config->Initialize(ClothComponentAdapter.GetPropertyCollections(Asset, ModelIndex));

					// Recreate cloth runtime simulation object
					Cloth = MakeUnique<FClothingSimulationCloth>(
						Config,
						Mesh,
						MoveTemp(ClothColliders),
						SimulationGroupId);

					// Re-add cloth to the solver
					Solver->AddCloth(Cloth.Get());

					// Update stats
					++LocalNumCloths;
					LocalNumKinematicParticles += Cloth->GetNumActiveKinematicParticles();
					LocalNumDynamicParticles += Cloth->GetNumActiveDynamicParticles();
				}
			}
		}
		check(ClothIndex == Cloths.Num());

		// Update cloth stats
		NumCloths = LocalNumCloths;
		NumKinematicParticles = LocalNumKinematicParticles;
		NumDynamicParticles = LocalNumDynamicParticles;
	}

	void FClothSimulationProxy::PreProcess_GameThread(float DeltaTime, bool bForceWaitForInitialization)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FClothSimulationProxy_PreProcess_GameThread);

		// Set bIsSimulating to its default value, this will be changed in PreSimulate_GameThread if the simulation ever runs
		bIsSimulating = false;
		bIsPreProcessed = false;
		if (IsValidRef(ParallelInitializationTask))
		{
			if (!bForceWaitForInitialization && !bWaitForAsyncClothInitialization && !ParallelInitializationTask->IsComplete())
			{
				return;
			}
			CompleteInitialization_GameThread();
		}

		PreProcess_Internal(DeltaTime);
	}

	void FClothSimulationProxy::PreProcess_Internal(float DeltaTime)
	{
		check(bIsInitialized);

		bIsPreProcessed = true;

		if (!ClothComponentAdapter.IsSimulationEnabled())
		{
			return;
		}

		// Fill a new context, note the context is also needed when the simulation is suspended or playing back the cache
		constexpr bool bIsInitializationFalse = false;
		FillSimulationContext(DeltaTime, bIsInitializationFalse);

		// Check whether the solver should be enabled for caching purpose
		const bool bShouldEnableSolver = ShouldEnableSolver(Solver->GetEnableSolver());  // Note: needs to be called after filling the context
		Solver->SetEnableSolver(bShouldEnableSolver);

		// If a cache is provided, then read it now in the Tick function
		if (!bShouldEnableSolver && ClothSimulationContext->CacheData.HasData())
		{
			// Tick with the solver disabled to read the sim from the cache, the simulation won't be called
			Tick();
		}
	}

	bool FClothSimulationProxy::PreSimulate_GameThread(float DeltaTime)
	{
		if (!bIsPreProcessed)
		{
			return false;
		}

		// Check if it is playing the cache back
		const bool bIsSolverEnabled = Solver->GetEnableSolver();
		if (!bIsSolverEnabled && ClothSimulationContext->CacheData.HasData())
		{
			return false;
		}

		// Check that the there's any simulation data for this LOD
		if (!ClothComponentAdapter.HasAnySimulationMeshData(ClothSimulationContext->LodIndex))
		{
			return false;
		}

		// Check whether some actual simulation needs to happen
		bIsSimulating = (DeltaTime > 0.f && !ClothComponentAdapter.IsSimulationSuspended() && bIsSolverEnabled);
		if (!bIsSimulating)
		{
			return false;
		}

		// Update the config properties
		InitializeConfigs();

		// Update world forces
		if (UWorld* const World = ClothComponentAdapter.GetOwnerComponent().GetWorld())
		{			
			const bool bUsesBuoyancy = Solver->UsesBuoyancy();

			FBox BoundingBox;
			if (World->PhysicsField || bUsesBuoyancy)
			{
				BoundingBox = CalculateBounds_AnyThread().GetBox().TransformBy(ClothComponentAdapter.GetOwnerComponent().GetComponentTransform());
			}

			if (UPhysicsFieldComponent* const PhysicsField = World->PhysicsField)
			{
				PhysicsField->FillTransientCommands(false, BoundingBox, Solver->GetTime(), Solver->GetPerSolverField().GetTransientCommands());
				PhysicsField->FillPersistentCommands(false, BoundingBox, Solver->GetTime(), Solver->GetPerSolverField().GetPersistentCommands());
			}
											
			// broadphase against water bodies			
			if (bUsesBuoyancy)
			{				
				// find all buoyancy fields and compute collisions				
				Solver->PreSimulateBuoyancyField_GameThread(DeltaTime, World, BoundingBox);
			}
		}

		// Update external collision sources
		CollisionSourcesProxy->ExtractCollisionData();

		return bIsSimulating;
	}

	void FClothSimulationProxy::PostSimulate_GameThread(TMap<int32, FClothSimulData>* ExternalClothSimulData)
	{
		if (bIsSimulating)
		{
			WriteSimulationData(ExternalClothSimulData);

			// Clean properties dirty flags to avoid re-updating the constraints at the next tick when properties have not changed
			// Note this does not clear the source collection whose dirty flags are cleared after the copy in InitializeConfigs()
			for (TUniquePtr<::Chaos::FClothingSimulationConfig>& Config : Configs)
			{
				const int32 LODIndex = Solver->GetSolverLOD();
				if (LODIndex < Config->GetNumLODs())
				{
					Config->GetProperties(LODIndex).ClearDirtyFlags();
				}
			}
		}
	}

	void FClothSimulationProxy::PostProcess_GameThread(TMap<int32, FClothSimulData>* ExternalClothSimulData)
	{
		if (!bIsPreProcessed)
		{
			CurrentSimulationData.Reset();
			return;
		}
		if (!bIsSimulating)
		{
			// If the simulation is enabled, then it is suspended or reading from cache and the simulation data still needs updating (transforms and LODs)
			if (ClothComponentAdapter.IsSimulationEnabled())
			{
				// Take care of the LOD switching, as the simulation won't do it
				UpdateClothLODs();
				WriteSimulationData(ExternalClothSimulData);
			}
			else
			{
				CurrentSimulationData.Reset();
			}
		}
	}

	void FClothSimulationProxy::ForcePendingReset_GameThread()
	{
		// Check for any in-flight work.
		if (IsValidRef(ParallelInitializationTask))
		{
			if (!bWaitForAsyncClothInitialization && !ParallelInitializationTask->IsComplete())
			{
				return;
			}
			CompleteInitialization_GameThread();
		}

		check(bIsInitialized);

		CompleteParallelSimulation_GameThread();

		using namespace ::Chaos;
		for (const TUniquePtr<FClothingSimulationCloth>& Cloth : Cloths)
		{
			Cloth->Reset();
		}
	}

	bool FClothSimulationProxy::Tick_GameThread(float DeltaTime)
	{
		SCOPE_CYCLE_COUNTER(STAT_ClothSimulationProxy_TickGame);

		PreProcess_GameThread(DeltaTime);
		PreSimulate_GameThread(DeltaTime);

		if (bIsSimulating)
		{
			ParallelTask = TGraphTask<FClothSimulationProxyParallelTask>::CreateTask(nullptr, ENamedThreads::GameThread).ConstructAndDispatchWhenReady(*this);

			return true;  // Note: Post simulate and post process will be handled when the parallel task completes
		}

		PostSimulate_GameThread();
		PostProcess_GameThread();

		return bIsSimulating;
	}

	void FClothSimulationProxy::Tick()
	{
		using namespace ::Chaos;

		if (!bIsPreProcessed)
		{
			return;
		}

		TRACE_CPUPROFILER_EVENT_SCOPE(FClothSimulationProxy_TickPhysics);
		SCOPE_CYCLE_COUNTER(STAT_ClothSimulationProxy_TickPhysics);
		const bool bUseCache = ClothSimulationContext->CacheData.HasData();

		if (ClothSimulationContext->DeltaTime == 0.f && !bUseCache)
		{
			return;
		}
		// Filter delta time to smoothen time variations and prevent unwanted vibrations
		static IConsoleVariable* const UseTimeStepSmoothingCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("p.ChaosCloth.UseTimeStepSmoothing"));
		const bool bUseTimeStepSmoothing = UseTimeStepSmoothingCVar ? UseTimeStepSmoothingCVar->GetBool() : true;
		const Softs::FSolverReal DeltaTime = (Softs::FSolverReal)ClothSimulationContext->DeltaTime;
		const Softs::FSolverReal PrevDeltaTime = Solver->GetDeltaTime() > 0.f ? Solver->GetDeltaTime() : DeltaTime;
		const Softs::FSolverReal SmoothedDeltaTime = PrevDeltaTime + (DeltaTime - PrevDeltaTime) * (bUseTimeStepSmoothing ? (Softs::FSolverReal)DeltaTimeDecay : 1.f);

		const double StartTime = FPlatformTime::Seconds();
		const float PrevSimulationTime = SimulationTime;  // Copy the atomic to prevent a re-read

		const bool bNeedsReset = (bNeedResetClothFromNanCheck || ClothSimulationContext->bReset || PrevSimulationTime == 0.f);  // Reset on the first frame too since the simulation is created in bind pose, and not in start pose
		const bool bNeedsTeleport = ClothSimulationContext->bTeleport;
		bIsTeleported = bNeedsTeleport;

		// Update Solver animatable parameters
		Solver->SetLocalSpaceLocation((FVec3)ClothSimulationContext->ComponentTransform.GetLocation(), bNeedsReset);
		Solver->SetLocalSpaceRotation((FQuat)ClothSimulationContext->ComponentTransform.GetRotation());
		Solver->SetWindVelocity(ClothSimulationContext->WindVelocity);
		Solver->SetGravity(ClothSimulationContext->WorldGravity);
		Solver->EnableClothGravityOverride(true);
		Solver->SetVelocityScale(!bNeedsReset ? (FReal)ClothSimulationContext->VelocityScale * (FReal)SmoothedDeltaTime / DeltaTime : 1.f);

		// Check teleport modes
		for (const TUniquePtr<FClothingSimulationCloth>& Cloth : Cloths)
		{
			// Update Cloth animatable parameters while in the cloth loop
			if (bNeedsReset)
			{
				Cloth->Reset();
			}
			if (bNeedsTeleport)
			{
				Cloth->Teleport();
			}
			if (ClothSimulationContext->bResetRestLengthsWithMorphTarget)
			{
				Cloth->ResetRestLengthsWithMorphTarget(ClothSimulationContext->ResetRestLengthsMorphTargetName);
			}
		}

		// Step the simulation
		if (Solver->GetEnableSolver() || !bUseCache)
		{
			Solver->Update(SmoothedDeltaTime);
		}
		else
		{
			Solver->UpdateFromCache(ClothSimulationContext->CacheData);
		}

		// Keep the actual used number of iterations for the stats
		NumIterations = Solver->GetNumUsedIterations();
		NumSubsteps = Solver->GetNumUsedSubsteps();

		// Update simulation time in ms (and provide an instant average instead of the value in real-time)
		const float CurrSimulationTime = (float)((FPlatformTime::Seconds() - StartTime) * 1000.);
		static const float SimulationTimeDecay = 0.03f; // 0.03 seems to provide a good rate of update for the instant average
		SimulationTime = PrevSimulationTime ? PrevSimulationTime + (CurrSimulationTime - PrevSimulationTime) * SimulationTimeDecay : CurrSimulationTime;

		// Update particle counts (could have changed if lod changed)
		NumKinematicParticles = 0;
		NumDynamicParticles = 0;
		int32 FirstActiveClothParticleRangeId = INDEX_NONE;
		for (const TUniquePtr<FClothingSimulationCloth>& Cloth : Cloths)
		{
			NumKinematicParticles += Cloth->GetNumActiveKinematicParticles();
			NumDynamicParticles += Cloth->GetNumActiveDynamicParticles();
			if (FirstActiveClothParticleRangeId == INDEX_NONE && Cloth->GetNumActiveDynamicParticles() > 0)
			{
				FirstActiveClothParticleRangeId = Cloth->GetParticleRangeId(Solver.Get());
			}
		}
		if (FirstActiveClothParticleRangeId != INDEX_NONE)
		{
			LastLinearSolveError = Solver->GetLinearSolverError(FirstActiveClothParticleRangeId);
			LastLinearSolveIterations = Solver->GetNumLinearSolverIterations(FirstActiveClothParticleRangeId);
		}
		else
		{
			LastLinearSolveError = 0.f;
			LastLinearSolveIterations = 0;
		}

		bNeedResetClothFromNanCheck = false;

		// Visualization
#if CHAOS_DEBUG_DRAW
		const UWorld* const World = ClothComponentAdapter.GetOwnerComponent().GetWorld();
		if (World && World->WorldType != EWorldType::EditorPreview)  // Avoid drawing any editor preview debug draw inside the PIE window
		{
			static const TConsoleVariableData<int32>* const DebugDrawSingleVertexCVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("p.ChaosCloth.DebugDrawSingleVertex"));
			static const TConsoleVariableData<FString>* const DebugDrawFilterSetCVar = IConsoleManager::Get().FindTConsoleVariableDataString(TEXT("p.ChaosCloth.DebugDrawFilterSet"));
			static const TConsoleVariableData<bool>* const DebugDrawLocalSpaceCVar = IConsoleManager::Get().FindTConsoleVariableDataBool(TEXT("p.ChaosCloth.DebugDrawLocalSpace"));
			static const TConsoleVariableData<bool>* const DebugDrawBoundsCVar = IConsoleManager::Get().FindTConsoleVariableDataBool(TEXT("p.ChaosCloth.DebugDrawBounds"));
			static const TConsoleVariableData<bool>* const DebugDrawGravityCVar = IConsoleManager::Get().FindTConsoleVariableDataBool(TEXT("p.ChaosCloth.DebugDrawGravity"));
			static const TConsoleVariableData<bool>* const DebugDrawPhysMeshWiredCVar = IConsoleManager::Get().FindTConsoleVariableDataBool(TEXT("p.ChaosCloth.DebugDrawPhysMeshWired"));
			static const TConsoleVariableData<bool>* const DebugDrawAnimMeshWiredCVar = IConsoleManager::Get().FindTConsoleVariableDataBool(TEXT("p.ChaosCloth.DebugDrawAnimMeshWired"));
			static const TConsoleVariableData<bool>* const DebugDrawPointVelocitiesCVar = IConsoleManager::Get().FindTConsoleVariableDataBool(TEXT("p.ChaosCloth.DebugDrawPointVelocities"));
			static const TConsoleVariableData<bool>* const DebugDrawAnimNormalsCVar = IConsoleManager::Get().FindTConsoleVariableDataBool(TEXT("p.ChaosCloth.DebugDrawAnimNormals"));
			static const TConsoleVariableData<bool>* const DebugDrawPointNormalsCVar = IConsoleManager::Get().FindTConsoleVariableDataBool(TEXT("p.ChaosCloth.DebugDrawPointNormals"));
			static const TConsoleVariableData<bool>* const DebugDrawCollisionCVar = IConsoleManager::Get().FindTConsoleVariableDataBool(TEXT("p.ChaosCloth.DebugDrawCollision"));
			static const TConsoleVariableData<bool>* const DebugDrawBackstopsCVar = IConsoleManager::Get().FindTConsoleVariableDataBool(TEXT("p.ChaosCloth.DebugDrawBackstops"));
			static const TConsoleVariableData<bool>* const DebugDrawBackstopDistancesCVar = IConsoleManager::Get().FindTConsoleVariableDataBool(TEXT("p.ChaosCloth.DebugDrawBackstopDistances"));
			static const TConsoleVariableData<bool>* const DebugDrawMaxDistancesCVar = IConsoleManager::Get().FindTConsoleVariableDataBool(TEXT("p.ChaosCloth.DebugDrawMaxDistances"));
			static const TConsoleVariableData<bool>* const DebugDrawMaxDistanceValuesCVar = IConsoleManager::Get().FindTConsoleVariableDataBool(TEXT("p.ChaosCloth.DebugDrawMaxDistanceValues"));
			static const TConsoleVariableData<bool>* const DebugDrawAnimDriveCVar = IConsoleManager::Get().FindTConsoleVariableDataBool(TEXT("p.ChaosCloth.DebugDrawAnimDrive"));
			static const TConsoleVariableData<bool>* const DebugDrawEdgeConstraintCVar = IConsoleManager::Get().FindTConsoleVariableDataBool(TEXT("p.ChaosCloth.DebugDrawEdgeConstraint"));
			static const TConsoleVariableData<bool>* const DebugDrawBendingConstraintCVar = IConsoleManager::Get().FindTConsoleVariableDataBool(TEXT("p.ChaosCloth.DebugDrawBendingConstraint"));
			static const TConsoleVariableData<bool>* const DebugDrawLongRangeConstraintCVar = IConsoleManager::Get().FindTConsoleVariableDataBool(TEXT("p.ChaosCloth.DebugDrawLongRangeConstraint"));
			static const TConsoleVariableData<bool>* const DebugDrawWindForcesCVar = IConsoleManager::Get().FindTConsoleVariableDataBool(TEXT("p.ChaosCloth.DebugDrawWindForces"));
			static const TConsoleVariableData<bool>* const DebugDrawWindVelocityCVar = IConsoleManager::Get().FindTConsoleVariableDataBool(TEXT("p.ChaosCloth.DebugDrawWindVelocity"));
			static const TConsoleVariableData<bool>* const DebugDrawSelfCollisionCVar = IConsoleManager::Get().FindTConsoleVariableDataBool(TEXT("p.ChaosCloth.DebugDrawSelfCollision"));
			static const TConsoleVariableData<bool>* const DebugDrawSelfIntersectionCVar = IConsoleManager::Get().FindTConsoleVariableDataBool(TEXT("p.ChaosCloth.DebugDrawSelfIntersection"));
			static const TConsoleVariableData<bool>* const DebugDrawSelfCollisionThicknessCVar = IConsoleManager::Get().FindTConsoleVariableDataBool(TEXT("p.ChaosCloth.DebugDrawSelfCollisionThickness"));
			static const TConsoleVariableData<bool>* const DebugDrawClothCollisionThicknessCVar = IConsoleManager::Get().FindTConsoleVariableDataBool(TEXT("p.ChaosCloth.DebugDrawClothCollisionThickness"));
			static const TConsoleVariableData<bool>* const DebugDrawInnerCollisionThicknessCVar = IConsoleManager::Get().FindTConsoleVariableDataBool(TEXT("p.ChaosCloth.DebugDrawInnerCollisionThickness"));
			static const TConsoleVariableData<bool>* const DebugDrawParticleIndicesCVar = IConsoleManager::Get().FindTConsoleVariableDataBool(TEXT("p.ChaosCloth.DebugDrawParticleIndices"));
			static const TConsoleVariableData<bool>* const DebugDrawElementIndicesCVar = IConsoleManager::Get().FindTConsoleVariableDataBool(TEXT("p.ChaosCloth.DebugDrawElementIndices"));
			static const TConsoleVariableData<bool>* const DebugDrawClothClothConstraintsCVar = IConsoleManager::Get().FindTConsoleVariableDataBool(TEXT("p.ChaosCloth.DebugDrawClothClothConstraints"));
			static const TConsoleVariableData<bool>* const DebugDrawTeleportResetCVar = IConsoleManager::Get().FindTConsoleVariableDataBool(TEXT("p.ChaosCloth.DebugDrawTeleportReset"));
			static const TConsoleVariableData<bool>* const DebugDrawExtremlyDeformedEdgesCVar = IConsoleManager::Get().FindTConsoleVariableDataBool(TEXT("p.ChaosCloth.DebugDrawExtremlyDeformedEdges"));
			static const TConsoleVariableData<bool>* const DebugDrawVelocityScaleCVar = IConsoleManager::Get().FindTConsoleVariableDataBool(TEXT("p.ChaosCloth.DebugDrawVelocityScale"));

			FClothVisualizationNoGC::FDrawContext DrawContext;
			DrawContext.Filter.SingleVertex = DebugDrawSingleVertexCVar ? DebugDrawSingleVertexCVar->GetValueOnAnyThread() : INDEX_NONE;
			DrawContext.Filter.VertexSet = DebugDrawFilterSetCVar ? DebugDrawFilterSetCVar->GetValueOnAnyThread() : FString();
			FClothVisualizationNoGC::FDrawTextsContext DrawTextsContext;
			DrawTextsContext.Filter = DrawContext.Filter;

			if (DebugDrawParticleIndicesCVar && DebugDrawParticleIndicesCVar->GetValueOnAnyThread()) { Visualization->DrawParticleIndices(DrawTextsContext); }
			if (DebugDrawElementIndicesCVar && DebugDrawElementIndicesCVar->GetValueOnAnyThread()) { Visualization->DrawElementIndices(DrawTextsContext); }
			if (DebugDrawLocalSpaceCVar && DebugDrawLocalSpaceCVar->GetValueOnAnyThread()) { Visualization->DrawLocalSpace(DrawContext); }
			if (DebugDrawBoundsCVar && DebugDrawBoundsCVar->GetValueOnAnyThread()) { Visualization->DrawBounds(DrawContext); }
			if (DebugDrawGravityCVar && DebugDrawGravityCVar->GetValueOnAnyThread()) { Visualization->DrawGravity(DrawContext); }
			if (DebugDrawPhysMeshWiredCVar && DebugDrawPhysMeshWiredCVar->GetValueOnAnyThread()) { Visualization->DrawPhysMeshWired(DrawContext); }
			if (DebugDrawAnimMeshWiredCVar && DebugDrawAnimMeshWiredCVar->GetValueOnAnyThread()) { Visualization->DrawAnimMeshWired(DrawContext); }
			if (DebugDrawPointVelocitiesCVar && DebugDrawPointVelocitiesCVar->GetValueOnAnyThread()) { Visualization->DrawPointVelocities(DrawContext); }
			if (DebugDrawAnimNormalsCVar && DebugDrawAnimNormalsCVar->GetValueOnAnyThread()) { Visualization->DrawAnimNormals(DrawContext); }
			if (DebugDrawPointNormalsCVar && DebugDrawPointNormalsCVar->GetValueOnAnyThread()) { Visualization->DrawPointNormals(DrawContext); }
			if (DebugDrawCollisionCVar && DebugDrawCollisionCVar->GetValueOnAnyThread()) { Visualization->DrawCollision(DrawContext); }
			if (DebugDrawBackstopsCVar && DebugDrawBackstopsCVar->GetValueOnAnyThread()) { Visualization->DrawBackstops(DrawContext); }
			if (DebugDrawBackstopDistancesCVar && DebugDrawBackstopDistancesCVar->GetValueOnAnyThread()) { Visualization->DrawBackstopDistances(DrawContext); }
			if (DebugDrawMaxDistancesCVar && DebugDrawMaxDistancesCVar->GetValueOnAnyThread()) { Visualization->DrawMaxDistances(DrawContext); }
			if (DebugDrawMaxDistanceValuesCVar && DebugDrawMaxDistanceValuesCVar->GetValueOnAnyThread()) { Visualization->DrawMaxDistanceValues(DrawTextsContext); }
			if (DebugDrawAnimDriveCVar && DebugDrawAnimDriveCVar->GetValueOnAnyThread()) { Visualization->DrawAnimDrive(DrawContext); }
			if (DebugDrawEdgeConstraintCVar && DebugDrawEdgeConstraintCVar->GetValueOnAnyThread()) { Visualization->DrawEdgeConstraint(DrawContext); }
			if (DebugDrawBendingConstraintCVar && DebugDrawBendingConstraintCVar->GetValueOnAnyThread()) { Visualization->DrawBendingConstraint(DrawContext); }
			if (DebugDrawLongRangeConstraintCVar && DebugDrawLongRangeConstraintCVar->GetValueOnAnyThread()) { Visualization->DrawLongRangeConstraint(DrawContext); }
			if (DebugDrawWindForcesCVar && DebugDrawWindForcesCVar->GetValueOnAnyThread()) { Visualization->DrawWindAndPressureForces(DrawContext); }
			if (DebugDrawWindVelocityCVar && DebugDrawWindVelocityCVar->GetValueOnAnyThread()) { Visualization->DrawWindVelocity(DrawContext); }
			if (DebugDrawSelfCollisionCVar && DebugDrawSelfCollisionCVar->GetValueOnAnyThread()) { Visualization->DrawSelfCollision(DrawContext); }
			if (DebugDrawSelfIntersectionCVar && DebugDrawSelfIntersectionCVar->GetValueOnAnyThread()) { Visualization->DrawSelfIntersection(DrawContext); }
			if (DebugDrawSelfCollisionThicknessCVar && DebugDrawSelfCollisionThicknessCVar->GetValueOnAnyThread()) { Visualization->DrawSelfCollisionThickness(DrawContext); }
			if (DebugDrawClothCollisionThicknessCVar && DebugDrawClothCollisionThicknessCVar->GetValueOnAnyThread()) { Visualization->DrawClothCollisionThickness(DrawContext); }
			if (DebugDrawInnerCollisionThicknessCVar && DebugDrawInnerCollisionThicknessCVar->GetValueOnAnyThread()) { Visualization->DrawInnerCollisionThickness(DrawContext); }
			if (DebugDrawParticleIndicesCVar && DebugDrawParticleIndicesCVar->GetValueOnAnyThread()) { Visualization->DrawParticleIndices(DrawTextsContext); }
			if (DebugDrawElementIndicesCVar && DebugDrawElementIndicesCVar->GetValueOnAnyThread()) { Visualization->DrawElementIndices(DrawTextsContext); }
			if (DebugDrawClothClothConstraintsCVar && DebugDrawClothClothConstraintsCVar->GetValueOnAnyThread()) { Visualization->DrawClothClothConstraints(DrawContext); }
			if (DebugDrawTeleportResetCVar && DebugDrawTeleportResetCVar->GetValueOnAnyThread()) { Visualization->DrawTeleportReset(DrawContext); }
			if (DebugDrawExtremlyDeformedEdgesCVar && DebugDrawExtremlyDeformedEdgesCVar->GetValueOnAnyThread()) { Visualization->DrawExtremlyDeformedEdges(DrawContext); }
			if (DebugDrawVelocityScaleCVar && DebugDrawVelocityScaleCVar->GetValueOnAnyThread()) { Visualization->DrawVelocityScale(DrawContext); }
		}
#endif  // #if CHAOS_DEBUG_DRAW
	}

	void FClothSimulationProxy::CompleteParallelSimulation_GameThread()
	{
		check(IsInGameThread());

		if (IsValidRef(ParallelTask))
		{
			SCOPE_CYCLE_COUNTER(STAT_ClothSimulationProxy_EndParallelClothTask);
			CSV_SCOPED_SET_WAIT_STAT(Cloth);

			// There's a simulation in flight
			FTaskGraphInterface::Get().WaitUntilTaskCompletes(ParallelTask, ENamedThreads::GameThread);

			// No longer need this task, it has completed
			ParallelTask.SafeRelease();

			// Write back to the GT cache
			PostSimulate_GameThread();
			PostProcess_GameThread();
		}
	}

	void FClothSimulationProxy::UpdateClothLODs()
	{
		using namespace ::Chaos;
		check(!IsValidRef(ParallelInitializationTask));
		check(bIsPreProcessed);

		bool bAnyLODsChanged = false;
		for (const TUniquePtr<FClothingSimulationCloth>& Cloth : Cloths)
		{
			const int32 AssetIndex = Cloth->GetGroupId();

			if (!Cloth->GetMesh())
			{
				continue;  // Invalid or empty cloth
			}

			// If the LOD has changed while the simulation is suspended, the cloth still needs to be updated with the correct LOD data
			const int32 LODIndex = Cloth->GetMesh()->GetLODIndex();
			if (LODIndex != Cloth->GetLODIndex(Solver.Get()))
			{
				if (!ClothComponentAdapter.IsSimulationEnabled())
				{
					// Mark the cloth as needing to be reset so it doesn't both proxy-deforming lod transitions.
					Cloth->Reset();
				}
				bAnyLODsChanged = true;
			}
		}
		if (bAnyLODsChanged)
		{
			Solver->Update(Softs::FSolverReal(0.));  // Update for LOD switching, but do not simulate
		}
	}

	void FClothSimulationProxy::WriteSimulationData(TMap<int32, FClothSimulData>* OutClothSimulDataOverride)
	{
		using namespace ::Chaos;
		check(!IsValidRef(ParallelInitializationTask));
		check(bIsPreProcessed);

		CSV_SCOPED_TIMING_STAT(Animation, Cloth);
		TRACE_CPUPROFILER_EVENT_SCOPE(FClothSimulationProxy_WriteSimulationData);
		SCOPE_CYCLE_COUNTER(STAT_ClothSimulationProxy_WriteSimulationData);
		SCOPE_CYCLE_COUNTER(STAT_ClothWriteback);

		// Use either the proxy owned or overridden data
		if (!OutClothSimulDataOverride)
		{
			OutClothSimulDataOverride = &CurrentSimulationData;
		}

		// Cleanup the unused simulation data map on exit so that the render vertex factory won't run unnecessarily
		TBitArray<TInlineAllocator<16>> SimulationGroupIdsToRemove;
		auto FlagForRemovalOnExit = [&SimulationGroupIdsToRemove](const int32 SimulationGroupId, bool bRemove)
			{
				constexpr bool bRemoveDefault = false;  // Only remove owned data
				SimulationGroupIdsToRemove.SetNum(FMath::Max(SimulationGroupIdsToRemove.Num(), SimulationGroupId + 1), bRemoveDefault);
				SimulationGroupIdsToRemove[SimulationGroupId] = bRemove;
			};

		for (const TUniquePtr<FClothingSimulationCloth>& Cloth : Cloths)
		{
			for (int32 SimulationGroupId = Cloth ? Cloth->GetGroupId() : INDEX_NONE;
				SimulationGroupId != INDEX_NONE;
				SimulationGroupId = ClothComponentAdapter.GetNextLinkedSimulationGroupId(SimulationGroupId))
			{
				FlagForRemovalOnExit(SimulationGroupId, true);  // Flag for removal all owned data until it has been successfully copied
			}
		}

		ON_SCOPE_EXIT
			{
				for (TMap<int32, FClothSimulData>::TIterator It = OutClothSimulDataOverride->CreateIterator(); It; ++It)
				{
					const int32 SimulationGroupId = It.Key();
					if (SimulationGroupIdsToRemove.IsValidIndex(SimulationGroupId) && SimulationGroupIdsToRemove[SimulationGroupId])
					{
						It.RemoveCurrent();
					}
				}
			};

		// Check if our bone map is actually valid, if not there is no clothing data to build
		const USkinnedMeshComponent& OwnerComponent = ClothComponentAdapter.GetOwnerComponent();
		const USkinnedMeshComponent* const LeaderPoseComponent = OwnerComponent.LeaderPoseComponent.Get();

		if (LeaderPoseComponent && !OwnerComponent.GetLeaderBoneMap().Num())
		{
			return;
		}

		// Get the solver's local space
		const FVec3& LocalSpaceLocation = Solver->GetLocalSpaceLocation(); // Note: Since the ReferenceSpaceTransform can be suspended with the simulation, it is important that the suspended local space location is used too in order to get the simulation data back into reference space
		const FReal LocalSpaceScale = Solver->GetLocalSpaceScale();

		// Retrieve the component's bones transforms
		const TArray<FTransform>& ComponentSpaceTransforms = LeaderPoseComponent ? LeaderPoseComponent->GetComponentSpaceTransforms() : OwnerComponent.GetComponentSpaceTransforms();

		// Set the simulation data for each of the cloths
		for (const TUniquePtr<FClothingSimulationCloth>& Cloth : Cloths)
		{
			if (!Cloth ||
				!Cloth->GetMesh() ||
				Cloth->GetParticleRangeId(Solver.Get()) == INDEX_NONE ||
				Cloth->GetLODIndex(Solver.Get()) == INDEX_NONE)  // Needs a valid LOD
			{
				continue;
			}

			// Get the reference bone index for this cloth
			const int32 ReferenceBoneIndex = LeaderPoseComponent ? OwnerComponent.GetLeaderBoneMap()[Cloth->GetReferenceBoneIndex()] : Cloth->GetReferenceBoneIndex();

			// Get the reference bone transform for this cloth
			if (!ComponentSpaceTransforms.IsValidIndex(ReferenceBoneIndex))
			{
				UE_CLOGF(!bHasInvalidReferenceBoneTransforms, LogChaosClothAsset, Warning, "Failed to write back clothing simulation data for component %ls as bone transforms are invalid.",
					OwnerComponent.GetSkinnedAsset() ? *OwnerComponent.GetSkinnedAsset()->GetName() : *OwnerComponent.GetName());
				bHasInvalidReferenceBoneTransforms = true;
				continue;
			}

			// Set the FClothSimulData common properties for this group
			const int32 SimulationGroupId = Cloth->GetGroupId();
			FClothSimulData& Data = OutClothSimulDataOverride->FindOrAdd(SimulationGroupId);

			// Get the reference transform used in the current animation pose
			FTransform ReferenceBoneTransform = ComponentSpaceTransforms[ReferenceBoneIndex];
			ReferenceBoneTransform *= ClothSimulationContext->ComponentTransform;
			ReferenceBoneTransform.SetScale3D(FVector(1.0f));  // Scale is already baked in the cloth mesh

			// Set the world space transform to be this cloth's reference bone
			Data.Transform = ReferenceBoneTransform;
			Data.ComponentRelativeTransform = ReferenceBoneTransform.GetRelativeTransform(ClothSimulationContext->ComponentTransform);

			// Set the current LOD these data apply to, so that the correct deformer mappings can be applied
			// Note: If the LOD has changed while the simulation is suspended, the cloth still needs to be updated with the correct LOD data
			//       This should be handled by calling UpdateClothLODs when not ticking/simulating.
			const int32 LODIndex = Cloth->GetMesh()->GetLODIndex();
			ensure(LODIndex == Cloth->GetLODIndex(Solver.Get()));
			Data.LODIndex = Cloth->GetMesh()->GetOwnerLODIndex(LODIndex);  // The owner component LOD index can be different to the cloth mesh LOD index

			// Retrieve the last reference space transform used for this cloth
			// Note: This won't necessary match the current bone reference transform when the simulation is paused,
			//       and still allows for the correct positioning of the sim data while the component is animated.
			FRigidTransform3 ReferenceSpaceTransform = Cloth->GetReferenceSpaceTransform();
			ReferenceSpaceTransform.AddToTranslation(-LocalSpaceLocation);

			// Copy positions and normals
			Data.Positions = Cloth->GetParticlePositions(Solver.Get());
			Data.Normals = Cloth->GetParticleNormals(Solver.Get());
			check(Data.Positions.Num() == Data.Normals.Num());

			// Transform into the cloth reference simulation space used at the time of simulation
			bool bSimDataIsFinite = true;
#if INTEL_ISPC
			if (bTransformClothSimulData_ISPC_Enabled)
			{
				// ISPC is assuming float input here
				check(sizeof(ispc::FVector3f) == Data.Positions.GetTypeSize());
				check(sizeof(ispc::FVector3f) == Data.Normals.GetTypeSize());

				if (bSimulationDataCheckFinite)
				{
					bSimDataIsFinite = ispc::TransformClothSimulDataAndCheckFinite(
						(ispc::FVector3f*)Data.Positions.GetData(),
						(ispc::FVector3f*)Data.Normals.GetData(),
						(ispc::FTransform&)ReferenceSpaceTransform,
						LocalSpaceScale,
						Data.Positions.Num());
				}
				else
				{
					ispc::TransformClothSimulData(
						(ispc::FVector3f*)Data.Positions.GetData(),
						(ispc::FVector3f*)Data.Normals.GetData(),
						(ispc::FTransform&)ReferenceSpaceTransform,
						LocalSpaceScale,
						Data.Positions.Num());
				}
			}
			else
#endif
			{
				if (bSimulationDataCheckFinite)
				{
					auto VectorIsFinite = [](const FSolverVec3& Vector)
						{
							return FMath::IsFinite(Vector.X) && FMath::IsFinite(Vector.Y) && FMath::IsFinite(Vector.Z);
						};
					for (int32 Index = 0; Index < Data.Positions.Num(); ++Index)
					{
						Data.Positions[Index] = FVec3f(ReferenceSpaceTransform.InverseTransformPosition(LocalSpaceScale * FVec3(Data.Positions[Index])));
						Data.Normals[Index] = FVec3f(ReferenceSpaceTransform.InverseTransformVector(FVec3(-Data.Normals[Index])));
						bSimDataIsFinite = bSimDataIsFinite && VectorIsFinite(Data.Positions[Index]) && VectorIsFinite(Data.Normals[Index]);
					}
				}
				else
				{
					for (int32 Index = 0; Index < Data.Positions.Num(); ++Index)
					{
						Data.Positions[Index] = FVec3f(ReferenceSpaceTransform.InverseTransformPosition(LocalSpaceScale * FVec3(Data.Positions[Index])));
						Data.Normals[Index] = FVec3f(ReferenceSpaceTransform.InverseTransformVector(FVec3(-Data.Normals[Index])));
					}
				}
			}
			if (bSimulationDataCheckFinite)
			{
				UE_CLOGF(!bSimDataIsFinite, LogChaosClothAsset, Warning, "Failed to write back clothing simulation data for component %ls as simulation contains non-finite values.", *OwnerComponent.GetName());
				ensure(bSimDataIsFinite);
				if (!bSimDataIsFinite && bSimulationDataResetOnNonFinite)
				{
					bNeedResetClothFromNanCheck = true;
				}
			}
			if (bSimDataIsFinite)
			{
				// Mark the sim data as valid
				FlagForRemovalOnExit(SimulationGroupId, false);

				// Copy linked data (for when a single cloth asset is used through several Clothing Data in multiple render sections)
				for (int32 NextLinkedSimulationGroupId = ClothComponentAdapter.GetNextLinkedSimulationGroupId(SimulationGroupId);
					NextLinkedSimulationGroupId != INDEX_NONE;
					NextLinkedSimulationGroupId = ClothComponentAdapter.GetNextLinkedSimulationGroupId(NextLinkedSimulationGroupId))
				{
					FClothSimulData& NextData = OutClothSimulDataOverride->FindOrAdd(NextLinkedSimulationGroupId);
					NextData = Data;  // TODO: Deprecate the current map type to use shared pointers to avoid Data duplication if ever possible (the editor warns for shared cloth asset setup to avoid this).

					FlagForRemovalOnExit(NextLinkedSimulationGroupId, false);
				}
			}
		}
	}

	const TMap<int32, FClothSimulData>& FClothSimulationProxy::GetCurrentSimulationData_AnyThread() const
	{
		// This is called during EndOfFrameUpdates, usually in a parallel-for loop. We need to be sure that
		// the cloth task (if there is one) is complete, but it cannot be waited for here. See OnPreEndOfFrameUpdateSync
		// which is called just before EOF updates and is where we would have waited for the cloth task.
		if (bIsPreProcessed && (!IsValidRef(ParallelTask) || ParallelTask->IsComplete()))
		{
			return CurrentSimulationData;
		}
		static const TMap<int32, FClothSimulData> EmptyClothSimulationData;
		return EmptyClothSimulationData;
	}

	FBoxSphereBounds FClothSimulationProxy::CalculateBounds_AnyThread() const
	{
		SCOPE_CYCLE_COUNTER(STAT_ClothSimulationProxy_CalculateBounds);

		if (bIsPreProcessed && (!IsValidRef(ParallelTask) || ParallelTask->IsComplete()))
		{
			check(Solver);
			FBoxSphereBounds Bounds = Solver->CalculateBounds();

			// The component could be moving while the simulation is suspended so getting the bounds
			// in world space isn't good enough and the bounds origin needs to be continuously updated.
			// 
			// This converts the bounds back to component space. Do not apply LocalSpaceScale, which may not match component space.
			// TODO: this will not apply the component's actual scale either.
			Bounds = Bounds.TransformBy(FTransform((FQuat)Solver->GetLocalSpaceRotation(), (FVector)Solver->GetLocalSpaceLocation()).Inverse());

			return Bounds;
		}
		return FBoxSphereBounds(ForceInit);
	}

	const::Chaos::FClothVisualizationNoGC* FClothSimulationProxy::GetClothVisualization() const
	{
		return Visualization.Get();
	}

	void FClothSimulationProxy::InitializeConfigs()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FClothSimulationProxy_InitializeConfigs);

		const int32 LodIndex = ClothSimulationContext->LodIndex;
		Solver->SetSolverLOD(LodIndex);

		if (Configs.Num())
		{
			// Replace physics thread's configs with the game thread's configs
			constexpr bool bForceDirty = false;  // Do not force the copy of the collection if it isn't dirty
			int32 ConfigIndex = 0;
			Configs[ConfigIndex++]->Initialize(ClothComponentAdapter.GetSolverPropertyCollections(), bForceDirty, LodIndex);

			for (const UChaosClothAssetBase* const Asset : ClothComponentAdapter.GetAssets())
			{
				if (Asset)
				{
					for (int32 ModelIndex = 0; ModelIndex < Asset->GetNumClothSimulationModels(); ++ModelIndex)
					{
						const int32 SimulationGroupId = ClothComponentAdapter.GetSimulationGroupId(Asset, ModelIndex);
						if (SimulationGroupId != INDEX_NONE &&  // INDEX_NONE means this model hasn't been added to the simulation
							ensure(Configs.IsValidIndex(ConfigIndex)))
						{
							Configs[ConfigIndex++]->Initialize(ClothComponentAdapter.GetPropertyCollections(Asset, ModelIndex), bForceDirty, LodIndex);
						}
					}
				}
			}
			ClothComponentAdapter.ClearPropertyCollectionsDirtyFlags(LodIndex);
		}
	}

	void FClothSimulationProxy::FillSimulationContext(float DeltaTime, bool bIsInitialization)
	{
		ClothSimulationContext->Fill(ClothComponentAdapter, DeltaTime, MaxDeltaTime, bIsInitialization, CacheData.Get());
		CacheData.Reset();
	}

	bool FClothSimulationProxy::ShouldEnableSolver(bool bSolverCurrentlyEnabled) const
	{
		switch (SolverMode)
		{
		case ESolverMode::EnableSolverForSimulateRecord:
			return true;
		case ESolverMode::DisableSolverForPlayback:
			return false;
		case ESolverMode::Default:
		default:
			if (ClothSimulationContext->CacheData.HasData())
			{
				return false;
			}
		}
		return bSolverCurrentlyEnabled;
	}
}
