// Copyright Epic Games, Inc. All Rights Reserved.

#define EXPLICIT_VECTOR4 1

#include "Math/Vector.isph"
#include "Math/Quat.isph"
#include "Math/Transform.isph"

export void TransformClothSimulData(uniform FVector3f Positions[],
									uniform FVector3f Normals[],
									const uniform FTransform &ReferenceSpaceTransform,
									const uniform double LocalSpaceScale,
									const uniform int NumPositions)
{
	const uniform FVector4 RotationInverse = QuatInverse(ReferenceSpaceTransform.Rotation);
	const uniform FVector Translation = SetVector(ReferenceSpaceTransform.Translation);
	const uniform FVector SafeReciprocal = TransformGetSafeScaleReciprocal(SetVector(ReferenceSpaceTransform.Scale3D), SMALL_NUMBER);

	foreach(i = 0 ... NumPositions)
	{
		const uniform int Index = extract(i, 0);
		const FVector3f PositionFloat = VectorLoad(&Positions[Index]);
		const FVector3f NormalFloat = VectorLoad(&Normals[Index]);

		const FVector InputPositionVector = ConvertVector3fTo3Native(PositionFloat);

		const FVector3f InputNormalVectorFloat = NormalFloat * -FLOAT_ONE;
		const FVector InputNormalVector = ConvertVector3fTo3Native(InputNormalVectorFloat);

		const FVector ScaledPosition = InputPositionVector * LocalSpaceScale;
		const FVector TranslatedPosition = ScaledPosition - Translation;

		const FVector VRPosition = VectorQuaternionRotateVector(RotationInverse, TranslatedPosition);
		const FVector VRNormal = VectorQuaternionRotateVector(RotationInverse, InputNormalVector);

		const FVector OutPosition = VRPosition * SafeReciprocal;
		const FVector OutNormal = VRNormal * SafeReciprocal;

		const FVector3f OutPositionFloat = ConvertVector3NativeTo3f(OutPosition);
		const FVector3f OutNormalFloat = ConvertVector3NativeTo3f(OutNormal);

		VectorStore(&Positions[Index], OutPositionFloat);
		VectorStore(&Normals[Index], OutNormalFloat);
	}
}

inline bool VectorIsFinite(const FVector3f& Vec)
{
	return (intbits(Vec.V[0]) & 0x7f800000) != 0x7f800000 
		&& (intbits(Vec.V[1]) & 0x7f800000) != 0x7f800000 
		&& (intbits(Vec.V[2]) & 0x7f800000) != 0x7f800000;
}

export uniform bool TransformClothSimulDataAndCheckFinite(uniform FVector3f Positions[],
									uniform FVector3f Normals[],
									const uniform FTransform &ReferenceSpaceTransform,
									const uniform double LocalSpaceScale,
									const uniform int NumPositions)
{
	const uniform FVector4 RotationInverse = QuatInverse(ReferenceSpaceTransform.Rotation);
	const uniform FVector Translation = SetVector(ReferenceSpaceTransform.Translation);
	const uniform FVector SafeReciprocal = TransformGetSafeScaleReciprocal(SetVector(ReferenceSpaceTransform.Scale3D), SMALL_NUMBER);

	varying bool bIsFinite = true;
	foreach(i = 0 ... NumPositions)
	{
		const uniform int Index = extract(i, 0);
		const FVector3f PositionFloat = VectorLoad(&Positions[Index]);
		const FVector3f NormalFloat = VectorLoad(&Normals[Index]);


		const FVector InputPositionVector = ConvertVector3fTo3Native(PositionFloat);

		const FVector3f InputNormalVectorFloat = NormalFloat * -FLOAT_ONE;
		const FVector InputNormalVector = ConvertVector3fTo3Native(InputNormalVectorFloat);

		const FVector ScaledPosition = InputPositionVector * LocalSpaceScale;
		const FVector TranslatedPosition = ScaledPosition - Translation;

		const FVector VRPosition = VectorQuaternionRotateVector(RotationInverse, TranslatedPosition);
		const FVector VRNormal = VectorQuaternionRotateVector(RotationInverse, InputNormalVector);

		const FVector OutPosition = VRPosition * SafeReciprocal;
		const FVector OutNormal = VRNormal * SafeReciprocal;

		const FVector3f OutPositionFloat = ConvertVector3NativeTo3f(OutPosition);
		const FVector3f OutNormalFloat = ConvertVector3NativeTo3f(OutNormal);
		
		bIsFinite = bIsFinite && VectorIsFinite(OutPositionFloat) && VectorIsFinite(OutNormalFloat);

		VectorStore(&Positions[Index], OutPositionFloat);
		VectorStore(&Normals[Index], OutNormalFloat);
	}
	return all(bIsFinite);
}
