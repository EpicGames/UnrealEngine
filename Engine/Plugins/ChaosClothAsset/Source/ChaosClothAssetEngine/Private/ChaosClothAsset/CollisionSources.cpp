// Copyright Epic Games, Inc. All Rights Reserved.
#include "ChaosClothAsset/CollisionSources.h"

#include "ChaosCloth/ChaosClothingSimulationCollider.h"
#include "Components/SkinnedMeshComponent.h"
#include "Engine/SkinnedAsset.h"
#include "PhysicsEngine/EnvironmentalCollisions.h"
#include "PhysicsEngine/PhysicsAsset.h"

namespace UE::Chaos::ClothAsset
{
	FCollisionSources::FCollisionSources(USkinnedMeshComponent* InOwnerComponent, bool bInCollideWithEnvironment)
		: OwnerComponent(InOwnerComponent)
		, bCollideWithEnvironment(bInCollideWithEnvironment)
	{}

	FCollisionSources::~FCollisionSources() = default;

	void FCollisionSources::Add(USkinnedMeshComponent* SourceComponent, const UPhysicsAsset* SourcePhysicsAsset, bool bUseSphylsOnly)
	{
		using namespace UE::Chaos::ClothAsset;

		if (OwnerComponent.IsValid() && SourceComponent && SourcePhysicsAsset)
		{
			FCollisionSource* const FoundCollisionSource = CollisionSources.FindByPredicate(
				[SourceComponent, SourcePhysicsAsset](const FCollisionSource& CollisionSource)
				{
					return CollisionSource.SourceComponent == SourceComponent && CollisionSource.SourcePhysicsAsset == SourcePhysicsAsset;
				}
			);

			if (!FoundCollisionSource)
			{
				// Add the new collision source
				CollisionSources.Emplace(SourceComponent, SourcePhysicsAsset, bUseSphylsOnly);

				// Add prerequisite so we don't end up with a frame delay
				OwnerComponent->PrimaryComponentTick.AddPrerequisite(SourceComponent, SourceComponent->PrimaryComponentTick);

				// Mark the collision sources as changed
				++Version;
			}
		}
	}

	void FCollisionSources::Remove(const USkinnedMeshComponent* SourceComponent)
	{
		using namespace UE::Chaos::ClothAsset;

		if (SourceComponent)
		{
			// Note: Stale prerequises are removed when in the QueueTickFunction once the source object has been destroyed.
			const int32 NumRemoved = CollisionSources.RemoveAll([SourceComponent](const FCollisionSource& CollisionSource)
				{
					return !CollisionSource.SourceComponent.IsValid() || CollisionSource.SourceComponent == SourceComponent;
				});
		
			// Mark the collision sources as changed
			if (NumRemoved > 0)
			{
				++Version;
			}
		}
	}

	void FCollisionSources::Remove(const USkinnedMeshComponent* SourceComponent, const UPhysicsAsset* SourcePhysicsAsset)
	{
		using namespace UE::Chaos::ClothAsset;

		if (SourceComponent)
		{
			// Note: Stale prerequises are removed when in the QueueTickFunction once the source object has been destroyed.
			const int32 NumRemoved = CollisionSources.RemoveAll([SourceComponent, SourcePhysicsAsset](const FCollisionSource& CollisionSource)
				{
					return !CollisionSource.SourceComponent.IsValid() ||
						(CollisionSource.SourceComponent == SourceComponent && CollisionSource.SourcePhysicsAsset == SourcePhysicsAsset);
				});

			// Mark the collision sources as changed
			if (NumRemoved > 0)
			{
				++Version;
			}
		}
	}

	void FCollisionSources::AddExternalCollisions(const FClothCollisionData& InExternalCollisions)
	{
		ExternalCollisions.Append(InExternalCollisions);
	}

	void FCollisionSources::ClearExternalCollisions()
	{
		ExternalCollisions.Reset();
	}

	void FCollisionSources::Reset()
	{
		CollisionSources.Reset();
		++Version;
	}

	void FCollisionSources::SetOwnerComponent(USkinnedMeshComponent* InOwnerComponent)
	{
		OwnerComponent = MakeWeakObjectPtr(InOwnerComponent);
	}

	void FCollisionSources::ExtractSkinnedMeshCollisionData(FClothCollisionData& CollisionData)
	{
		CollisionData.Reset();
		if (OwnerComponent.IsValid())
		{
			for (FCollisionSource& CollisionSource : CollisionSources)
			{
				CollisionSource.ExtractCollisionData(*OwnerComponent, CollisionData);
			}

		}
	}

	bool FCollisionSources::IsCollisionDataUpToDate(int32 InVersion) const
	{
		if (InVersion != Version)
		{
			return false;
		}

		for (const FCollisionSource& CollisionSource : CollisionSources)
		{
			if (CollisionSource.ResolvedSourceComponent)
			{
				if (CollisionSource.BoneTransformRevisionNumber != CollisionSource.ResolvedSourceComponent->GetBoneTransformRevisionNumber())
				{
					return false;
				}
			}
		}

		return true;
	}

	void FCollisionSources::MarkCollisionDataUpToDate(int32 InVersion)
	{
		Version = InVersion;

		for (FCollisionSource& CollisionSource : CollisionSources)
		{
			if (CollisionSource.ResolvedSourceComponent)
			{
				CollisionSource.BoneTransformRevisionNumber = CollisionSource.ResolvedSourceComponent->GetBoneTransformRevisionNumber();
			}
		}
	}

	void FCollisionSources::ResolveWeakComponentPtrs()
	{
		for (FCollisionSource& CollisionSource : CollisionSources)
		{
			CollisionSource.ResolvedSourceComponent = CollisionSource.SourceComponent.Get();
		}
	}

	void FCollisionSources::ExtractEnvironmentalCollisionData(FClothCollisionData& CollisionData)
	{
		CollisionData.Reset();
		if (OwnerComponent.IsValid())
		{
			if (bCollideWithEnvironment)
			{
				FEnvironmentalCollisions::AppendCollisionDataFromEnvironment(OwnerComponent.Get(), CollisionData);
			}
		}
	}

	void FCollisionSources::ExtractExternalCollisionData(FClothCollisionData& CollisionData)
	{
		CollisionData = ExternalCollisions;
	}

	void FCollisionSourcesProxy::ExtractCollisionData()
	{
		bool bCollisionDataChanged = false;

		CollisionSources.ResolveWeakComponentPtrs();

		if (!CollisionSources.IsCollisionDataUpToDate(Version))
		{
			CollisionSources.ExtractSkinnedMeshCollisionData(SkinnedMeshCollisionData);
			CollisionSources.MarkCollisionDataUpToDate(Version);
			bCollisionDataChanged = true;
		}

		// EnvironmentalCollisionData changes every frame, so if we had some before, it's now stale.
		bCollisionDataChanged |= !EnvironmentalCollisionData.IsEmpty();

		CollisionSources.ExtractEnvironmentalCollisionData(EnvironmentalCollisionData);
		bCollisionDataChanged |= !EnvironmentalCollisionData.IsEmpty();

		// ExternalCollisionData changes every frame, so if we had some before, it's now stale.
		bCollisionDataChanged |= !ExternalCollisionData.IsEmpty();

		CollisionSources.ExtractExternalCollisionData(ExternalCollisionData);
		bCollisionDataChanged |= !ExternalCollisionData.IsEmpty();

		if (bCollisionDataChanged)
		{
			CollisionData = SkinnedMeshCollisionData;
			CollisionData.Append(EnvironmentalCollisionData);
			CollisionData.Append(ExternalCollisionData);
		}
	}

	FCollisionSources::FCollisionSource::FCollisionSource(
		USkinnedMeshComponent* InSourceComponent,
		const UPhysicsAsset* InSourcePhysicsAsset,
		bool bInUseSphylsOnly)
		: SourceComponent(InSourceComponent)
		, SourcePhysicsAsset(InSourcePhysicsAsset)
		, bUseSphylsOnly(bInUseSphylsOnly)
	{
		if (InSourceComponent)
		{
			BoneTransformRevisionNumber = InSourceComponent->GetBoneTransformRevisionNumber();
		}
	}

	void FCollisionSources::FCollisionSource::ExtractCollisionData(const USkinnedMeshComponent& InOwnerComponent, FClothCollisionData& CollisionData)
	{
		if (ResolvedSourceComponent)
		{
			const USkinnedAsset* SkinnedAsset = ResolvedSourceComponent->GetSkinnedAsset();

			// Extract the collision data if not already cached
			if (CachedSkinnedAsset != SkinnedAsset)
			{
				CachedSkinnedAsset = SkinnedAsset;
				CachedCollisionData.Reset();
				CachedUsedBoneIndices.Reset();

				const UPhysicsAsset* const PhysicsAsset = SourcePhysicsAsset.Get();

				if (SkinnedAsset && PhysicsAsset)
				{
					// Extract collisions
					// Currently not handling extended collision data
					::Chaos::FClothCollisionDataExtended ExtendedData;
					TArray<int32> UsedSubBoneIndices;

					constexpr bool bSkipMissingBones = true;
					::Chaos::FClothingSimulationCollider::ExtractPhysicsAssetCollision(
						PhysicsAsset,
						&SkinnedAsset->GetRefSkeleton(),
						CachedCollisionData,
						ExtendedData,
						CachedUsedBoneIndices,
						UsedSubBoneIndices,
						bUseSphylsOnly,
						bSkipMissingBones);
				}
			}

			// Transform and add the cached collisions
			if (CachedUsedBoneIndices.Num())
			{
				// Calculate the component to component transform
				FTransform ComponentToComponentTransform;
				if (SourceComponent != &InOwnerComponent)
				{
					FTransform DestClothComponentTransform = InOwnerComponent.GetComponentTransform();
					DestClothComponentTransform.RemoveScaling();  // The collision source doesn't need the scale of the cloth skeletal mesh applied to it (but it does need the source scale from the component transform)
					ComponentToComponentTransform = SourceComponent->GetComponentTransform() * DestClothComponentTransform.Inverse();
				}

				// Retrieve the bone transforms
				TArray<FTransform> BoneTransforms;
				BoneTransforms.Reserve(CachedUsedBoneIndices.Num());
				for (const int32 UsedBoneIndex : CachedUsedBoneIndices)
				{
					BoneTransforms.Emplace(SourceComponent->GetBoneTransform(UsedBoneIndex, ComponentToComponentTransform));
				}

				// Append the transformed collision elements
				CollisionData.AppendTransformed(CachedCollisionData, BoneTransforms);
			}
		}
	}
}  // End namespace UE::Chaos::ClothAsset
