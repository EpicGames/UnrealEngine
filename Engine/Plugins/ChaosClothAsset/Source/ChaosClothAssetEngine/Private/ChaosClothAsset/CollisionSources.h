// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ClothCollisionData.h"
#include "Delegates/Delegate.h"

class USkinnedMeshComponent;
class USkinnedAsset;
class UPhysicsAsset;

namespace UE::Chaos::ClothAsset
{
	/**
	 * Cloth collision source container.
	 */
	class FCollisionSources final
	{
	public:
		explicit FCollisionSources(USkinnedMeshComponent* InOwnerComponent = nullptr, bool bInCollideWithEnvironment = false);
		~FCollisionSources();

		void Add(USkinnedMeshComponent* SourceComponent, const UPhysicsAsset* SourcePhysicsAsset, bool bUseSphylsOnly = false);

		void Remove(const USkinnedMeshComponent* SourceComponent);

		void Remove(const USkinnedMeshComponent* SourceComponent, const UPhysicsAsset* SourcePhysicsAsset);

		void AddExternalCollisions(const FClothCollisionData& InExternalCollisions);

		void ClearExternalCollisions();

		void Reset();

		void SetOwnerComponent(USkinnedMeshComponent* InOwnerComponent);

		USkinnedMeshComponent* GetOwnerComponent() const
		{
			return OwnerComponent.Get();
		}

		void SetCollideWithEnvironment(bool bCollide)
		{
			bCollideWithEnvironment = bCollide;
		}

	private:
		friend class FCollisionSourcesProxy;

		struct FCollisionSource
		{
			const TWeakObjectPtr<USkinnedMeshComponent> SourceComponent;
			// Resolved copy of SourceComponent, refreshed prior to each update to avoid multiple TWeakObjectPtr resolves
			const USkinnedMeshComponent* ResolvedSourceComponent = nullptr;
			const TWeakObjectPtr<const UPhysicsAsset> SourcePhysicsAsset;
			TWeakObjectPtr<const USkinnedAsset> CachedSkinnedAsset;
			FClothCollisionData CachedCollisionData;
			TArray<int32> CachedUsedBoneIndices;
			uint32 BoneTransformRevisionNumber = 0;
			bool bUseSphylsOnly = false;

			FCollisionSource(
				USkinnedMeshComponent* InSourceComponent,
				const UPhysicsAsset* InSourcePhysicsAsset,
				bool bInUseSphylsOnly);

			void ExtractCollisionData(const USkinnedMeshComponent& InOwnerComponent, FClothCollisionData& CollisionData);
		};

		void ExtractSkinnedMeshCollisionData(FClothCollisionData& CollisionData);
		void ExtractEnvironmentalCollisionData(FClothCollisionData& CollisionData);
		void ExtractExternalCollisionData(FClothCollisionData& CollisionData);
		bool IsCollisionDataUpToDate(int32 InVersion) const;
		void MarkCollisionDataUpToDate(int32 InVersion);
		void ResolveWeakComponentPtrs();

		TWeakObjectPtr<USkinnedMeshComponent> OwnerComponent;
		TArray<FCollisionSource> CollisionSources;
		FClothCollisionData ExternalCollisions;
		int32 Version = INDEX_NONE;
		bool bCollideWithEnvironment = false;
	};

	/**
	 * Use a proxy object to extract collision data from the collision sources.
	 * The proxy allows for a different ownership than of the CollisionSources' owning component,
	 * permitting the collision data to remains with the simulation proxy even after the simulation proxy has been replaced.
	 */
	class FCollisionSourcesProxy final
	{
	public:
		explicit FCollisionSourcesProxy(FCollisionSources& InCollisionSources) : CollisionSources(InCollisionSources) {}

		const FClothCollisionData& GetCollisionData() const { return CollisionData; }

		void ExtractCollisionData();

	protected:
		FCollisionSources& CollisionSources;
		FClothCollisionData SkinnedMeshCollisionData;
		FClothCollisionData EnvironmentalCollisionData;
		FClothCollisionData ExternalCollisionData;

		FClothCollisionData CollisionData;
		int32 Version = INDEX_NONE;
	};
}
