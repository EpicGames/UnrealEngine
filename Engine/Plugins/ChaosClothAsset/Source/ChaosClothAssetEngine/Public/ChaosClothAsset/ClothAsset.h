// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosClothAsset/ClothAssetBase.h"
#include "ReferenceSkeleton.h"
#include "RenderCommandFence.h"
#include "PerQualityLevelProperties.h"
#include "ClothAsset.generated.h"

#define UE_API CHAOSCLOTHASSETENGINE_API

namespace UE_DEPRECATED(5.5, "Use UE::Dataflow instead.") Dataflow {}

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
class UAnimationAsset;
class USkeletalMesh;
#endif

class FSkeletalMeshModel;
struct FSkeletalMeshLODInfo;
struct FChaosClothAssetLodTransitionDataCache;

UENUM()
enum class EClothAssetAsyncProperties : uint64
{
	None = 0,
	RenderData UE_DEPRECATED(5.6, "Use EChaosClothAssetBaseAsyncProperties::RenderData instead.") = 1 << 0,
	ThumbnailInfo = 1 << 1,
	ImportedModel = 1 << 2,
	ClothCollection = 1 << 3,
	RefSkeleton UE_DEPRECATED(5.6, "Use EChaosClothAssetBaseAsyncProperties::RefSkeleton instead.") = 1 << 4,
	All = MAX_uint32  // Max is uint32 as we need some space for the UChaosClothAssetBase ones
};
ENUM_CLASS_FLAGS(EClothAssetAsyncProperties);

/**
 * Cloth asset for pattern based simulation.
 */
UCLASS(MinimalAPI, hidecategories = Object, BlueprintType, PrioritizeCategories = ("Dataflow"))
class UChaosClothAsset : public UChaosClothAssetBase
{
	GENERATED_BODY()
public:
	UE_API UChaosClothAsset(const FObjectInitializer& ObjectInitializer);
	UE_API UChaosClothAsset(FVTableHelper& Helper);  // This is declared so we can use TUniquePtr<FClothSimulationModel> with just a forward declare of that class
	UE_API virtual ~UChaosClothAsset() override;

	//~ Begin UObject interface
	UE_API virtual void Serialize(FArchive& Ar) override;
	UE_API virtual void PostLoad() override;
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject interface

	//~ Begin USkinnedAsset interface
	virtual UPhysicsAsset* GetPhysicsAsset() const								{ return PhysicsAsset; }
	virtual USkeleton* GetSkeleton() override									{ return Skeleton; }  // Note: The USkeleton isn't a reliable source of reference skeleton
	virtual const USkeleton* GetSkeleton() const override						{ return Skeleton; }
	virtual void SetSkeleton(USkeleton* InSkeleton) override					{ Skeleton = InSkeleton; }

#if WITH_EDITOR
	/* Build a LOD model for the targeted platform. */
	UE_API virtual void BuildLODModel(FSkeletalMeshRenderData& RenderData, const ITargetPlatform* TargetPlatform, int32 LODIndex) override;
	UE_API virtual FString BuildDerivedDataKey(const ITargetPlatform* TargetPlatform) override;
	virtual const TCHAR* GetDerivedDataPrefix() const override
	{
		return TEXT("CHAOSCLOTH");
	}
	UE_API virtual const TCHAR* GetDerivedDataVersion() const override;
	UE_API virtual void PrepareMeshModel() override;
#endif
#if WITH_EDITORONLY_DATA
	virtual FSkeletalMeshModel* GetImportedModel() const override
	{
		WaitUntilAsyncPropertyReleased(EClothAssetAsyncProperties::ImportedModel);
		return MeshModel.Get();
	}
#endif
	//~ End USkinnedAsset interface

	UE_DEPRECATED(5.7, "Use SetClothCollections() or GetClothCollectionsInternal() instead.")
	TArray<TSharedRef<const FManagedArrayCollection>>& GetClothCollections()
	{
		return GetClothCollectionsInternal();
	}

	/** Return the enclosed Cloth Collections. */
	const TArray<TSharedRef<const FManagedArrayCollection>>& GetClothCollections() const
	{
		WaitUntilAsyncPropertyReleased(EClothAssetAsyncProperties::ClothCollection, ESkinnedAssetAsyncPropertyLockType::ReadOnly);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return ClothCollections;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/**
	 * Set new Cloth Collections for this Cloth Asset.
	 * @param InClothCollections The source managed array collections to set to the asset.
	 * @param bReregisterComponents Whether to reregister all components using this asset. This is the safe way to hard reset all simulations.
	 */
	UE_API void SetClothCollections(TArray<TSharedRef<const FManagedArrayCollection>>&& InClothCollections, bool bReregisterComponents = false);

	/**
	 * Rebuild this asset from the specified managed array collection data.
	 * @param InClothCollections The source managed array collections to build/rebuild the asset with.
	 * @param InOutTransitionCache The LOD transition cache to speed up the build process, if available.
	 * @param ErrorText The headline error text if any error occurs, an empty text otherwise.
	 * @param VerboseText The verbose description of the error if any.
	 */
	UE_API void Build(
		const TArray<TSharedRef<const FManagedArrayCollection>>& InClothCollections,
		TArray<FChaosClothAssetLodTransitionDataCache>* InOutTransitionCache = nullptr,
		FText* ErrorText = nullptr,
		FText* VerboseText = nullptr);

	UE_DEPRECATED(5.6, "Will be made private. Use Build with cloth collections instead.")
	UE_API void Build(TArray<FChaosClothAssetLodTransitionDataCache>* InOutTransitionCache = nullptr);

	/**
	 * Copy the draped simulation mesh patterns into the render mesh data.
	 * This is useful to visualize the simulation mesh, or when the simulation mesh can be used for both simulation and rendering.
	 * @param Material The material used to render. If none is specified the default debug cloth material will be used.
	 */
	UE_DEPRECATED(5.6, "Use FClothGeometryTools:CopySimMeshToRenderMesh on the cloth collections instead.")
	UE_API void CopySimMeshToRenderMesh(UMaterialInterface* Material = nullptr);

	/** Set the physics asset for this cloth. */
	UE_API void SetPhysicsAsset(UPhysicsAsset* InPhysicsAsset);

	UE_DEPRECATED(5.6, "Will be made private. Use Build with cloth collections instead.")
	UE_API void UpdateSkeletonFromCollection(bool bRebuildModels);

	UE_DEPRECATED(5.6, "SetReferenceSkeleton is deprecated. Skeletons must only been set through the cloth collections.")
	UE_API void SetReferenceSkeleton(const FReferenceSkeleton* ReferenceSkeleton, bool bRebuildModels, bool bRebindMeshes);

	UE_DEPRECATED(5.6, "No longer used.")
	static void OnLodStrippingQualityLevelChanged(IConsoleVariable* /*Variable*/) {}

	/**
	 * Set the name of the Dataflow terminal node for this cloth asset.
	 * @param InDataflowTerminal The new name of the Dataflow terminal node.
	 */
	UE_DEPRECATED(5.6, "Function will be removed as the DataflowTerminal shouldn't be set by name/string.")
	void SetDataflowTerminal(const FString& InDataflowTerminal) 
	{ 
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		DataflowInstance.SetDataflowTerminal(FName(*InDataflowTerminal));
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** Return the Dataflow graph asset associated to this cloth asset if any. */
	UE_DEPRECATED(5.6, "Use GetDataflowInstance() instead")
	FString GetDataflowTerminal() const 
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return DataflowInstance.GetDataflowTerminal().ToString();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	//~ Begin UChaosClothAssetBase interface
	/** Return the cloth simulation ready LOD model data. */
	virtual TSharedPtr<const FChaosClothSimulationModel> GetClothSimulationModel(int32 /*ModelIndex*/ = 0) const override
	{
		return ClothSimulationModel;
	}
	UE_API virtual bool HasValidClothSimulationModels() const override;
	virtual int32 GetNumClothSimulationModels() const override
	{
		return ClothSimulationModel.IsValid() ? 1 : 0;
	}
	virtual FName GetClothSimulationModelName(int32 /*ModelIndex*/) const override
	{
		return GetFName();
	}
	virtual const TArray<TSharedRef<const FManagedArrayCollection>>& GetCollections(int32 /*ModelIndex*/) const override
	{
		return GetClothCollections();
	}
	virtual const UPhysicsAsset* GetPhysicsAssetForModel(int32 /*ModelIndex*/) const override
	{
		return PhysicsAsset;
	}
#if WITH_EDITOR
	/** Load render data from DDC if the data is cached, otherwise generate render data and save into DDC */
	UE_API virtual void CacheDerivedData(FSkinnedAssetCompilationContext* Context) override;
	UE_API virtual UChaosClothAssetBase* CreatePreviewAssetCopy(UObject* Outer, EObjectFlags Flags) const override;
#endif
	UE_DEPRECATED(5.6, "SetReferenceSkeleton is deprecated. Skeletons must only been set through the cloth collections.")
	virtual void SetReferenceSkeleton(const FReferenceSkeleton* ReferenceSkeleton) override
	{
		Super::SetReferenceSkeleton(ReferenceSkeleton);
	}
	virtual FGuid GetAssetGuid(int32 /*ModelIndex*/) const override
	{
		return AssetGuid;
	}
	//~ End UChaosClothAssetBase interface

private:
	//~ Begin USkinnedAsset interface
#if WITH_EDITOR
	/** Initial step for the building process - Can't be done in parallel. USkinnedAsset Interface. */
	UE_API virtual void BeginBuildInternal(FSkinnedAssetBuildContext& Context) override;
	/** Thread-safe part. USkinnedAsset Interface. */
	UE_API virtual void ExecuteBuildInternal(FSkinnedAssetBuildContext& Context) override;
	/** Complete the building process - Can't be done in parallel. USkinnedAsset Interface. */
	UE_API virtual void FinishBuildInternal(FSkinnedAssetBuildContext& Context) override;
	virtual void BeginPostLoadAssetImpl(FSkinnedAssetPostLoadContext& Context) override;
	virtual void ExecutePostLoadAssetImpl(FSkinnedAssetPostLoadContext& Context) override;
#endif
	/** Convert async property from enum value to string. */
	UE_API virtual FString GetAsyncPropertyName(uint64 Property) const override;
	//~ End USkinnedAsset interface

	/** Build the clothing simulation meshes from the Cloth Collection. */
	UE_API void BuildClothSimulationModel(TArray<FChaosClothAssetLodTransitionDataCache>* InOutTransitionCache = nullptr);

	/** Return the cloth collection array with async protection access. */
	TArray<TSharedRef<const FManagedArrayCollection>>& GetClothCollectionsInternal()  // TODO: Can be renamed GetClothCollections() once the current public method is removed in 5.9
	{
		WaitUntilAsyncPropertyReleased(EClothAssetAsyncProperties::ClothCollection);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return ClothCollections;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

#if WITH_EDITORONLY_DATA
	/** Dataflow asset. */
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Dataflow Asset is now stored in DataflowInstance"))
	TObjectPtr<UDataflow> DataflowAsset_DEPRECATED;

	/** Dataflow Asset terminal node. */
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Dataflow terminal name is now stored in DataflowInstance"))
	FString DataflowTerminal_DEPRECATED = "ClothAssetTerminal";
#endif  // WITH_EDITORONLY_DATA

	/**
	 * Skeleton asset used at creation time.
	 * This is of limited use since this USkeleton's reference skeleton might not necessarily match the one created for this asset.
	 * Set by the Dataflow evaluation.
	 */
	UPROPERTY(EditAnywhere, Setter = SetSkeleton, Category = Skeleton, Meta = (EditCondition = "!HasDataflow()"))
	TObjectPtr<USkeleton> Skeleton;

	/** Physics asset used for collision. Set by the Dataflow evaluation. */
	UPROPERTY(EditAnywhere, Category = Collision, Meta = (EditCondition = "!HasDataflow()"))
	TObjectPtr<UPhysicsAsset> PhysicsAsset;

	/** Whether to blend positions between the skinned/simulated transitions of the cloth render mesh. Only used when not overridden in Dataflow by the ProxyDeformer node. */
	UE_DEPRECATED(5.4, "Superseded by the ProxyDeformer node.")
	UPROPERTY()
	bool bSmoothTransition_DEPRECATED = true;

	/** Whether to use multiple triangle influences on the proxy wrap deformer to help smoothe deformations. Only used when not overridden in Dataflow by the ProxyDeformer node. */
	UE_DEPRECATED(5.4, "Superseded by the ProxyDeformer node.")
	UPROPERTY()
	bool bUseMultipleInfluences_DEPRECATED = false;

	/** The radius from which to get the multiple triangle influences from the simulated proxy mesh. Only used when not overridden in Dataflow with the ProxyDeformer node. */
	UE_DEPRECATED(5.4, "Superseded by the ProxyDeformer node.")
	UPROPERTY()
	float SkinningKernelRadius_DEPRECATED = 30.f;

	/** A unique identifier as used by the section rendering code. */
	UPROPERTY()
	FGuid AssetGuid;

	/** Cloth Collection containing this asset data. One per LOD. */
	UE_DEPRECATED(5.4, "This must be protected for async build, always use the accessors even internally.")
	TArray<TSharedRef<const FManagedArrayCollection>> ClothCollections;

	/** Simulation mesh Lods as fed to the solver for constraints creation. Ownership gets transferred to the proxy when it is changed during a simulation. */
	TSharedPtr<FChaosClothSimulationModel> ClothSimulationModel;

#if WITH_EDITOR
	struct FBuilder;
#endif
};

#undef UE_API
