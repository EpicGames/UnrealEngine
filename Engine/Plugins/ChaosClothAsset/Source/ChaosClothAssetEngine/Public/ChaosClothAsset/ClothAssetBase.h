// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/SkinnedAsset.h"
#include "Dataflow/DataflowContent.h"
#include "Dataflow/DataflowInstance.h"
#include "ClothAssetBase.generated.h"

#define UE_API CHAOSCLOTHASSETENGINE_API

class FSkeletalMeshModel;
class FSkeletalMeshRenderData;
class FSkinnedAssetCompilationContext;
class ITargetPlatform;
class UDataflow;
class UActorComponent;
class UMaterialInterface;
struct FChaosClothSimulationModel;
struct FManagedArrayCollection;

/**
 * Async build property lock identifiers for UChaosClothAssetBase.
 * Bits 0-31 are reserved for the base class. Derived classes (e.g. UChaosClothAsset, UChaosOutfitAsset)
 * use bits 32-63 via the generic AcquireAsyncProperty/ReleaseAsyncProperty/WaitUntilAsyncPropertyReleased
 * templates declared on the base, which left-shift derived enum values by 32. 
 * The base class uses these bits directly via the EChaosClothAssetBaseAsyncProperties specializations.
 */
UENUM()
enum class EChaosClothAssetBaseAsyncProperties : uint32
{
	None = 0,
	RenderData = 1 << 0,
	RefSkeleton = 1 << 1,
	HasVertexColors = 1 << 2,
	OverlayMaterial = 1 << 3,
	OverlayMaterialMaxDrawDistance = 1 << 4,
	All = MAX_uint32
};
ENUM_CLASS_FLAGS(EChaosClothAssetBaseAsyncProperties);

/**
 * Base cloth asset class.
 * Any object slot based on this type can be either a Cloth Asset or an Outfit Asset.
 */
UCLASS(MinimalAPI, Abstract, BlueprintType)
class UChaosClothAssetBase
	: public USkinnedAsset
	, public IDataflowContentOwner
	, public IDataflowInstanceInterface
{
	GENERATED_BODY()

public:
	UE_API UChaosClothAssetBase(const FObjectInitializer& ObjectInitializer);

	//~ Begin UChaosClothAssetBase interface
	/** Return whether the asset has any valid cloth simulation models and is simulation enabled. */
	virtual bool HasValidClothSimulationModels() const
	PURE_VIRTUAL(UChaosClothAssetBase::HasValidClothSimulationModels, return false;);

	/** Return the number of cloth simulation models in this asset. */
	virtual int32 GetNumClothSimulationModels() const
	PURE_VIRTUAL(UChaosClothAssetBase::GetNumClothSimulationModels, return 0;);

	/** Return the simulation model physical data. */
	virtual TSharedPtr<const FChaosClothSimulationModel> GetClothSimulationModel(int32 ModelIndex) const
	PURE_VIRTUAL(UChaosClothAssetBase::GetClothSimulationModel, return nullptr;);

	/** Return the name of the specified cloth simulation models in this asset. */
	virtual FName GetClothSimulationModelName(int32 ModelIndex) const
	PURE_VIRTUAL(UChaosClothAssetBase::GetClothSimulationModelName, return NAME_None;);

	/** Return the collections for this asset model (one per LOD). */
	virtual const TArray<TSharedRef<const FManagedArrayCollection>>& GetCollections(int32 ModelIndex) const
	PURE_VIRTUAL(UChaosClothAssetBase::GetCollections, static const TArray<TSharedRef<const FManagedArrayCollection>> EmptyArray; return EmptyArray;);

	/** Return the physics asset used for the simulation by this asset model. */
	virtual const UPhysicsAsset* GetPhysicsAssetForModel(int32 ModelIndex) const
	PURE_VIRTUAL(UChaosClothAssetBase::GetPhysicsAssetForModel, return nullptr;);

	/** Return the asset GUID used to match the render sections for this asset model. */
	virtual FGuid GetAssetGuid(int32 ModelIndex) const
	PURE_VIRTUAL(UChaosClothAssetBase::GetAssetGuid, return FGuid(););

#if WITH_EDITOR
	/** Load the running-platform render data from the DDC if cached, otherwise leave the render data untouched for the caller to rebuild. */
	UE_API virtual void CacheDerivedData(FSkinnedAssetCompilationContext* Context);

	/**
	 * Create a simplified copy of this asset suitable for preview rendering (thumbnails, etc.).
	 * @param Outer The outer object for the preview copy.
	 * @param Flags Object flags for the preview copy.
	 * @return A transient asset copy, or nullptr if this asset can be used directly for previewing.
	 */
	virtual UChaosClothAssetBase* CreatePreviewAssetCopy(UObject* Outer, EObjectFlags Flags) const { return nullptr; }

	/**
	 * Create a transient asset with render data built from the source asset's simulation models.
	 * Used for preview rendering when an asset has no render data, or for rendering the
	 * sim model specifically (e.g. Cloth3DSimView).
	 * @param SourceAsset Asset to read simulation models from.
	 * @param Outer The outer object for the preview asset.
	 * @param Flags Object flags for the preview asset.
	 * @param Material Material to apply. Use FClothEngineTools::GetSimPreviewMaterial() for the default.
	 * @return A transient asset, or nullptr if the source has no valid simulation models.
	 */
	static UE_API UChaosClothAssetBase* CreateSimModelPreviewAsset(
		const UChaosClothAssetBase& SourceAsset,
		UObject* Outer,
		EObjectFlags Flags,
		UMaterialInterface* Material);
#endif

	/** Set the specified reference skeleton or a default reference skeleton with a simple root if ReferenceSkeleton is nullptr. */
	UE_API virtual void SetReferenceSkeleton(const FReferenceSkeleton* ReferenceSkeleton);
	//~ End UChaosClothAssetBase interface

	//~ Begin IDataflowContentOwner interface 
	UE_API virtual TObjectPtr<UDataflowBaseContent> CreateDataflowContent() override;
	UE_API virtual void WriteDataflowContent(const TObjectPtr<UDataflowBaseContent>& DataflowContent) const override;
	UE_API virtual void ReadDataflowContent(const TObjectPtr<UDataflowBaseContent>& DataflowContent) override;
	//~ End IDataflowContentOwner interface 

	//~ Begin IDataflowInstanceInterface interface
	UE_API virtual const FDataflowInstance& GetDataflowInstance() const override;
	UE_API virtual FDataflowInstance& GetDataflowInstance() override;
	//~ End IDataflowInstanceInterface interface

	//~ Begin UObject interface
	UE_API virtual void BeginDestroy() override;
	UE_API virtual bool IsReadyForFinishDestroy() override;
	UE_API virtual void Serialize(FArchive& Ar) override;
	UE_API virtual void PostLoad() override;
	UE_API virtual void PostInitProperties() override;
	UE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual void BeginCacheForCookedPlatformData(const ITargetPlatform* TargetPlatform) override;
	UE_API virtual bool IsCachedCookedPlatformDataLoaded(const ITargetPlatform* TargetPlatform) override;
	UE_API virtual void ClearAllCachedCookedPlatformData() override;
#endif
	UE_API virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	//~ End UObject interface

	//~ Begin USkinnedAsset interface
	UE_API virtual FReferenceSkeleton& GetRefSkeleton() override;
	UE_API virtual const FReferenceSkeleton& GetRefSkeleton() const override;
	UE_API virtual FSkeletalMeshLODInfo* GetLODInfo(int32 Index);
	UE_API virtual const FSkeletalMeshLODInfo* GetLODInfo(int32 Index) const;
	UFUNCTION(BlueprintGetter)
	virtual class UPhysicsAsset* GetShadowPhysicsAsset() const override
	{
		return ShadowPhysicsAsset;
	}
	virtual FMatrix GetComposedRefPoseMatrix(FName BoneName) const override
	{
		return FMatrix::Identity;
	}
	virtual FMatrix GetComposedRefPoseMatrix(int32 BoneIndex) const override
	{
		return FMatrix::Identity;
	}
	UE_API virtual const FMeshUVChannelInfo* GetUVChannelData(int32 MaterialIndex) const override;
	virtual bool GetSupportRayTracing() const override
	{
		return bSupportRayTracing;
	}
	virtual int32 GetRayTracingMinLOD() const override
	{
		return RayTracingMinLOD;
	}
	virtual TArray<FMatrix44f>& GetRefBasesInvMatrix() override
	{
		return RefBasesInvMatrix;
	}
	virtual const TArray<FMatrix44f>& GetRefBasesInvMatrix() const override
	{
		return RefBasesInvMatrix;
	}
	virtual TArray<FSkeletalMeshLODInfo>& GetLODInfoArray() override
	{
		return LODInfo;
	}
	virtual const TArray<FSkeletalMeshLODInfo>& GetLODInfoArray() const override
	{
		return LODInfo;
	}
	UE_API virtual FSkeletalMeshRenderData* GetResourceForRendering() const override;
	virtual int32 GetDefaultMinLod() const
	{
		return 0;
	}
	virtual const FPerPlatformInt& GetMinLod() const override
	{
		return MinLod;
	}
	virtual TArray<FSkeletalMaterial>& GetMaterials() override
	{
		return Materials;
	}
	virtual const TArray<FSkeletalMaterial>& GetMaterials() const override
	{
		return Materials;
	}
	virtual bool IsMaterialUsed(int32 MaterialIndex) const override
	{
		return Materials.IsValidIndex(MaterialIndex);
	}
	virtual int32 GetLODNum() const override
	{
		return LODInfo.Num();
	}
	virtual FBoxSphereBounds GetBounds() const override
	{
		return Bounds;
	}
	virtual TArray<class USkeletalMeshSocket*> GetActiveSocketList() const override
	{
		return TArray<class USkeletalMeshSocket*>();
	}
	virtual USkeletalMeshSocket* FindSocket(FName InSocketName) const override
	{
		return nullptr;
	}
	virtual USkeletalMeshSocket* FindSocketInfo(FName InSocketName, FTransform& OutTransform, int32& OutBoneIndex, int32& OutIndex) const override
	{
		return nullptr;
	}
	virtual UMeshDeformer* GetDefaultMeshDeformer() const override
	{
		return nullptr;
	}
	virtual UMeshDeformerCollection* GetTargetMeshDeformers() const override
	{
		return nullptr;
	}
	virtual bool HasHalfEdgeBuffer(int32 LODIndex) const override
	{
		return false;
	}
	
	/** Get the default overlay material used by this mesh */
	UFUNCTION(BlueprintCallable, Category = "Rendering|Material")
	UE_API virtual class UMaterialInterface* GetOverlayMaterial() const override;
	/** Get the default overlay material max draw distance used by this mesh */
	UFUNCTION(BlueprintCallable, Category = "Rendering|Material")
	UE_API virtual float GetOverlayMaterialMaxDrawDistance() const override;
	virtual bool IsValidLODIndex(int32 Index) const override
	{
		return LODInfo.IsValidIndex(Index);
	}
	UE_API virtual int32 GetMinLodIdx(bool bForceLowestLODIdx = false) const override;
	virtual bool NeedCPUData(int32 LODIndex) const override
	{
		return false;
	}
	UE_API virtual bool GetHasVertexColors() const override;
	UE_API virtual int32 GetPlatformMinLODIdx(const ITargetPlatform* TargetPlatform) const override;
	virtual const FPerPlatformBool& GetDisableBelowMinLodStripping() const override
	{
		return DisableBelowMinLodStripping;
	}
	UE_API virtual bool IsMinLodQualityLevelEnable() const override;

#if WITH_EDITOR
	virtual bool GetEnableLODStreaming(const class ITargetPlatform* TargetPlatform) const override
	{
		return false;
	}
	virtual int32 GetMaxNumStreamedLODs(const class ITargetPlatform* TargetPlatform) const override
	{
		return 0;
	}
	virtual int32 GetMaxNumOptionalLODs(const ITargetPlatform* TargetPlatform) const override
	{
		return 0;
	}
#endif
#if WITH_EDITORONLY_DATA
	/** Default GetImportedModel returns the inherited MeshModel; derived classes override to add async-property waits. */
	virtual FSkeletalMeshModel* GetImportedModel() const override
	{
		return MeshModel.Get();
	}
#endif
	//~ End USkinnedAsset interface

	const FPerQualityLevelInt& GetQualityLevelMinLod() const
	{
		return MinQualityLevelLOD;
	}
	void SetQualityLevelMinLod(FPerQualityLevelInt InMinLod)
	{
		MinQualityLevelLOD = MoveTemp(InMinLod);
	}

	/** Change the default overlay material used by this mesh */
	UFUNCTION(BlueprintCallable, Category = "Rendering|Material")
	UE_API void SetOverlayMaterial(class UMaterialInterface* NewOverlayMaterial);
	/** Change the default overlay material max draw distance used by this mesh */
	UFUNCTION(BlueprintCallable, Category = "Rendering|Material")
	UE_API void SetOverlayMaterialMaxDrawDistance(float InMaxDrawDistance);

	/**
	 * Set the Dataflow graph asset for this asset.
	 * @param InDataflow The new dataflow asset.
	 */
	UE_API void SetDataflow(UDataflow* InDataflow);

	/** Return the Dataflow graph asset associated to this asset if any. */
	UE_API UDataflow* GetDataflow();

	/** Return the Dataflow graph asset associated to this asset if any, const version. */
	UE_API const UDataflow* GetDataflow() const;

	/**
	 * Return whether there is a Dataflow graph associated with this asset.
	 * This function can be used in EditCondition.
	 */
	UFUNCTION()
	bool HasDataflow() const
	{
		return GetDataflow() != nullptr;
	}

#if WITH_EDITORONLY_DATA
	UE_API void SetPreviewSceneSkeletalMesh(class USkeletalMesh* Mesh);
	UE_API class USkeletalMesh* GetPreviewSceneSkeletalMesh() const;

	UE_API void SetPreviewSceneAnimation(class UAnimationAsset* Animation);
	UE_API class UAnimationAsset* GetPreviewSceneAnimation() const;

	class UAssetImportData* GetAssetImportData() const
	{
		return AssetImportData;
	}

	void SetAssetImportData(class UAssetImportData* InAssetImportData)
	{
		AssetImportData = InAssetImportData;
	}

	static FName GetAssetImportDataMemberName()
	{
		return GET_MEMBER_NAME_CHECKED(UChaosClothAssetBase, AssetImportData);
	}
#endif

	UE_API void SetHasVertexColors(bool InbHasVertexColors);

#if WITH_EDITOR
	/**
	 * Export the graphical representation of this asset to a SkeletalMesh asset.
	 * Includes skinning, but excludes all clothing simulation data since it isn't compatible with the Cloth Asset.
	 * @param SkeletalMesh The empty skeletal mesh asset to be written to.
	 * @return Whether the initialization of the skeletal mesh has succeeded or not.
	 */
	UE_API bool ExportToSkeletalMesh(USkeletalMesh& SkeletalMesh) const;
#endif

protected:
#if WITH_EDITOR
	/** DDC key prefix that distinguishes this asset type's cache entries, e.g. "CHAOSOUTFIT", "CHAOSCLOTH". */
	virtual const TCHAR* GetDerivedDataPrefix() const
	PURE_VIRTUAL(UChaosClothAssetBase::GetDerivedDataPrefix, return TEXT(""););

	/** DDC key version GUID; bump to invalidate cached entries when the on-disk layout changes. */
	virtual const TCHAR* GetDerivedDataVersion() const
	PURE_VIRTUAL(UChaosClothAssetBase::GetDerivedDataVersion, return TEXT(""););

	/** Append build settings to a DDC key suffix: GPU bone limits, bone influences, LODInfo GUIDs. */
	UE_API void AppendBuildSettingsToDDCKey(FString& InOutKeySuffix, const ITargetPlatform* TargetPlatform);

	/** Store the current running-platform render data to DDC and set the head DerivedDataKey. */
	UE_API void StoreDerivedData();

	/** True once the head render data carries a DerivedDataKey: set by Build->Store, DDC hit, or cooked-load. */
	UE_API virtual bool IsInitialBuildDone() const override;

	/** Prepare an empty FSkeletalMeshModel sized to the current head render data, used by BuildLODModel on a Cache miss. */
	UE_API virtual void PrepareMeshModel();

	/** Maintain the cooked-load head invariant: head holds running-platform render data with matching DerivedDataKey. */
	UE_API void EstablishCookedHeadKey();

	/** Cooked-save render data switch on Ar.CookingTarget(); builds the per-platform variant on miss. */
	UE_API FSkeletalMeshRenderData* GetSerializeRenderData(FArchive& Ar);

	/** Asset-specific BeginPostLoadInternal work, called on the game thread. */
	virtual void BeginPostLoadAssetImpl(FSkinnedAssetPostLoadContext& Context) {}

	/** Asset-specific ExecutePostLoadInternal work, may run on a worker thread. */
	virtual void ExecutePostLoadAssetImpl(FSkinnedAssetPostLoadContext& Context) {}
#endif

	/** Resolve the names of base-class async property bits (bits 0-31) for diagnostic logging. */
	UE_API virtual FString GetAsyncPropertyName(uint64 Property) const override;

	/** Update all properties for components currently using this asset. */
	UE_API void OnPropertyChanged(const bool bReregisterComponents = false) const;

	/** Update all components currently using this asset. */
	UE_API void OnAssetChanged(const bool bReregisterComponents = true) const;

	/** Reregister all components using this asset to reset the simulation in case anything has changed. */
	UE_DEPRECATED(5.7, "Use OnAssetChanged instead.")
	UE_API void ReregisterComponents() const;

	/** Return all components currently using this asset. */
	UE_API TArray<UActorComponent*> GetDependentComponents() const;

	/** Lock properties that should not be modified/accessed during async build. */
	template <typename EnumType UE_REQUIRES(TIsEnum<EnumType>::Value)>
	void AcquireAsyncProperty(const EnumType AsyncProperties, ESkinnedAssetAsyncPropertyLockType LockType = ESkinnedAssetAsyncPropertyLockType::ReadWrite)
	{
		checkf((uint64)AsyncProperties <= (uint64)TNumericLimits<uint32>::Max(), TEXT("Cannot have more than 32 async properties per derived class."));
		Super::AcquireAsyncProperty((uint64)AsyncProperties << 32ull, LockType);
	}

	/** Release properties that should not be modified/accessed during async build. */
	template <typename EnumType UE_REQUIRES(TIsEnum<EnumType>::Value)>
	void ReleaseAsyncProperty(const EnumType AsyncProperties, ESkinnedAssetAsyncPropertyLockType LockType = ESkinnedAssetAsyncPropertyLockType::ReadWrite)
	{
		checkf((uint64)AsyncProperties <= (uint64)TNumericLimits<uint32>::Max(), TEXT("Cannot have more than 32 async properties per derived class."));
		Super::ReleaseAsyncProperty((uint64)AsyncProperties << 32ull, LockType);
	}

	/**
	 * Wait for the asset to finish compilation to protect internal skinned asset data from race conditions during async build.
	 * This should be called before accessing all async accessible properties.
	 */
	template <typename EnumType UE_REQUIRES(TIsEnum<EnumType>::Value)>
	void WaitUntilAsyncPropertyReleased(const EnumType AsyncProperties, ESkinnedAssetAsyncPropertyLockType LockType = ESkinnedAssetAsyncPropertyLockType::ReadWrite) const
	{
		checkf((uint64)AsyncProperties <= (uint64)TNumericLimits<uint32>::Max(), TEXT("Cannot have more than 32 async properties per derived class."));
		WaitUntilAsyncPropertyReleasedInternal((uint64)AsyncProperties << 32ull, LockType);
	}

	/** Set render data. */
	UE_API void SetResourceForRendering(TUniquePtr<FSkeletalMeshRenderData>&& InSkeletalMeshRenderData);

	/** Initialize all render resources. */
	UE_API void InitResources();

	/** Safely release the render data. */
	UE_API void ReleaseResources();

	/** Pre-calculate refpose-to-local transforms. */
	UE_API void CalculateInvRefMatrices();

	/** Calculate bounds from the render data's position vertex buffer. */
	UE_API void CalculateBounds();

	UE_DEPRECATED(5.6, "This property isn't deprecated, but the proper getter and setter should be used instead in order to preserve correct behavior.")
	UPROPERTY(EditAnywhere, Category = "Dataflow")
	FDataflowInstance DataflowInstance;

	/** List of materials for this cloth asset. Set by the Dataflow evaluation. */
	UPROPERTY(EditAnywhere, Category = Materials, Meta = (EditCondition = "!HasDataflow()"))
	TArray<FSkeletalMaterial> Materials;

	/** Struct containing information for each LOD level, such as materials to use, and when use the LOD. Not currently editable or customizable through the Dataflow. */
	UPROPERTY(VisibleAnywhere, EditFixedSize, Category = LevelOfDetails)
	TArray<FSkeletalMeshLODInfo> LODInfo;

	/** Set the Minimum LOD by Quality Level. This property is used when "Use Cloth Asset Min LOD Per Quality Levels" is set at the Project level. Otherwise, the (per platform) Minimum LOD value is used.*/
	UPROPERTY(EditAnywhere, Category = LODSettings, meta = (DisplayName = "Quality Level Minimum LOD"))
	FPerQualityLevelInt MinQualityLevelLOD = 0;

	UPROPERTY(EditAnywhere, Category = LODSettings)
	FPerPlatformBool DisableBelowMinLodStripping;

	/** Set the Minimum LOD by platform. This property is overridden by "Quality Level Minimum LOD" when "Use Cloth Asset Min LOD Per Quality Levels" is set at the Project level.*/
	UPROPERTY(EditAnywhere, Category = LODSettings, Meta = (DisplayName = "Minimum LOD"))
	FPerPlatformInt MinLod = 0;

	/** Enable raytracing for this asset. */
	UPROPERTY(EditAnywhere, Category = RayTracing)
	uint8 bSupportRayTracing : 1;

	/** Minimum raytracing LOD for this asset. */
	UPROPERTY(EditAnywhere, Category = RayTracing)
	int32 RayTracingMinLOD;

	/**
	 * Physics asset whose shapes will be used for shadowing when components have bCastCharacterCapsuleDirectShadow or bCastCharacterCapsuleIndirectShadow enabled.
	 * Only spheres and sphyl shapes in the physics asset can be supported.  The more shapes used, the higher the cost of the capsule shadows will be.
	 */
	UPROPERTY(EditAnywhere, AssetRegistrySearchable, BlueprintGetter = GetShadowPhysicsAsset, Category = Lighting)
	TObjectPtr<class UPhysicsAsset> ShadowPhysicsAsset;

	/** Default translucent material to blend on top of this mesh. Mesh will be rendered twice - once with a base material and once with overlay material */
	UE_DEPRECATED(5.6, "This must be protected for async build, always use the accessors even internally.")
	UPROPERTY(EditAnywhere, Category = Rendering)
	TObjectPtr<class UMaterialInterface> OverlayMaterial;

	/** Default max draw distance for overlay material. A distance of 0 indicates that overlay will be culled using primitive max distance. */
	UE_DEPRECATED(5.6, "This must be protected for async build, always use the accessors even internally.")
	UPROPERTY(EditAnywhere, Category = Rendering)
	float OverlayMaterialMaxDrawDistance;

	UE_DEPRECATED(5.6, "This must be protected for async build, always use the accessors even internally.")
	UPROPERTY()
	uint8 bHasVertexColors : 1 = false;

	/** Reference skeleton created from the provided skeleton asset. */
	UE_DEPRECATED(5.4, "This must be protected for async build, always use the accessors even internally.")
	FReferenceSkeleton RefSkeleton;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.8, "Use HasDataflow instead.")
	UPROPERTY()
	bool bHasDataflowAsset = false;

	/** Transient source mesh model used to feed BuildLODModel during Cache(); built in PrepareMeshModel() and reset once GetPlatformSkeletalMeshRenderData() finishes caching. */
	TSharedPtr<FSkeletalMeshModel> MeshModel;
#endif

	/** Bounds for this asset. */
	UPROPERTY()
	FBoxSphereBounds Bounds;

	/** A fence which is used to keep track of the rendering thread releasing the static mesh resources. */
	FRenderCommandFence ReleaseResourcesFence;

private:
	/** Lock base-class async property bits (0-31) directly, without the derived-class left-shift by 32. */
	template <>
	void AcquireAsyncProperty<EChaosClothAssetBaseAsyncProperties>(const EChaosClothAssetBaseAsyncProperties AsyncProperties, ESkinnedAssetAsyncPropertyLockType LockType)
	{
		Super::AcquireAsyncProperty((uint64)AsyncProperties, LockType);
	}

	/** Release base-class async property bits (0-31) directly, without the derived-class left-shift by 32. */
	template <>
	void ReleaseAsyncProperty<EChaosClothAssetBaseAsyncProperties>(const EChaosClothAssetBaseAsyncProperties AsyncProperties, ESkinnedAssetAsyncPropertyLockType LockType)
	{
		Super::ReleaseAsyncProperty((uint64)AsyncProperties, LockType);
	}

	/** Wait for base-class async property bits (0-31) directly, without the derived-class left-shift by 32. */
	template <>
	void WaitUntilAsyncPropertyReleased<EChaosClothAssetBaseAsyncProperties>(const EChaosClothAssetBaseAsyncProperties AsyncProperties, ESkinnedAssetAsyncPropertyLockType LockType) const
	{
		WaitUntilAsyncPropertyReleasedInternal((uint64)AsyncProperties, LockType);
	}

	//~ Begin USkinnedAsset interface
	UE_API virtual void BeginPostLoadInternal(FSkinnedAssetPostLoadContext& Context) override;
	UE_API virtual void ExecutePostLoadInternal(FSkinnedAssetPostLoadContext& Context) override;
	/** Complete the postload process - shared by all cloth asset types. */
	UE_API virtual void FinishPostLoadInternal(FSkinnedAssetPostLoadContext& Context) override;
	//~ End USkinnedAsset interface

	/** Update simulation actor/components from the asset data */
	void UpdateSimulationActor(TObjectPtr<AActor>& SimulationActor) const;

#if WITH_EDITOR
	/** Register the OnPackageReloaded delegate to call OnAssetChanged() when reloading this asset. */
	void RegisterOnPackageReloadedDelegate();
	/** Unregister the OnPackageReloaded delegate. */
	void UnregisterOnPackageReloadedDelegate();

	/** Get or build platform-specific render data for cooking via the linked-list pattern. */
	UE_API FSkeletalMeshRenderData& GetPlatformSkeletalMeshRenderData(const ITargetPlatform* TargetPlatform, bool bIsSerializeSaving);
#endif

#if WITH_EDITORONLY_DATA
	/** Helper function to move deprecated Cloth Editor preview assets to the new BP variable system. */
	template<typename T>
	void SyncBPPreviewAsset(TSoftObjectPtr<T>& PreviewAsset, const TCHAR* const VarName);

	// The following PreviewScene properties are modeled after PreviewSkeletalMesh in USkeleton
	//	- they are inside WITH_EDITORONLY_DATA because they are not used at game runtime
	//	- TSoftObjectPtrs since that will make it possible to avoid loading these assets until the PreviewScene asks for them
	//	- DuplicateTransient so that if you copy a ClothAsset it won't copy these preview properties
	//	- AssetRegistrySearchable makes it so that if the user searches the name of a PreviewScene asset in the Asset Browser, it will return any ClothAssets that use it

	/** Optional Skeletal Mesh that the cloth asset is attached to in the Preview Scene in the Cloth Editor */
	UPROPERTY(DuplicateTransient, AssetRegistrySearchable)
	TSoftObjectPtr<class USkeletalMesh> PreviewSceneSkeletalMesh;

	/** Optional animation attached to PreviewSceneSkeletalMesh in the Preview Scene in the Cloth Editor */
	UPROPERTY(DuplicateTransient, AssetRegistrySearchable)
	TSoftObjectPtr<class UAnimationAsset> PreviewSceneAnimation;

	/** Asset data to be used when re-importing */
	UPROPERTY(VisibleAnywhere, Instanced, Category = ImportSettings)
	TObjectPtr<class UAssetImportData> AssetImportData;
#endif

	/** Rendering data. */
	TUniquePtr<FSkeletalMeshRenderData> SkeletalMeshRenderData;

	/** Reference skeleton precomputed bases. */
	TArray<FMatrix44f> RefBasesInvMatrix;

	/** Mesh-space ref pose, where parent matrices are applied to ref pose matrices. */
	TArray<FMatrix> CachedComposedRefPoseMatrices;
};

#undef UE_API
