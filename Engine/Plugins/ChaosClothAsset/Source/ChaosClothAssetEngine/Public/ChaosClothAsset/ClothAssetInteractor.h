// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ClothingSimulationInteractor.h"
#include "CoreTypes.h"
#include "Math/MathFwd.h"
#include "ClothAssetInteractor.generated.h"

namespace Chaos::Softs
{
class FCollectionPropertyFacade;
}
struct FClothAssetInteractorPropertyBag;

UCLASS(BlueprintType, MinimalAPI)
class UChaosClothAssetInteractor : public UClothingInteractor
{
	GENERATED_BODY()
public:

	/** Set properties this interactor references.*/
	CHAOSCLOTHASSETENGINE_API void SetProperties(const TArray<TSharedPtr<::Chaos::Softs::FCollectionPropertyFacade>>& InCollectionPropertyFacades);

	/** Empty references to all properties.*/
	CHAOSCLOTHASSETENGINE_API void ResetProperties();

	/** Generate a list of all properties held by this interactor. Properties for all LODs will be returned if LODIndex = -1.*/
	UFUNCTION(BlueprintCallable, Category = ClothProperty)
	CHAOSCLOTHASSETENGINE_API TArray<FName> GetAllPropertyNames(int32 LODIndex = -1) const;

	/** Get the value for a property cast to float. DefaultValue will be returned if the property is not found.*/
	UFUNCTION(BlueprintCallable, Category = ClothProperty)
	CHAOSCLOTHASSETENGINE_API float GetFloatPropertyValue(const FName PropertyName, int32 LODIndex = 0, float DefaultValue = 0.f) const;

	/** Get the low value for a weighted property value (same as GetFloatValue). DefaultValue will be returned if the property is not found.*/
	UFUNCTION(BlueprintCallable, Category = ClothProperty)
	CHAOSCLOTHASSETENGINE_API float GetLowFloatPropertyValue(const FName PropertyName, int32 LODIndex = 0, float DefaultValue = 0.f) const;

	/** Get the high value for a weighted property value. DefaultValue will be returned if the property is not found.*/
	UFUNCTION(BlueprintCallable, Category = ClothProperty)
	CHAOSCLOTHASSETENGINE_API float GetHighFloatPropertyValue(const FName PropertyName, int32 LODIndex = 0, float DefaultValue = 0.f) const;

	/** Get the low and high values for a weighted property value. DefaultValue will be returned if the property is not found.*/
	UFUNCTION(BlueprintCallable, Category = ClothProperty)
	CHAOSCLOTHASSETENGINE_API FVector2D GetWeightedFloatPropertyValue(const FName PropertyName, int32 LODIndex = 0, FVector2D DefaultValue = FVector2D(0., 0.)) const;

	/** Get the value for a property cast to int. DefaultValue will be returned if the property is not found.*/
	UFUNCTION(BlueprintCallable, Category = ClothProperty)
	CHAOSCLOTHASSETENGINE_API int32 GetIntPropertyValue(const FName PropertyName, int32 LODIndex = 0, int32 DefaultValue = 0) const;

	/** Get the value for a property cast to vector. DefaultValue will be returned if the property is not found.*/
	UFUNCTION(BlueprintCallable, Category = ClothProperty)
	CHAOSCLOTHASSETENGINE_API FVector GetVectorPropertyValue(const FName PropertyName, int32 LODIndex = 0, FVector DefaultValue = FVector(0.)) const;

	/** Get the string value for a property (typically the associated map name for weighted values). DefaultValue will be returned if the property is not found.*/
	UFUNCTION(BlueprintCallable, Category = ClothProperty)
	CHAOSCLOTHASSETENGINE_API FString GetStringPropertyValue(const FName PropertyName, int32 LODIndex = 0, const FString& DefaultValue = "") const;

	/** Set properties from an interactor data set */
	UFUNCTION(BlueprintCallable, Category = ClothProperty)
	CHAOSCLOTHASSETENGINE_API void SetPropertySet(const FClothAssetInteractorPropertyBag& PropertySet, int32 LODIndex = -1);

	/**Set the value for a property (if it exists). This sets the Low and High values for weighted values. All LODs will be set when LODIndex = -1.*/
	UFUNCTION(BlueprintCallable, Category = ClothProperty)
	CHAOSCLOTHASSETENGINE_API void SetFloatPropertyValue(const FName PropertyName, int32 LODIndex = -1, float Value = 0.f);

	/**Set the low value for a weighted property (if it exists). All LODs will be set when LODIndex = -1.*/
	UFUNCTION(BlueprintCallable, Category = ClothProperty)
	CHAOSCLOTHASSETENGINE_API void SetLowFloatPropertyValue(const FName PropertyName, int32 LODIndex = -1, float Value = 0.f);

	/**Set the high value for a weighted property (if it exists). All LODs will be set when LODIndex = -1.*/
	UFUNCTION(BlueprintCallable, Category = ClothProperty)
	CHAOSCLOTHASSETENGINE_API void SetHighFloatPropertyValue(const FName PropertyName, int32 LODIndex = -1, float Value = 0.f);

	/**Set the low and high values for a weighted property (if it exists). All LODs will be set when LODIndex = -1.*/
	UFUNCTION(BlueprintCallable, Category = ClothProperty)
	CHAOSCLOTHASSETENGINE_API void SetWeightedFloatPropertyValue(const FName PropertyName, int32 LODIndex = -1, FVector2D Value = FVector2D(0., 0.));

	/**Set the value for a property (if it exists). All LODs will be set when LODIndex = -1.*/
	UFUNCTION(BlueprintCallable, Category = ClothProperty)
	CHAOSCLOTHASSETENGINE_API void SetIntPropertyValue(const FName PropertyName, int32 LODIndex = -1, int32 Value = 0);

	/**Set the value for a property (if it exists). All LODs will be set when LODIndex = -1.*/
	UFUNCTION(BlueprintCallable, Category = ClothProperty)
	CHAOSCLOTHASSETENGINE_API void SetVectorPropertyValue(const FName PropertyName, int32 LODIndex = -1, FVector Value = FVector(0.));

	/**Set the string value for a property (if it exists). This is typically the map name associated with a property. All LODs will be set when LODIndex = -1.*/
	UFUNCTION(BlueprintCallable, Category = ClothProperty)
	CHAOSCLOTHASSETENGINE_API void SetStringPropertyValue(const FName PropertyName, int32 LODIndex = -1, const FString& Value = "");

	// Deprecated string-based key versions.
	/** Generate a list of all properties held by this interactor. Properties for all LODs will be returned if LODIndex = -1.*/
	UFUNCTION(BlueprintCallable, meta = (DeprecatedFunction = "Note", DeprecationMessage = "Use GetAllPropertyNames instead"))
	CHAOSCLOTHASSETENGINE_API TArray<FString> GetAllProperties(int32 LODIndex = -1) const;

	/** Get the value for a property cast to float. DefaultValue will be returned if the property is not found.*/
	UFUNCTION(BlueprintCallable, Category = "ClothProperty|Deprecated", meta = (DeprecatedFunction = "Note", DeprecationMessage = "Use GetFloatPropertyValue instead"))
	UE_DEPRECATED(5.7, "Use GetFloatPropertyValue instead")
	float GetFloatValue(const FString& PropertyName, int32 LODIndex = 0, float DefaultValue = 0.f) const
	{
		return GetFloatPropertyValue(FName(PropertyName), LODIndex, DefaultValue);
	}

	/** Get the low value for a weighted property value (same as GetFloatValue). DefaultValue will be returned if the property is not found.*/
	UFUNCTION(BlueprintCallable, Category = "ClothProperty|Deprecated", meta = (DeprecatedFunction = "Note", DeprecationMessage = "Use GetLowFloatPropertyValue instead"))
	UE_DEPRECATED(5.7, "Use GetLowFloatPropertyValue instead")
	float GetLowFloatValue(const FString& PropertyName, int32 LODIndex = 0, float DefaultValue = 0.f) const
	{
		return GetLowFloatPropertyValue(FName(PropertyName), LODIndex, DefaultValue);
	}

	/** Get the high value for a weighted property value. DefaultValue will be returned if the property is not found.*/
	UFUNCTION(BlueprintCallable, Category = "ClothProperty|Deprecated", meta = (DeprecatedFunction = "Note", DeprecationMessage = "Use GetHighFloatPropertyValue instead"))
	UE_DEPRECATED(5.7, "Use GetHighFloatPropertyValue instead")
	float GetHighFloatValue(const FString& PropertyName, int32 LODIndex = 0, float DefaultValue = 0.f) const
	{
		return GetHighFloatPropertyValue(FName(PropertyName), LODIndex, DefaultValue);
	}

	/** Get the low and high values for a weighted property value. DefaultValue will be returned if the property is not found.*/
	UFUNCTION(BlueprintCallable, Category = "ClothProperty|Deprecated", meta = (DeprecatedFunction = "Note", DeprecationMessage = "Use GetWeightedFloatPropertyValue instead"))
	UE_DEPRECATED(5.7, "Use GetWeightedFloatPropertyValue instead")
	FVector2D GetWeightedFloatValue(const FString& PropertyName, int32 LODIndex = 0, FVector2D DefaultValue = FVector2D(0., 0.)) const
	{
		return GetWeightedFloatPropertyValue(FName(PropertyName), LODIndex, DefaultValue);
	}

	/** Get the value for a property cast to int. DefaultValue will be returned if the property is not found.*/
	UFUNCTION(BlueprintCallable, Category = "ClothProperty|Deprecated", meta = (DeprecatedFunction = "Note", DeprecationMessage = "Use GetIntPropertyValue instead"))
	UE_DEPRECATED(5.7, "Use GetIntPropertyValue instead")
	int32 GetIntValue(const FString& PropertyName, int32 LODIndex = 0, int32 DefaultValue = 0) const
	{
		return GetIntPropertyValue(FName(PropertyName), LODIndex, DefaultValue);
	}

	/** Get the value for a property cast to vector. DefaultValue will be returned if the property is not found.*/
	UFUNCTION(BlueprintCallable, Category = "ClothProperty|Deprecated", meta = (DeprecatedFunction = "Note", DeprecationMessage = "Use GetVectorPropertyValue instead"))
	UE_DEPRECATED(5.7, "Use GetVectorPropertyValue instead")
	FVector GetVectorValue(const FString& PropertyName, int32 LODIndex = 0, FVector DefaultValue = FVector(0.)) const
	{
		return GetVectorPropertyValue(FName(PropertyName), LODIndex, DefaultValue);
	}

	/** Get the string value for a property (typically the associated map name for weighted values). DefaultValue will be returned if the property is not found.*/
	UFUNCTION(BlueprintCallable, Category = "ClothProperty|Deprecated", meta = (DeprecatedFunction = "Note", DeprecationMessage = "Use GetStringPropertyValue instead"))
	UE_DEPRECATED(5.7, "Use GetStringPropertyValue instead")
	FString GetStringValue(const FString& PropertyName, int32 LODIndex = 0, const FString& DefaultValue = "") const
	{
		return GetStringPropertyValue(FName(PropertyName), LODIndex, DefaultValue);
	}

	/**Set the value for a property (if it exists). This sets the Low and High values for weighted values. All LODs will be set when LODIndex = -1.*/
	UFUNCTION(BlueprintCallable, Category = "ClothProperty|Deprecated", meta = (DeprecatedFunction = "Note", DeprecationMessage = "Use SetFloatPropertyValue instead"))
	UE_DEPRECATED(5.7, "Use SetFloatPropertyValue instead")
	void SetFloatValue(const FString& PropertyName, int32 LODIndex = -1, float Value = 0.f)
	{
		SetFloatPropertyValue(FName(PropertyName), LODIndex, Value);
	}

	/**Set the low value for a weighted property (if it exists). All LODs will be set when LODIndex = -1.*/
	UFUNCTION(BlueprintCallable, Category = "ClothProperty|Deprecated", meta = (DeprecatedFunction = "Note", DeprecationMessage = "Use SetLowFloatPropertyValue instead"))
	UE_DEPRECATED(5.7, "Use SetLowFloatPropertyValue instead")
	void SetLowFloatValue(const FString& PropertyName, int32 LODIndex = -1, float Value = 0.f)
	{
		SetLowFloatPropertyValue(FName(PropertyName), LODIndex, Value);
	}

	/**Set the high value for a weighted property (if it exists). All LODs will be set when LODIndex = -1.*/
	UFUNCTION(BlueprintCallable, Category = "ClothProperty|Deprecated", meta = (DeprecatedFunction = "Note", DeprecationMessage = "Use SetHighFloatPropertyValue instead"))
	UE_DEPRECATED(5.7, "Use SetHighFloatPropertyValue instead")
	void SetHighFloatValue(const FString& PropertyName, int32 LODIndex = -1, float Value = 0.f)
	{
		SetHighFloatPropertyValue(FName(PropertyName), LODIndex, Value);
	}

	/**Set the low and high values for a weighted property (if it exists). All LODs will be set when LODIndex = -1.*/
	UFUNCTION(BlueprintCallable, Category = "ClothProperty|Deprecated", meta = (DeprecatedFunction = "Note", DeprecationMessage = "Use SetWeightedFloatPropertyValue instead"))
	UE_DEPRECATED(5.7, "Use SetWeightedFloatPropertyValue instead")
	void SetWeightedFloatValue(const FString& PropertyName, int32 LODIndex = -1, FVector2D Value = FVector2D(0., 0.))
	{
		SetWeightedFloatPropertyValue(FName(PropertyName), LODIndex, Value);
	}

	/**Set the value for a property (if it exists). All LODs will be set when LODIndex = -1.*/
	UFUNCTION(BlueprintCallable, Category = "ClothProperty|Deprecated", meta = (DeprecatedFunction = "Note", DeprecationMessage = "Use SetIntPropertyValue instead"))
	UE_DEPRECATED(5.7, "Use SetIntPropertyValue instead")
	void SetIntValue(const FString& PropertyName, int32 LODIndex = -1, int32 Value = 0)
	{
		SetIntPropertyValue(FName(PropertyName), LODIndex, Value);
	}

	/**Set the value for a property (if it exists). All LODs will be set when LODIndex = -1.*/
	UFUNCTION(BlueprintCallable, Category = "ClothProperty|Deprecated", meta = (DeprecatedFunction = "Note", DeprecationMessage = "Use SetVectorPropertyValue instead"))
	UE_DEPRECATED(5.7, "Use SetVectorPropertyValue instead")
	void SetVectorValue(const FString& PropertyName, int32 LODIndex = -1, FVector Value = FVector(0.))
	{
		SetVectorPropertyValue(FName(PropertyName), LODIndex, Value);
	}

	/**Set the string value for a property (if it exists). This is typically the map name associated with a property. All LODs will be set when LODIndex = -1.*/
	UFUNCTION(BlueprintCallable, Category = "ClothProperty|Deprecated", meta = (DeprecatedFunction = "Note", DeprecationMessage = "Use SetStringPropertyValue instead"))
	UE_DEPRECATED(5.7, "Use SetStringPropertyValue instead")
	void SetStringValue(const FString& PropertyName, int32 LODIndex = -1, const FString& Value = "")
	{
		SetStringPropertyValue(FName(PropertyName), LODIndex, Value);
	}

private:
	TArray<TWeakPtr<::Chaos::Softs::FCollectionPropertyFacade>> CollectionPropertyFacades;
};