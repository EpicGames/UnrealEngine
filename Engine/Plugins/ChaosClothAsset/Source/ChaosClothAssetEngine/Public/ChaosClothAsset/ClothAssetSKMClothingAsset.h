// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ClothingAssetBase.h"
#include "ClothAssetSKMClothingAsset.generated.h"

#define UE_API CHAOSCLOTHASSETENGINE_API

struct FManagedArrayCollection;
struct FChaosClothSimulationModel;
class UChaosClothAssetBase;

/**
 * Clothing Data implementation for the Cloth/Outfit Asset.
 */
UCLASS(BlueprintType, MinimalAPI)
class UChaosClothAssetSKMClothingAsset final : public UClothingAssetBase
{
	GENERATED_BODY()
public:
	UE_API UChaosClothAssetSKMClothingAsset(const FObjectInitializer& ObjectInitializer);

#if WITH_EDITOR
	/**
	 * Assign a Cloth or an Outfit Asset to this clothing data.
	 * @note This is only possible WITH_EDITOR since the Skeletal Mesh might need to be rebuild when the asset is changed.
	 * @param InAsset The new asset to set.
	 */
	UE_API void SetAsset(const UChaosClothAssetBase* InAsset);

	/**
	 * Call this function whenever the Cloth or Outfit Asset has changed (e.g. once rebuilt in Dataflow).
	 * @param bReregisterComponents Whether to reregister the dependent components. When set to false, the registration must be carried outside of this function call.
	 */
	UE_API void OnAssetChanged(const bool bReregisterComponents = true);
#endif
	
	/** Return the underlaying Cloth or Outfit Asset used in this clothing data. */
	const UChaosClothAssetBase* GetAsset() const
	{
		return Asset;
	}

	/** Return the index of the simulation model used from the Cloth or Outfit Asset. */
	int32 GetClothSimulationModelIndex() const
	{
		return ClothSimulationModelIndex;
	}

	const TSharedPtr<const FChaosClothSimulationModel>& GetTransientSimulationModel() const;

	bool HasAnySimulationMeshData(const int32 LODIndex) const;

private:
	//~ Begin UObject Interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject Interface

	//~ Begin UClothingAssetBase Interface
	virtual bool BindToSkeletalMesh(USkeletalMesh* SkeletalMesh, const int32 MeshLodIndex, const int32 SectionIndex, const int32 AssetLodIndex) override;
	virtual void UnbindFromSkeletalMesh(USkeletalMesh* SkeletalMesh, const int32 MeshLodIndexx, const int32 SectionIndex) override;
	virtual void UpdateAllLODBiasMappings(USkeletalMesh* SkeletalMesh) override;
	virtual void GetSimulationMesh(const int32 InMeshLodIndex, TArray<FVector3f>& OutPositions, TArray<uint32>& OutIndices, TArray<float>& OutMaxDistances) const override;
	virtual TArray<TPair<FText, FText>> GetStats() const override;
	virtual USkinnedAsset* GetSkinnedAssetDependency() const override;
#endif  // #if WITH_EDITOR
	virtual void RefreshBoneMapping(USkeletalMesh* SkeletalMesh) override;
	virtual bool IsValid() const override
	{
		return GetClothSimulationModelIndex() != INDEX_NONE;
	}
	virtual void PostUpdateAllAssets() override {}
	//~ End UClothingAssetBase Interface

#if WITH_EDITOR
	void OnModelChanged();

	FName GetClothSimulationModelName() const;
	FGuid GetClothSimulationModelGuid() const;
#endif  // #if WITH_EDITOR

	UFUNCTION(CallInEditor)
	TArray<FString> GetClothSimulationModelIds() const;

	/** Cloth or Outfit Asset to use with this Skeletal Mesh. */
	UPROPERTY(EditAnywhere, Category = "Chaos Cloth Asset")
	TObjectPtr<const UChaosClothAssetBase> Asset;

#if WITH_EDITORONLY_DATA
	/**
	 * The Simulation Model to use for this Clothing Data. Useful for Outfit Assets that may contain multiple models.
	 * The name is followed by the model GUID, since it is technically possible to have two pieces of clothing with the same name in an Outfit Asset.
	 */
	UPROPERTY(EditAnywhere, Category = "Chaos Cloth Asset", Meta = (DisplayName = "Cloth Simulation Model", GetOptions = "GetClothSimulationModelIds"))
	FString ClothSimulationModelId;
#endif  // #if WITH_EDITORONLY_DATA

	UPROPERTY()
	int32 ClothSimulationModelIndex = INDEX_NONE;

	TSharedPtr<const FChaosClothSimulationModel> TransientSimulationModel;

};

#undef UE_API
