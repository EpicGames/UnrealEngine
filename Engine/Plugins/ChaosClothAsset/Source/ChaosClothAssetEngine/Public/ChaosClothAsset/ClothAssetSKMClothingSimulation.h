// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosClothAsset/ClothComponentAdapter.h"
#include "ChaosClothAsset/ClothSimulationProxy.h"
#include "ClothingSimulationInterface.h"
#include "ClothingSimulationFactory.h"
#include "UObject/GCObject.h"
#include "ClothAssetSKMClothingSimulation.generated.h"

class USkinnedMeshComponent;
class UChaosClothAssetInteractor;
class UChaosClothAssetSKMClothingAsset;
class UChaosClothAssetSKMClothingSimulationInteractor;
struct FChaosClothSimulationProperties;

namespace Chaos
{
	class FClothVisualizationNoGC;
	namespace Softs
	{
		class FCollectionPropertyFacade;
	}
}

namespace UE::Chaos::ClothAsset
{

	/** Context for the SKM simulation, only used to temporarily store the simulation timestep, the actual simulation context is managed by the simulation proxy. */
	class FSKMClothingSimulationContext final: public IClothingSimulationContext
	{
	private:
		friend class FSKMClothingSimulation;

		FSKMClothingSimulationContext() : IClothingSimulationContext() {}
		virtual ~FSKMClothingSimulationContext() override = default;

		/** Simulation timestep. */
		float DeltaTime = 0.f;
	};

	/** Skeletal Mesh simulation class that interfaces between the Skeletal Mesh Component and the Cloth Simulation Proxy. */
	class FSKMClothingSimulation final
		: public IClothingSimulationInterface
		, public IClothComponentAdapter
		, public FGCObject
	{
	public:
		FSKMClothingSimulation();
		virtual ~FSKMClothingSimulation() override;

		/** Reset all cloth simulation config properties to the values stored in the original cloth asset. Game thread only. */
		void ResetConfigProperties();

		/** Hard reset the cloth simulation by recreating the proxy. */
		void RecreateClothSimulationProxy();

		/** Return a property interactor for the specified asset and model. */
		UChaosClothAssetInteractor* GetPropertyInteractor(const UChaosClothAssetBase* Asset, int32 ModelIndex) const;

		/** Return the visualization object for this simulation. */
		const ::Chaos::FClothVisualizationNoGC* GetClothVisualization() const
		{
			return ClothSimulationProxy.IsValid() ? ClothSimulationProxy->GetClothVisualization() : nullptr;
		}

	private:
		friend UChaosClothAssetSKMClothingSimulationInteractor;  // For the SetNumIterations/SetNumSubsteps methods

		void SetNumIterations(int32 NumIterations);
		void SetMaxNumIterations(int32 MaxNumIterations);
		void SetNumSubsteps(int32 NumSubsteps);

		//~ Begin IClothingSimulationInterface Interface
		virtual void Initialize() override;
		virtual void Shutdown() override;

		virtual IClothingSimulationContext* CreateContext() override
		{
			return new FSKMClothingSimulationContext();
		}
		virtual void FillContextAndPrepareTick(const USkeletalMeshComponent* InComponent, float InDeltaTime, IClothingSimulationContext* InOutContext, bool bIsInitialization, bool bForceTeleportResetOnly) override;
		virtual void DestroyContext(IClothingSimulationContext* Context) override
		{
			delete Context;
		}

		virtual void CreateActor(USkeletalMeshComponent* InOwnerComponent, const UClothingAssetBase* ClothingAsset, int32 SimDataIndex) override;
		virtual void EndCreateActor() override;
		virtual void DestroyActors() override;

		virtual bool ShouldSimulateLOD(int32 OwnerLODIndex) const override;
		virtual void Simulate_AnyThread(const IClothingSimulationContext* Context) override;
		virtual void ForceClothNextUpdateTeleportAndReset_AnyThread() override;
		virtual void HardResetSimulation(const IClothingSimulationContext* InContext) override;
		virtual void AppendSimulationData(TMap<int32, FClothSimulData>& OutData, const USkeletalMeshComponent* OwnerComponent, const USkinnedMeshComponent* OverrideComponent) const override;

		virtual FBoxSphereBounds GetBounds(const USkeletalMeshComponent* OwnerComponent) const override;

		virtual void AddExternalCollisions(const FClothCollisionData& Data) override;
		virtual void ClearExternalCollisions() override;
		virtual int32 GetNumCloths() const override { return ClothSimulationProxy.IsValid() ? ClothSimulationProxy->GetNumCloths() : 0; }
		virtual int32 GetNumKinematicParticles() const override { return ClothSimulationProxy.IsValid() ? ClothSimulationProxy->GetNumKinematicParticles() : 0; }
		virtual int32 GetNumDynamicParticles() const override { return ClothSimulationProxy.IsValid() ? ClothSimulationProxy->GetNumDynamicParticles() : 0; }
		virtual int32 GetNumIterations() const override { return ClothSimulationProxy.IsValid() ? ClothSimulationProxy->GetNumIterations() : 0; }
		virtual int32 GetNumSubsteps() const override { return ClothSimulationProxy.IsValid() ? ClothSimulationProxy->GetNumSubsteps() : 0; }
		virtual float GetSimulationTime() const override { return ClothSimulationProxy.IsValid() ? ClothSimulationProxy->GetSimulationTime() : 0.f; }
		virtual bool IsTeleported() const override { return ClothSimulationProxy.IsValid() ? ClothSimulationProxy->IsTeleported() : false; }
		//~ End IClothingSimulationInterface Interface

		//~ Begin IClothComponentAdapter Interface
		virtual const USkinnedMeshComponent& GetOwnerComponent() const override;
		virtual FCollisionSources& GetCollisionSources() const override
		{
			return *CollisionSources;
		}
		virtual TArray<const UChaosClothAssetBase*> GetAssets() const override;
		virtual TSharedPtr<const FChaosClothSimulationModel> GetActiveClothSimulationModel(const UChaosClothAssetBase* Asset, int32 ModelIndex) const override;
		virtual int32 GetSimulationGroupId(const UChaosClothAssetBase* Asset, int32 ModelIndex) const override;
		virtual int32 GetNextLinkedSimulationGroupId(int32 PrevSimulationGroupId) const override;
		virtual const FReferenceSkeleton* GetReferenceSkeleton() const override;
		virtual EClothingTeleportMode GetClothTeleportMode() const override;
		virtual bool IsSimulationEnabled() const override;
		virtual bool IsSimulationSuspended() const override;

		virtual bool NeedsResetRestLengths() const override  // TODO: Make this a property interactor function
		{
			return bNeedsResetRestLengths;
		}
		virtual const FString& GetRestLengthsMorphTargetName() const override  // TODO: Make this a property interactor function
		{
			return RestLengthsMorphTargetName;
		}
		virtual float GetClothGeometryScale() const override;
		virtual const TArray<TSharedPtr<const FManagedArrayCollection>>& GetPropertyCollections(const UChaosClothAssetBase* Asset, int32 ModelIndex) const override;
		virtual const TArray<TSharedPtr<const FManagedArrayCollection>>& GetSolverPropertyCollections() const override;
		virtual void ClearPropertyCollectionsDirtyFlags(int32 LodIndex) const override;
		virtual bool HasAnySimulationMeshData(int32 LodIndex) const override;
		//~ End IClothComponentAdapter Interface

		//~ Begin FGCObject Interface
		virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
		virtual FString GetReferencerName() const override;
		//~ End FGCObject Interface

		const FChaosClothSimulationProperties* GetSolverProperties() const;
		const TArray<TSharedPtr<::Chaos::Softs::FCollectionPropertyFacade>>& GetSolverPropertyFacades() const;

		void CalculateLODHasAnyRenderClothMappingData();

		USkeletalMeshComponent* OwnerComponent = nullptr;
		TUniquePtr<FCollisionSources> CollisionSources;
		TSharedPtr<FClothSimulationProxy> ClothSimulationProxy;
		
		using FAssetClothSimulationProperties = TPair<const UChaosClothAssetSKMClothingAsset*, FChaosClothSimulationProperties>;
		TArray<FAssetClothSimulationProperties> AssetClothSimulationProperties;  // Asset properties, matches CreateActor's SimDataIndex, some assets may appear multiple times, some entries might have a null asset
		
		TBitArray<> LODHasAnyRenderClothMappingData;
	
		bool bNeedsResetRestLengths = false;
		FString RestLengthsMorphTargetName;
	};
}  // namespace UE::Chaos::ClothAsset

UCLASS(MinimalAPI)
class UChaosClothAssetSKMClothingSimulationFactory final : public UClothingSimulationFactory
{
	GENERATED_BODY()

public:
	UChaosClothAssetSKMClothingSimulationFactory();
	virtual ~UChaosClothAssetSKMClothingSimulationFactory() override;

private:
	//~ Begin UClothingSimulationFactory Interface
	virtual IClothingSimulationInterface* CreateSimulation() const override
	{
		return static_cast<IClothingSimulationInterface*>(new UE::Chaos::ClothAsset::FSKMClothingSimulation());
	}

	virtual void DestroySimulation(IClothingSimulationInterface* Simulation) const override
	{
		delete Simulation;
	}

	virtual bool SupportsAsset(const UClothingAssetBase* Asset) const override;

	virtual bool SupportsRuntimeInteraction() const override
	{
		return true;
	}

	virtual UClothingSimulationInteractor* CreateInteractor() override;

	virtual TArrayView<const TSubclassOf<UClothConfigBase>> GetClothConfigClasses() const override
	{
		return TArrayView<const TSubclassOf<UClothConfigBase>>();
	}

	const UEnum* GetWeightMapTargetEnum() const override
	{
		return nullptr;
	}
	//~ End UClothingSimulationFactory Interface
};
