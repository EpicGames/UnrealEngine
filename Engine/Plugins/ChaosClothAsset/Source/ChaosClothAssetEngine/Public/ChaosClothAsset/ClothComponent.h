// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosClothAsset/ClothComponentAdapter.h"
#include "Containers/Array.h"
#include "Components/SkinnedMeshComponent.h"
#include "Dataflow/Interfaces/DataflowPhysicsSolver.h"
#include "ClothComponent.generated.h"

#define UE_API CHAOSCLOTHASSETENGINE_API

namespace UE_DEPRECATED(5.5, "Use UE::Dataflow instead.") Dataflow {}

class UChaosClothAssetBase;
class UChaosClothAsset;
class UChaosClothComponent;
class UChaosClothAssetInteractor;
class UThumbnailInfo;
struct FManagedArrayCollection;

namespace Chaos::Softs
{
	class FCollectionPropertyFacade;
}

namespace UE::Chaos::ClothAsset
{
	class FClothSimulationProxy;
	class FClothComponentCacheAdapter;
	class FCollisionSources;
}

/**
 * Private structure that contains all simulation properties runtime elements.
 * These get created per cloth asset/outfit piece when the component is registered.
 */
USTRUCT()
struct FChaosClothSimulationProperties final
{
	GENERATED_BODY()

	UPROPERTY(Transient)
	TObjectPtr<UChaosClothAssetInteractor> ClothOutfitInteractor;
	TArray<TSharedPtr<const FManagedArrayCollection>> PropertyCollections;
	TArray<TSharedPtr<::Chaos::Softs::FCollectionPropertyFacade>> CollectionPropertyFacades;

	FChaosClothSimulationProperties() = default;
	explicit FChaosClothSimulationProperties(const TArray<TSharedRef<const FManagedArrayCollection>>& AssetPropertyCollections)
	{
		Initialize(AssetPropertyCollections);
	}
	void Initialize(const TArray<TSharedRef<const FManagedArrayCollection>>& AssetPropertyCollections);
};

/** TG_LastDemotable wait on the parallel cloth task. Mirror of FSkeletalMeshComponentClothEndTickFunction. */
USTRUCT()
struct FChaosClothComponentEndTickFunction : public FTickFunction
{
	GENERATED_USTRUCT_BODY()

	UChaosClothComponent* Target = nullptr;

	virtual void ExecuteTick(float DeltaTime, enum ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent) override;
	virtual FString DiagnosticMessage() override;
	virtual FName DiagnosticContext(bool bDetailed) override;
};

template<>
struct TStructOpsTypeTraits<FChaosClothComponentEndTickFunction> : public TStructOpsTypeTraitsBase2<FChaosClothComponentEndTickFunction>
{
	enum
	{
		WithCopy = false
	};
};

/**
 * Cloth simulation component.
 */
UCLASS(MinimalAPI,
	ClassGroup = Physics,
	Meta = (BlueprintSpawnableComponent, ToolTip = "Chaos Cloth Component"),
	DisplayName = "Chaos Cloth Component",
	HideCategories = (Object, "Mesh|SkeletalAsset", Constraints, Advanced, Cooking, Collision, Navigation))
	class UChaosClothComponent
	: public USkinnedMeshComponent
	, public IDataflowPhysicsSolverInterface
	, public UE::Chaos::ClothAsset::IClothComponentAdapter
{
	GENERATED_BODY()
public:
	UE_API UChaosClothComponent(const FObjectInitializer& ObjectInitializer);
	UE_API UChaosClothComponent(FVTableHelper& Helper);
	UE_API ~UChaosClothComponent();

	/** Set the cloth object used by this component, could be a cloth asset, an outfit asset, or any other type of asset inheriting from UChaosClothAssetBase. */
	UFUNCTION(BlueprintCallable, Category = "ClothComponent", Meta = (Keywords = "Chaos Cloth Outfit Asset"))
	UE_API void SetAsset(UChaosClothAssetBase* InAsset);

	/** Get the cloth object used by this component, could be a cloth asset, an outfit asset, or any other type of asset inheriting from UChaosClothAssetBase. */
	UFUNCTION(BlueprintPure, Category = "ClothComponent", Meta = (Keywords = "Chaos Cloth Outfit Asset"))
	UE_API UChaosClothAssetBase* GetAsset() const;

	/** Set the cloth asset used by this component. */
	UFUNCTION(BlueprintCallable, Category = "ClothComponent", Meta = (DeprecatedFunction, Keywords = "Chaos Cloth Asset"))
	UE_DEPRECATED(5.6, "Use SetAsset instead.")
	UE_API void SetClothAsset(UChaosClothAsset* InClothAsset);

	/** Get the cloth asset used by this component. */
	UFUNCTION(BlueprintPure, Category = "ClothComponent", Meta = (DeprecatedFunction, Keywords = "Chaos Cloth Asset"))
	UE_DEPRECATED(5.6, "Use GetAsset instead.")
	UE_API UChaosClothAsset* GetClothAsset() const;

	/** Reset the teleport mode. */
	UFUNCTION(BlueprintCallable, Category = "ClothComponent", Meta = (Keywords = "Chaos Cloth Teleport"))
	void ResetTeleportMode() { bTeleport = bReset = false; }

	/** Teleport the cloth particles to the new reference bone location keeping pose and velocities prior to advancing the simulation. */
	UFUNCTION(BlueprintCallable, Category = "ClothComponent", Meta = (Keywords = "Chaos Cloth Teleport"))
	void ForceNextUpdateTeleport() { bTeleportOnce = true; bResetOnce = false; }

	/** Teleport the cloth particles to the new reference bone location while reseting the pose and velocities prior to advancing the simulation. */
	UFUNCTION(BlueprintCallable, Category = "ClothComponent", Meta = (Keywords = "Chaos Cloth Teleport Reset"))
	void ForceNextUpdateTeleportAndReset() { bTeleportOnce = bResetOnce = true; }

	/** Reset the cloth rest lengths with a given morph target (applied to base pose). If MorphTargetName is empty or does not exist, the rest lengths will be reset to default. */
	UFUNCTION(BlueprintCallable, Category = "ClothComponent", Meta = (Keywords = "Chaos Cloth Reset Rest Length Morph Target"))
	void ResetRestLengthsWithMorphTarget(const FString& MorphTargetName) { bResetRestLengthsFromMorphTarget = true; ResetRestLengthsMorphTargetName = MorphTargetName; }

	/** Return whether teleport is currently requested. Use GetClothTeleportMode to get teleport mode including any auto-teleport based on teleport thresholds. */
	bool NeedsTeleport() const { return bTeleport || bTeleportOnce; }

	/** Return whether reseting the pose is currently requested. Use GetClothTeleportMode to get teleport mode including any auto-teleport based on teleport thresholds*/
	bool NeedsReset() const { return bReset || bResetOnce; }

	/** Part of the IClothComponentAdapter implementation. */
	virtual bool NeedsResetRestLengths() const override { return bResetRestLengthsFromMorphTarget; }
	virtual const FString& GetRestLengthsMorphTargetName() const override { return ResetRestLengthsMorphTargetName; }

	/**
	 * Get currently calculated teleport mode.
	 * Part of the IClothComponentAdapter implementation.
	 */
	virtual EClothingTeleportMode GetClothTeleportMode() const override { return ClothTeleportMode; }

	/** Stop the simulation, and keep the cloth in its last pose. */
	UFUNCTION(CallInEditor, BlueprintCallable, Category = "ClothComponent", Meta = (UnsafeDuringActorConstruction, Keywords = "Chaos Cloth Simulation Suspend"))
	void SuspendSimulation() { bSuspendSimulation = true; }

	/** Resume a previously suspended simulation. */
	UFUNCTION(CallInEditor, BlueprintCallable, Category = "ClothComponent", Meta = (UnsafeDuringActorConstruction, Keywords = "Chaos Cloth Simulation Resume"))
	void ResumeSimulation() { bSuspendSimulation = false; }

	/**
	 * Return whether or not the simulation is currently suspended.
	 * Part of the IClothComponentAdapter implementation.
	 */
	UFUNCTION(BlueprintCallable, Category = "ClothComponent", Meta = (Keywords = "Chaos Cloth Simulation Suspend"))
	UE_API virtual bool IsSimulationSuspended() const override;

	/** Set whether or not to enable simulation. */
	UFUNCTION(BlueprintCallable, Category = "ClothComponent", Meta = (UnsafeDuringActorConstruction, Keywords = "Chaos Cloth Simulation Enable"))
	void SetEnableSimulation(bool bEnable) { bEnableSimulation = bEnable; }

	/**
	 * Return whether or not the simulation is currently enabled.
	 * Part of the IClothComponentAdapter implementation.
	 */
	UFUNCTION(BlueprintCallable, Category = "ClothComponent", Meta = (Keywords = "Chaos Cloth Simulation Enable"))
	UE_API virtual bool IsSimulationEnabled() const override;

	/** Reset all cloth simulation config properties to the values stored in the original cloth asset.*/
	UFUNCTION(BlueprintCallable, Category = "ClothComponent", Meta = (Keywords = "Chaos Cloth Config Property"))
	UE_API void ResetConfigProperties();

	/** Hard reset the cloth simulation by recreating the proxy. */
	UFUNCTION(CallInEditor, BlueprintCallable, Category = "ClothComponent", Meta = (DisplayName = "Hard Reset Simulation", Keywords = "Chaos Cloth Recreate Simulation Proxy"))
	UE_API void RecreateClothSimulationProxy();

	/**
	 * Find the current interactor for the cloth outfit associated with this cloth component.
	 * The default parameter values will always find the interactor when the cloth component is using a cloth asset.
	 * When using an outfit asset, an interactor from each cloth simulation models can be chosen by specifying either the model index or the model name.
	 * The solver properties interactor can also be obtained on model index 0 as it is shared across all models, but only set on the first one.
	 * @param ModelIndex The index of the outfit simulation model to retrieve the interactor for. Can be ignored or used as a fallback when seraching by name. Must be 0 for the solver interactor.
	 * @param ClothSimulationModelName The name of the outfit model to retrieve the interactor for. Leave it empty to use the ModelIndex instead of when retieving the solver interactor.
	 */
	UFUNCTION(BlueprintCallable, Category = "ClothComponent")
	UE_API UChaosClothAssetInteractor* GetClothOutfitInteractor(int32 ModelIndex = 0, const FName ClothSimulationModelName = NAME_None);

	/**
	 * Add a collision source for the cloth on this component.
	 * Each cloth tick, the collision defined by the physics asset, transformed by the bones in the source component, will be applied to the simulation.
	 * @param SourceComponent The component to extract collision transforms from.
	 * @param SourcePhysicsAsset The physics asset that defines the collision primitives (that will be transformed by the SourceComponent's bones).
	 * @param bUseSphylsOnly Whether to only use spheres and capsules from the collision sources (which is faster and matches the legacy behavior).
	 */
	UFUNCTION(BlueprintCallable, Category = "ClothComponent", Meta = (Keywords = "Chaos Cloth Collision Source"))
	UE_API void AddCollisionSource(USkinnedMeshComponent* SourceComponent, const UPhysicsAsset* SourcePhysicsAsset, bool bUseSphylsOnly = false);

	/** Remove a cloth collision source matching the specified component and physics asset, */
	UFUNCTION(BlueprintCallable, Category = "ClothComponent", Meta = (Keywords = "Chaos Cloth Collision Source"))
	UE_API void RemoveCollisionSource(const USkinnedMeshComponent* SourceComponent, const UPhysicsAsset* SourcePhysicsAsset);

	/** Remove all cloth collision sources matching the specified component. */
	UFUNCTION(BlueprintCallable, Category = "ClothComponent", Meta = (Keywords = "Chaos Cloth Collision Source"))
	UE_API void RemoveCollisionSources(const USkinnedMeshComponent* SourceComponent);

	/** Remove all cloth collision sources. */
	UFUNCTION(BlueprintCallable, Category = "ClothComponent", Meta = (Keywords = "Chaos Cloth Collision Source"))
	UE_API void ResetCollisionSources();

	/**
	 * Return all collision sources currently assigned to this component.
	 * Part of the IClothComponentAdapter implementation.
	 */
	virtual UE::Chaos::ClothAsset::FCollisionSources& GetCollisionSources() const override { return *CollisionSources; }

	/** Set whether or not to collide with the environment. */
	UFUNCTION(BlueprintCallable, Category = "ClothComponent", Meta = (Keywords = "Chaos Cloth Collide Environment"))
	UE_API void SetCollideWithEnvironment(bool bCollide);

	/** Set whether or not to collision with the environment is enabled. */
	UFUNCTION(BlueprintCallable, Category = "ClothComponent", Meta = (Keywords = "Chaos Cloth Collide Environment"))
	bool GetCollideWithEnvironment() const { return bCollideWithEnvironment; }

	/**
	* Sets whether or not to simulate cloth in the editor.
	* This is supported only in the editor
	*/
	UFUNCTION(BlueprintCallable, Category = "ClothComponent", meta = (DevelopmentOnly, UnsafeDuringActorConstruction = "true"))
	UE_API void SetSimulateInEditor(const bool bInSimulateInEditor);

#if WITH_EDITORONLY_DATA
	/** Return whether the cloth is set to simulate in the editor. */
	bool GetSimulateInEditor() const { return bSimulateInEditor; }
#endif

	/**
	 * Return the property collections holding the runtime properties for this cloth component model (one per LOD).
	 * This might be different from the cloth asset's since the component's properties can be modified in code or in blueprints.
	 * This could also be different from the cloth simulation object until the cloth simulation thread synchronise the properties.
	 */
	const TArray<TSharedPtr<const FManagedArrayCollection>>& GetPropertyCollections(int32 ModelIndex) const
	{
		return ClothSimulationProperties[ModelIndex].PropertyCollections;
	}
	/**
	 * Return the solver property collections.
	 * Part of the IClothComponentAdapter implementation.
	 */
	virtual const TArray<TSharedPtr<const FManagedArrayCollection>>& GetSolverPropertyCollections() const override
	{
		return ClothSimulationProperties[0].PropertyCollections;
	}
	UE_DEPRECATED(5.6, "Use GetPropertyCollections(int32 ModelIndex) instead.")
	const TArray<TSharedPtr<const FManagedArrayCollection>>& GetPropertyCollections() const
	{
		return reinterpret_cast<const TArray<TSharedPtr<const FManagedArrayCollection>>&>(ClothSimulationProperties[0].PropertyCollections);
	}

	const UE::Chaos::ClothAsset::FClothSimulationProxy* GetClothSimulationProxy() const { return ClothSimulationProxy.Get(); }

	/**
	 * This scale is applied to all cloth geometry (e.g., cloth meshes and collisions) in order to simulate in a different scale space than world.
	 * This scale is not applied to distance-based simulation parameters such as MaxDistance.
	 * This property is currently only read by the cloth solver when creating cloth actors, but may become animatable in the future.
	 * Part of the IClothComponentAdapter implementation.
	 */
	virtual float GetClothGeometryScale() const override { return ClothGeometryScale; }
	void SetClothGeometryScale(float Scale) { ClothGeometryScale = Scale; }

	/**
	* Gets the teleportation distance threshold.
	*
	* @return Threshold value.
	*/
	UFUNCTION(BlueprintGetter, Category = ClothComponent)
	UE_API float GetTeleportDistanceThreshold() const;

	/**
	* Sets the teleportation distance threshold.
	*
	* @param threshold Threshold value.
	*/
	UFUNCTION(BlueprintSetter, Category = ClothComponent)
	UE_API void SetTeleportDistanceThreshold(float Threshold);

	/**
	* Gets the teleportation rotation threshold.
	*
	* @return Threshold in degrees.
	*/
	UFUNCTION(BlueprintGetter, Category = ClothComponent)
	UE_API float GetTeleportRotationThreshold() const;

	void SetBindToLeaderComponent(bool bValue)
	{
		bBindToLeaderComponent = bValue;
	}

	bool GetBindToLeaderComponent() const
	{
		return bBindToLeaderComponent;
	}

	/**
	* Sets the teleportation rotation threshold.
	*
	* @param threshold Threshold in degrees.
	*/
	UFUNCTION(BlueprintSetter, Category = ClothComponent)
	UE_API void SetTeleportRotationThreshold(float Threshold);

	/** Update config properties from the asset. Will only update existing values.*/
	UE_API void UpdateConfigProperties();

	/** Stalls on any currently running clothing simulations.*/
	UE_API void WaitForExistingParallelClothSimulation_GameThread();

#if WITH_EDITOR
	/** This will cause the component to tick once in editor. Both flags will be consumed on that tick. Used for the cache adapter. */
	void SetTickOnceInEditor() { bTickOnceInEditor = true; bTickInEditor = true; }
#endif

#if WITH_EDITORONLY_DATA
	UThumbnailInfo* GetThumbnailInfo()
	{
		return ThumbnailInfo;
	}
#endif

protected:
	
	//~ Begin UObject Interface
	UE_API virtual void PostLoad() override;
	UE_API virtual void PostDuplicate(bool bDuplicateForPIE) override;
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual void PostEditImport() override;
	UE_API virtual bool CanEditChange(const FProperty* InProperty) const override;
#endif // WITH_EDITOR
	//~ End UObject Interface

	//~ Begin UActorComponent Interface
	UE_API virtual void OnRegister() override;
	UE_API virtual void OnUnregister() override;
	UE_API virtual void SetComponentTickEnabled(bool bEnabled) override;
	UE_API virtual bool IsComponentTickEnabled() const override;
	UE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;
	UE_API virtual bool RequiresPreEndOfFrameSync() const override;
	UE_API virtual void OnPreEndOfFrameSync() override;
#if WITH_EDITOR
	UE_API virtual TStructOnScope<FActorComponentInstanceData> GetComponentInstanceData() const override;
#endif
	//~ End UActorComponent Interface

	//~ Begin USceneComponent Interface.
	UE_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	UE_API virtual void OnAttachmentChanged() override;
	UE_API virtual bool IsVisible() const override;
	//~ End USceneComponent Interface

	//~ Begin USkinnedMeshComponent Interface
	UE_API virtual void RefreshBoneTransforms(FActorComponentTickFunction* TickFunction = nullptr) override;
	UE_API virtual void GetUpdateClothSimulationData_AnyThread(TMap<int32, FClothSimulData>& OutClothSimulData, FMatrix& OutLocalToWorld, float& OutBlendWeight) const override;
	UE_API virtual void SetSkinnedAssetAndUpdate(USkinnedAsset* InSkinnedAsset, bool bReinitPose = true) override;
	UE_API virtual void GetAdditionalRequiredBonesForLeader(int32 LODIndex, TArray<FBoneIndexType>& InOutRequiredBones) const override;
	UE_API virtual void FinalizeBoneTransform() override;
	UE_API virtual FDelegateHandle RegisterOnBoneTransformsFinalizedDelegate(const FOnBoneTransformsFinalizedMultiCast::FDelegate& Delegate) override;
	UE_API virtual void UnregisterOnBoneTransformsFinalizedDelegate(const FDelegateHandle& DelegateHandle) override;
	//~ End USkinnedMeshComponent Interface

	//~ Begin IDataflowPhysicsSolverInterface Interface
	virtual FString GetSimulationName() const override {return GetName();};
	virtual FDataflowSimulationAsset& GetSimulationAsset() override {return SimulationAsset;};
	virtual const FDataflowSimulationAsset& GetSimulationAsset() const override {return SimulationAsset;};
	UE_API virtual FDataflowSimulationProxy* GetSimulationProxy() override;
	UE_API virtual const FDataflowSimulationProxy* GetSimulationProxy() const  override;
	UE_API virtual void BuildSimulationProxy() override;
	UE_API virtual void ResetSimulationProxy() override;
	UE_API virtual void WriteToSimulation(const float DeltaTime, const bool bAsyncTask) override;
	UE_API virtual void ReadFromSimulation(const float DeltaTime, const bool bAsyncTask) override;
	UE_API virtual void PreProcessSimulation(const float DeltaTime) override;
	UE_API virtual void PostProcessSimulation(const float DeltaTime) override;
	//~ End IDataflowPhysicsSolverInterface Interface

	//~ Begin IClothComponentAdapter Interface
	virtual const USkinnedMeshComponent& GetOwnerComponent() const override
	{
		return *this;
	}
	UE_API virtual const FReferenceSkeleton* GetReferenceSkeleton() const override;
	virtual TArray<const UChaosClothAssetBase*> GetAssets() const override
	{
		return GetAsset() ? TArray<const UChaosClothAssetBase*>{ GetAsset() } : TArray<const UChaosClothAssetBase*>{};
	}
	UE_API virtual int32 GetSimulationGroupId(const UChaosClothAssetBase* InAsset, int32 ModelIndex) const override;
	virtual const TArray<TSharedPtr<const FManagedArrayCollection>>& GetPropertyCollections(const UChaosClothAssetBase* InAsset, int32 ModelIndex) const override
	{
		checkf(GetSimulationGroupId(InAsset, ModelIndex) != INDEX_NONE, TEXT("Invalid arguments"));
		return GetPropertyCollections(ModelIndex);  // There's only this one asset for now, therefore this is the same as calling GetPropertyCollections(ModelIndex)
	}
	UE_API virtual void ClearPropertyCollectionsDirtyFlags(int32 LODIndex) const override;
	UE_API virtual bool HasAnySimulationMeshData(int32 LODIndex) const override;
	//~ End IClothComponentAdapter Interface

	/** Override this function for setting up custom simulation proxies when the component is registered. */
	UE_API virtual TSharedPtr<UE::Chaos::ClothAsset::FClothSimulationProxy> CreateClothSimulationProxy();

private:
	UE_API void CreateClothSimulationProxyImpl();
	UE_API void StartNewParallelSimulation(float DeltaTime);
	UE_API void HandleExistingParallelSimulation();
	UE_API bool ShouldWaitForParallelSimulationInTickComponent() const;
	bool ShouldUseEndTickWait() const;
	UE_API void UpdateComponentSpaceTransforms();
	UE_API void UpdateVisibility();
	UE_API void UpdateClothTeleport();

	/** Sync the editor-only Asset alias from the underlying SkinnedAsset. */
	void RefreshAsset();

	/** Register/unregister the TG_LastDemotable wait tick. Drives registration from the in-tick wait CVar gate. */
	void RegisterEndTick(bool bRegister);

	/** Tick function that waits on the parallel cloth simulation task in TG_LastDemotable. */
	FChaosClothComponentEndTickFunction EndTickFunction;

	friend struct FChaosClothComponentEndTickFunction;
	friend UE::Chaos::ClothAsset::FClothComponentCacheAdapter;
	friend struct FChaosClothComponentInstanceData;

	/* Solver dataflow asset used to advance in time */
	UPROPERTY(EditAnywhere, Category = ClothComponent, meta=(EditConditionHides), AdvancedDisplay)
	FDataflowSimulationAsset SimulationAsset;

	/** Blend amount between the skinned (=0) and the simulated pose (=1). */
	UPROPERTY(Interp, Category = ClothComponent)
	float BlendWeight = 1.f;

	/** This scale is applied to all cloth geometry (e.g., cloth meshes and collisions) in order to simulate in a different scale space than world.This scale is not applied to distance-based simulation parameters such as MaxDistance.
	* This property is currently only read by the cloth solver when creating cloth actors, but may become animatable in the future.
	*/
	UPROPERTY(EditAnywhere, Category = ClothComponent, meta = (UIMin = 0.0, UIMax = 10.0, ClampMin = 0.0, ClampMax = 10000.0))
	float ClothGeometryScale = 1.f;
	
	/** If enabled, and the parent is another Skinned Mesh Component (e.g. another Cloth Component, Poseable Mesh Component, Skeletal Mesh Component, ...etc.), use its pose. */
	UPROPERTY(EditAnywhere, Category = ClothComponent)
	uint8 bUseAttachedParentAsPoseComponent : 1;

	/** Whether to wait for the cloth simulation to end in the TickComponent instead of in the EndOfFrameUpdates. */
	UPROPERTY(EditAnywhere, Category = ClothComponent)
	uint8 bWaitForParallelTask : 1;

	/** Whether to enable the simulation or use the skinned pose instead. */
	UPROPERTY(Interp, Category = ClothComponent)
	uint8 bEnableSimulation : 1;

	/** Whether to suspend the simulation and use the last simulated pose. */
	UPROPERTY(Interp, Category = ClothComponent)
	uint8 bSuspendSimulation : 1;

	/** Whether to use the leader component simulation result. 
	 * Currently only supported when using UseAttachedParentAsPoseComponent
	 * 
	 * If this component has a valid LeaderPoseComponent then this makes cloth items on the follower component
	 * take the transforms of the cloth items on the leader component instead of simulating separately.
	 * @Note The meshes used in the components must be identical for the cloth to bind correctly
	 */
	UPROPERTY(EditAnywhere, Category = ClothComponent)
	uint8 bBindToLeaderComponent : 1;

	/** Whether to teleport the cloth prior to advancing the simulation. */
	UPROPERTY(Interp, Category = ClothComponent)
	uint8 bTeleport : 1;

	/** Whether to reset the pose, bTeleport must be true. */
	UPROPERTY(Interp, Category = ClothComponent)
	uint8 bReset : 1;

	/** Collide with the environment. */
	UPROPERTY(EditAnywhere, Category = ClothComponent)
	uint8 bCollideWithEnvironment : 1;

#if WITH_EDITORONLY_DATA
	/** Whether to run the simulation in editor. */
	UPROPERTY(EditInstanceOnly, Transient, Setter = SetSimulateInEditor, Getter = GetSimulateInEditor, Category = ClothComponent)
	uint8 bSimulateInEditor : 1;

	/** Asset used by this component. Can be either a cloth asset or an outfit asset. */
	UE_DEPRECATED(5.6, "This property isn't deprecated, but getter and setter must be used at all times to preserve correct operations.")
	UPROPERTY(EditAnywhere, Transient, Setter = SetAsset, BlueprintSetter = SetAsset, Getter = GetAsset, BlueprintGetter = GetAsset, Category = ClothComponent)
	TObjectPtr<UChaosClothAssetBase> Asset;
#endif

	/** Cloth leader component used to drive simulation. This may have its own leader component */
	UPROPERTY()
	TWeakObjectPtr<UChaosClothComponent> LeaderClothComponent;

	/**
	* Conduct teleportation if the character's movement is greater than this threshold in 1 frame.
	* Zero or negative values will skip the check.
	* You can also do force teleport manually using ForceNextUpdateTeleport() / ForceNextUpdateTeleportAndReset().
	*/
	UPROPERTY(EditAnywhere, BlueprintGetter = GetTeleportDistanceThreshold, BlueprintSetter = SetTeleportDistanceThreshold, Category = ClothComponent)
	float TeleportDistanceThreshold;

	/**
	* Rotation threshold in degrees, ranging from 0 to 180.
	* Conduct teleportation if the character's rotation is greater than this threshold in 1 frame.
	* Zero or negative values will skip the check.
	*/
	UPROPERTY(EditAnywhere, BlueprintGetter = GetTeleportRotationThreshold, BlueprintSetter = SetTeleportRotationThreshold, Category = ClothComponent)
	float TeleportRotationThreshold;

	/** Used for pre-computation using tTeleportDistanceThreshold property */
	float ClothTeleportDistThresholdSquared;

	/** Used for pre-computation using TeleportRotationThreshold property */
	float ClothTeleportCosineThresholdInRad;

	/** previous root bone matrix to compare the difference and decide to do clothing teleport  */
	FMatrix	PrevRootBoneMatrix;

	/** Currently calculated ClothTeleport based on bTeleport, bReset as well as any teleport calculated based on TeleportDistanceThreshold and TeleportRotationThreshold */
	EClothingTeleportMode ClothTeleportMode;

	/** Like bTeleport, but cleared every frame */
	bool bTeleportOnce = false;

	/** Like bReset, but cleared every frame */
	bool bResetOnce = false;

	bool bHasValidRenderDataForVisibility = false;

	/** Reset restlengths from morph target. */
	bool bResetRestLengthsFromMorphTarget = false;
	FString ResetRestLengthsMorphTargetName;

	/**
	 * Simulation properties per cloth model. 
	 * Transient because it doesn't need to be serialized but contains pointers to UObject that needs not be garbage collected.
	 */
	UPROPERTY(Transient, NonTransactional)
	TArray<FChaosClothSimulationProperties> ClothSimulationProperties;

	TSharedPtr<UE::Chaos::ClothAsset::FClothSimulationProxy> ClothSimulationProxy;

	// Multicaster fired when this component bone transforms are finalized
	FOnBoneTransformsFinalizedMultiCast OnBoneTransformsFinalizedMC;

	// External sources for collision
	TUniquePtr<UE::Chaos::ClothAsset::FCollisionSources> CollisionSources;

#if WITH_EDITOR
	bool bTickOnceInEditor = false;
#endif

#if WITH_EDITORONLY_DATA
	/** Information for thumbnail rendering */
	UPROPERTY(VisibleAnywhere, Instanced, AdvancedDisplay, Category=StaticMesh)
	TObjectPtr<UThumbnailInfo> ThumbnailInfo;
#endif
};

#undef UE_API
