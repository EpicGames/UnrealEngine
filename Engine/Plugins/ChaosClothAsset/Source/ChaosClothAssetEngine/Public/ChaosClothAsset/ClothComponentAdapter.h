// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Templates/SharedPointer.h"

#include "ChaosClothAsset/ClothAssetBase.h"

class USkinnedMeshComponent;
struct FManagedArrayCollection;
struct FReferenceSkeleton;
struct FChaosClothSimulationModel;
enum class EClothingTeleportMode : uint8;

namespace UE::Chaos::ClothAsset
{
	class FCollisionSources;

	class IClothComponentAdapter
	{
	public:
		/**
		 * Return the SkinnedMeshComponent owning this adapter.
		 * It can be this object pointer or a different owner component if this interface isn't directly implemented on a component.
		 */
		virtual const USkinnedMeshComponent& GetOwnerComponent() const = 0;

		/** Return the collision sources to use in cloth simulation proxy. */
		virtual FCollisionSources& GetCollisionSources() const = 0;

		/** Return the list of cloth/outfit assets in use by this adapter. */
		virtual TArray<const UChaosClothAssetBase*> GetAssets() const = 0;

		/** Return the cloth simulation model at ModelIndex for Asset. */
		virtual TSharedPtr<const FChaosClothSimulationModel> GetActiveClothSimulationModel(const UChaosClothAssetBase* Asset, int32 ModelIndex) const
		{
			return Asset ? Asset->GetClothSimulationModel(ModelIndex) : nullptr;
		}
		
		/** Return the index of the model assigned to the render section, or INDEX_NONE when the model shouldn't be simulated. */
		virtual int32 GetSimulationGroupId(const UChaosClothAssetBase* Asset, int32 ModelId) const = 0;

		/**
		 * Return the next SimulationGroupId when multiple group IDs are sharing the same simulated cloth,
		 * or INDEX_NONE when SimulationGroupId is the last linked element.
		 * Use GetSimulationGroupId() to get the initial ID, and call GetNextLinkedSimulationGroupId() with the previous ID
		 * to retrieve the subsequent linked IDs.
		 * This function allows for a single simulated cloth to update multiple simulation data slots.
		 */
		virtual int32 GetNextLinkedSimulationGroupId(int32 PrevSimulationGroupId) const { return INDEX_NONE; }

		/** Return the reference skeleton as passed to the component renderer. */
		virtual const FReferenceSkeleton* GetReferenceSkeleton() const = 0;

		/** Return the current teleport mode. */
		virtual EClothingTeleportMode GetClothTeleportMode() const = 0;

		/** Return whether the simulation is currently running. */
		virtual bool IsSimulationEnabled() const = 0;

		/** Return whether the simulation is currently suspended. */
		virtual bool IsSimulationSuspended() const = 0;

		/** Return whether the cloth mesh rest lengths needs to be reset. */
		virtual bool NeedsResetRestLengths() const = 0;

		/** Return the name of the current morph target. */
		virtual const FString& GetRestLengthsMorphTargetName() const = 0;

		/** Return the scale applied to all cloth geometries (cloth meshes and collisions) in order to simulate in a different scale space. */
		virtual float GetClothGeometryScale() const = 0;

		/** Return the runtime property collections used for modifications of the specified cloth/outfit asset simulation model. */
		virtual const TArray<TSharedPtr<const FManagedArrayCollection>>& GetPropertyCollections(const UChaosClothAssetBase* Asset, int32 ModelIndex) const = 0;

		/** Return the runtime solver property collections. */
		virtual const TArray<TSharedPtr<const FManagedArrayCollection>>& GetSolverPropertyCollections() const = 0;

		/** Clear all property collections dirty flags. This is handled by the simulation proxy once the collections have been copied. */
		virtual void ClearPropertyCollectionsDirtyFlags(int32 LodIndex) const {}

		/** Return true when any of the simulated assets LOD models have valid simulation data, or false otherwise. */
		virtual bool HasAnySimulationMeshData(int32 LodIndex) const = 0;

	protected:
		virtual ~IClothComponentAdapter() = default;
	};
}
