// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Vector2D.h"
#include "Templates/SharedPointer.h"
#include "PointWeightMap.h"

#define UE_API CHAOSCLOTHASSETENGINE_API

class FArchive;
class FSkeletalMeshRenderData;
class UChaosClothAssetBase;
class UMaterialInterface;
struct FManagedArrayCollection;
class FName;
class FSkeletalMeshLODModel;
struct FReferenceSkeleton;
struct FSoftObjectPath;
namespace Chaos
{
	namespace Softs
	{
		class FCollectionPropertyConstFacade;
	}
}

namespace UE::Chaos::ClothAsset
{
	class FCollectionClothFacade;
	class FCollectionClothConstFacade;
	class FCollectionClothFacade;

	/**
	 *  Tools operating on cloth collections with Engine dependency
	 */
	struct FClothEngineTools
	{
		/** Generate tether data. */
		static UE_API void GenerateTethers(const TSharedRef<FManagedArrayCollection>& ClothCollection, const FName& WeightMap, const bool bGeodesicTethers, const FVector2f& MaxDistanceValue = FVector2f(0.f, 1.f));
		static UE_API void GenerateTethersFromSelectionSet(const TSharedRef<FManagedArrayCollection>& ClothCollection, const FName& FixedEndSet, const bool bGeodesicTethers);
		/** @param CustomTetherEndSets: First element of each pair is DynamicSet, second is FixedSet */
		static UE_API void GenerateTethersFromCustomSelectionSets(const TSharedRef<FManagedArrayCollection>& ClothCollection, const FName& FixedEndSet, const TArray<TPair<FName, FName>>& CustomTetherEndSets, const bool bGeodesicTethers);
	
		/** Retrieve the MaxDistance weight map from the cloth facades. Return an empty map if the collection doesn't match the requested NumLodSimVertices.*/
		static FPointWeightMap GetMaxDistanceWeightMap(const FCollectionClothConstFacade& ClothFacade, const ::Chaos::Softs::FCollectionPropertyConstFacade& PropertyFacade, const int32 NumLodSimVertices);
		/** Retrieve the MaxDistance weight map from the cloth collection. Return an empty map if the collection doesn't match the requested NumLodSimVertices. */
		static FPointWeightMap GetMaxDistanceWeightMap(const TSharedRef<const FManagedArrayCollection>& ClothCollection, const int32 NumLodSimVertices);

		/** Calculate ReferenceBone from bone closest to the root of all used bones */
		static UE_API int32 CalculateReferenceBoneIndex(const TArray<int32>& UsedBones, const FReferenceSkeleton& ReferenceSkeleton);
		/** Calculate ReferenceBone from bone closest to the root of all used weighted sim bones in a collection */
		static UE_API int32 CalculateReferenceBoneIndex(const TSharedRef<const FManagedArrayCollection>& ClothCollection, const FReferenceSkeleton& ReferenceSkeleton);

		/** Determine if skeletal meshes on two different cloth collections are compatible (one's reference skeleton is a subset of the other).
		 *  If they are, determine which skeletal mesh is the superset, and calculate remap indices that can be used to remap bone indices from
		 *  the subset collection to the superset skeletal mesh.
		 *  @param Cloth1 The cloth facade for the first cloth collection.
		 *  @param Cloth2 The cloth facade for the second cloth collection.
		 *  @param OutMergedSkeletalMeshPath The superset skeletal mesh used as reference skeleton, either Cloth1 or Cloth2's skeletal mesh path.
		 *  @param OutBoneIndicesRemapCloth1 Remap indices to convert Cloth1 BoneIndices to OutMergedSkeletalMeshPath's ref skeleton. Empty if no remapping is required.
		 *  @param OutBoneIndicesRemapCloth2 Remap indices to convert Cloth2 BoneIndices to OutMergedSkeletalMeshPath's ref skeleton. Empty if no remapping is required.
		 *  @param OutIncompatibleErrorDetails optional text describing why the skeletal meshes aren't compatible (empty if no error).
		 *  @return Whether or not the collections are compatible.
		 */
		static UE_API bool CalculateRemappedBoneIndicesIfCompatible(
			const FCollectionClothConstFacade& Cloth1,
			const FCollectionClothConstFacade& Cloth2,
			FSoftObjectPath& OutMergedSkeletalMeshPath,
			TArray<int32>& OutBoneIndicesRemapCloth1,
			TArray<int32>& OutBoneIndicesRemapCloth2,
			FText* OutIncompatibleErrorDetails = nullptr);

		/**
		 * Remap Bone indices for a cloth collection using remap array calculated by CalculateRemappedBoneIndicesIfCompatible. 
		 * Only vertices beginning with the provided Offsets will be remapped (can be set to non-zero to remap after merging collections) */
		static UE_API void RemapBoneIndices(FCollectionClothFacade& Cloth, const TArray<int32>& BoneIndicesRemap, const int32 SimVertex3DOffset = 0, const int32 RenderVertexOffset = 0);

#if WITH_EDITOR
		/**
		 * Build a FSkeletalMeshRenderData from the asset's simulation model vertex data.
		 * Used for preview rendering (thumbnails) when the asset has no render data.
		 * @param Asset The cloth asset base to build preview render data from.
		 * @return The built render data, or nullptr if the asset has no valid simulation models.
		 */
		static UE_API TUniquePtr<FSkeletalMeshRenderData> BuildSimPreviewRenderData(const UChaosClothAssetBase& Asset);

		/** Load the default material for sim-model preview rendering (CameraLitDoubleSided). */
		static UE_API UMaterialInterface* GetSimPreviewMaterial();
#endif

		/** Create a new SimAccessoryMesh by copying the SimMesh data from one collection into another.
		 *  Optionally use SimImportVertexID to match data rather than a straight copy of SimVertices3D.
		 *  Any unmatched vertices will copy the data from the existing Sim Mesh.
		 *  Returns the newly created accessory mesh's index */
		static UE_API int32 CopySimMeshToSimAccessoryMesh(const FName& AccessoryMeshName, FCollectionClothFacade& ToCloth, const FCollectionClothConstFacade& FromCloth, bool bUseSimImportVertexID, FText* OutIncompatibleErrorDetails = nullptr);

#if WITH_EDITORONLY_DATA
		/**
		 * Return a cook-trimmed copy of a cloth collection, or the cloth collection itself when trimming is disabled.
		 * Falls back to the input collection when p.ClothCollectionOnlyCookRequiredFacades is disabled.
		 * @param AssetName Label used when logging the trim result.
		 * @param ClothCollection Source collection to trim.
		 * @param bLog Pass false to avoid log entries (the cooker calls Serialize twice per package).
		 * @return The trimmed collection, or the input collection when trimming is disabled.
		 */
		static UE_API TSharedRef<const FManagedArrayCollection> TrimClothCollectionOnCook(
			const FString& AssetName,
			const TSharedRef<const FManagedArrayCollection>& ClothCollection,
			bool bLog);

		/**
		 * Return the underlying asset path for a save archive, stripping the PackageHarvester wrapper where necessary.
		 * @param Ar Cook save archive to inspect.
		 * @return The unwrapped asset path, or Ar.GetArchiveName() unchanged when no wrapper is present.
		 */
		static UE_API FString GetCookedAssetPath(const FArchive& Ar);
#endif
	};
}  // End namespace UE::Chaos::ClothAsset

#undef UE_API
