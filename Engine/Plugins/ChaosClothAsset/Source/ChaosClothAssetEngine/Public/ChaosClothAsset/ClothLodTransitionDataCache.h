// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Misc/SecureHash.h"
#include "SkeletalMeshTypes.h"
#include "ClothLodTransitionDataCache.generated.h"

USTRUCT()
struct FChaosClothAssetLodTransitionDataCache
{
	GENERATED_BODY()

	/** Hash of this model. Require valid hash of Up and Down models to be able to re-use the transition data.**/
	FMD5Hash ModelHash;

	/** LOD Transition mesh to mesh skinning weights. */
	TArray<FMeshToMeshVertData> LODTransitionUpData;
	TArray<FMeshToMeshVertData> LODTransitionDownData;

	// Custom serialize for MD5Hash
	bool Serialize(FArchive& Ar);
};

template<>
struct TStructOpsTypeTraits<FChaosClothAssetLodTransitionDataCache> : public TStructOpsTypeTraitsBase2<FChaosClothAssetLodTransitionDataCache>
{
	enum
	{
		WithSerializer = true,
	};
};
