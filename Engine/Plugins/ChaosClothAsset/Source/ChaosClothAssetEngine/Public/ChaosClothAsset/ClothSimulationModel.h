// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Math/MathFwd.h"
#include "Templates/SharedPointer.h"
#include "ClothVertBoneData.h"
#include "ClothTetherData.h"
#include "SkeletalMeshTypes.h"
#include "ClothSimulationModel.generated.h"

struct FReferenceSkeleton;
struct FManagedArrayCollection;
struct FChaosClothAssetLodTransitionDataCache;

/**
 * Cloth simulation LOD model.
 * Contains the LOD data as fed to the solver for constraints creation.
 */
USTRUCT()
struct FChaosClothSimulationLodModel
{
	GENERATED_BODY()

	/** Vertex positions. */
	UPROPERTY()
	TArray<FVector3f> Positions;
	
	/** Vertex Normals. */
	UPROPERTY()
	TArray<FVector3f> Normals;

	/** Triangles indices. */
	UPROPERTY()
	TArray<uint32> Indices;

	/** Skinning weights. */
	UPROPERTY()
	TArray<FClothVertBoneData> BoneData;

	/** Already remapped using UsedBoneIndices. These are bones that are needed by this LOD for skinning that aren't needed for render.*/
	UPROPERTY()
	TArray<uint16> RequiredExtraBoneIndices;

	/** LOD Transition mesh to mesh skinning weights. */
	TArray<FMeshToMeshVertData> LODTransitionUpData;
	TArray<FMeshToMeshVertData> LODTransitionDownData;

	/** 2d pattern positions. */
	UPROPERTY()
	TArray<FVector2f> PatternPositions;

	/** Pattern triangle indices. Indexing into PatternPositions. */
	UPROPERTY()
	TArray<uint32> PatternIndices;

	/** Map from PatternsPositions indices to Positions indices. */
	UPROPERTY()
	TArray<uint32> PatternToWeldedIndices;

	/** Weight maps for storing painted attributes modifiers on constraint properties. */
	TMap<FName, TArray<float>> WeightMaps;

	/** Tether data*/
	UPROPERTY()
	FClothTetherData TetherData;

	/** Vertex sets */
	TMap<FName, TSet<int32>> VertexSets;

	/** Face int maps (currently used by cloth collision layers)*/
	TMap<FName, TArray<int32>> FaceIntMaps;

	/** Face sets */
	TMap<FName, TSet<int32>> FaceSets;

	// Custom serialize for weight maps
	bool Serialize(FArchive& Ar);

	void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize);

	/** Calculate the maximum number of bone influences used by the skinning weights. */
	int32 CalcMaxBoneInfluences() const;
};

template<>
struct TStructOpsTypeTraits<FChaosClothSimulationLodModel> : public TStructOpsTypeTraitsBase2<FChaosClothSimulationLodModel>
{
	enum
	{
		WithSerializer = true,
	};
};

/**
 * Cloth simulation model.
 * Contains the LOD models.
 */
USTRUCT()
struct FChaosClothSimulationModel
{
	GENERATED_BODY()

	/** LOD data. */
	UPROPERTY()
	TArray<FChaosClothSimulationLodModel> ClothSimulationLodModels;

	/** List of bones this asset uses from the reference skeleton. */
	UPROPERTY()
	TArray<FName> UsedBoneNames;

	/** List of the indices for the bones in UsedBoneNames, used for remapping. */
	UPROPERTY()
	TArray<int32> UsedBoneIndices;

	/** Bone to treat as the root of the simulation space. */
	UPROPERTY()
	int32 ReferenceBoneIndex = INDEX_NONE;

	FChaosClothSimulationModel() = default;
	CHAOSCLOTHASSETENGINE_API FChaosClothSimulationModel(const TArray<TSharedRef<const FManagedArrayCollection>>& ClothCollections, const FReferenceSkeleton& ReferenceSkeleton, TArray<FChaosClothAssetLodTransitionDataCache>* InOutTransitionCache = nullptr);

	int32 GetNumLods() const { return ClothSimulationLodModels.Num(); }

	bool IsValidLodIndex(int32 LodIndex) const { return ClothSimulationLodModels.IsValidIndex(LodIndex); }

	int32 GetNumVertices(int32 LodIndex) const { return IsValidLodIndex(LodIndex) ? ClothSimulationLodModels[LodIndex].Positions.Num() : 0; }
	int32 GetNumTriangles(int32 LodIndex) const { return IsValidLodIndex(LodIndex) ? ClothSimulationLodModels[LodIndex].Indices.Num() / 3 : 0; }

	TConstArrayView<FVector3f> GetPositions(int32 LodIndex) const { return IsValidLodIndex(LodIndex) ? ClothSimulationLodModels[LodIndex].Positions : TConstArrayView<FVector3f>(); }
	TConstArrayView<FVector2f> GetPatternPositions(int32 LodIndex) const { return IsValidLodIndex(LodIndex) ? ClothSimulationLodModels[LodIndex].PatternPositions : TConstArrayView<FVector2f>(); }
	TConstArrayView<FVector3f> GetNormals(int32 LodIndex) const { return IsValidLodIndex(LodIndex) ? ClothSimulationLodModels[LodIndex].Normals : TConstArrayView<FVector3f>(); }
	TConstArrayView<uint32> GetIndices(int32 LodIndex) const { return IsValidLodIndex(LodIndex) ? ClothSimulationLodModels[LodIndex].Indices : TConstArrayView<uint32>(); }
	TConstArrayView<uint32> GetPatternIndices(int32 LodIndex) const { return IsValidLodIndex(LodIndex) ? ClothSimulationLodModels[LodIndex].PatternIndices : TConstArrayView<uint32>(); }
	TConstArrayView<uint32> GetPatternToWeldedIndices(int32 LodIndex) const { return IsValidLodIndex(LodIndex) ? ClothSimulationLodModels[LodIndex].PatternToWeldedIndices : TConstArrayView<uint32>(); }
	TConstArrayView<FClothVertBoneData> GetBoneData(int32 LodIndex) const { return IsValidLodIndex(LodIndex) ? ClothSimulationLodModels[LodIndex].BoneData : TConstArrayView<FClothVertBoneData>(); }
	TArray<TConstArrayView<TTuple<int32, int32, float>>> GetTethers(int32 LodIndex) const;

	void CalculateLODTransitionUpDownData(TArray<FChaosClothAssetLodTransitionDataCache>* InOutTransitionCache = nullptr);

	CHAOSCLOTHASSETENGINE_API void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize);

	/** Remap used bones to a new skeleton. */
	CHAOSCLOTHASSETENGINE_API void RemapBoneIndices(const TArray<int32>& BoneMap);
};
