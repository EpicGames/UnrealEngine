// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/TaskGraphInterfaces.h"
#include "Containers/Array.h"
#include "Math/Transform.h"
#include "Templates/UniquePtr.h"
#include "ClothingSystemRuntimeTypes.h"
#include "Dataflow/Interfaces/DataflowPhysicsSolver.h"

#define UE_API CHAOSCLOTHASSETENGINE_API

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7
class UChaosClothAsset;
class UChaosClothComponent;
namespace Chaos
{
	class FClothVisualization;
}
#endif

namespace Chaos
{
	class FClothingSimulationSolver;
	class FClothingSimulationMesh;
	class FClothingSimulationCloth;
	class FClothingSimulationConfig;
	class FClothingSimulationCollider;
	class FClothVisualizationNoGC;
}

class UChaosClothAssetBase;
struct FChaosClothSimulationModel;
struct FReferenceSkeleton;
struct FClothingSimulationCacheData;

namespace UE::Chaos::ClothAsset
{
	class FCollisionSourcesProxy;
	class IClothComponentAdapter;
	struct FClothSimulationContext;

	/**
	 * Cloth simulation proxy.
	 * Class used to share data between the cloth simulation and the component.
	 */
	class FClothSimulationProxy : public FDataflowPhysicsSolverProxy
	{
	public:
		UE_API explicit FClothSimulationProxy(const IClothComponentAdapter& ClothComponentAdapter);
		UE_API ~FClothSimulationProxy();

		FClothSimulationProxy() = delete;  // This object cannot be created without a valid reference to an owner component 
		FClothSimulationProxy(const FClothSimulationProxy&) = delete;  // Disable the copy as there must be a single unique proxy per component
		FClothSimulationProxy(FClothSimulationProxy&&) = delete;  // Disable the move to force it to be associated with a valid component reference
		FClothSimulationProxy& operator=(const FClothSimulationProxy&) = delete;
		FClothSimulationProxy& operator=(FClothSimulationProxy&&) = delete;

		UE_API virtual void PostConstructor();

		/** Start the simulation if it isn't paused or suspended and return true, or simply update the existing simulation data and return false otherwise. */
		UE_API bool Tick_GameThread(float DeltaTime);

		/** Wait for the parallel task to complete if one was running, and update the simulation data. */
		UE_API void CompleteParallelSimulation_GameThread();

		/** Returns true if a parallel simulation task has been launched and the simulation data has not been updated, i.e., CompleteParallelSimulation_GameThread will do something. */
		bool IsParallelSimulationTaskValid() const
		{
			return IsValidRef(ParallelTask);
		}

		/** Prepare the proxy before the simulation is setup. For specialized usage when not calling Tick_GameThread. 
		 * @param bForceWaitForInitialization will ignore the value of p.ChaosClothAsset.WaitForAsyncClothInitialization and force waiting for any in flight initialization, guaranteeing successful preprocessing..*/
		UE_API void PreProcess_GameThread(float DeltaTime, bool bForceWaitForInitialization = false);
		/** Prepare the simulation data. For specialized usage when not calling Tick_GameThread. */
        UE_API bool PreSimulate_GameThread(float DeltaTime);
		/**
		 * Write data back onto the game thread once the simulation is done.
		 * For specialized usage when not calling Tick_GameThread.
		 * @param OutClothSimulDataOverride An optional override to store the resulting simulation data, or nullptr to use the internally owned map insted.
		 */
		UE_API void PostSimulate_GameThread(TMap<int32, FClothSimulData>* OutClothSimulDataOverride = nullptr);
		/**
		 * Post setup, required for when the simulation didn't run.
		 * For specialized usage when not calling Tick_GameThread.
		 * @param OutClothSimulDataOverride An optional override to store the resulting simulation data, or nullptr to use the internally owned map insted.
		 */
		UE_API void PostProcess_GameThread(TMap<int32, FClothSimulData>* OutClothSimulDataOverride = nullptr);

		/* Force a pending cloth reset. For specialized usage when not calling Tick_GameThread. */
		UE_API void ForcePendingReset_GameThread();

		/** Hard reset the simulation after config changes. */
		UE_API void HardResetSimulation_GameThread();

		/**
		 * Return a map of all simulation data as used by the skeletal rendering code.
		 * The map key is the rendering section's cloth index as set in FSkelMeshRenderSection::CorrespondClothAssetIndex,
		 * which is 0 for the entire cloth component since all of its sections share the same simulation data.
		 */
		UE_API const TMap<int32, FClothSimulData>& GetCurrentSimulationData_AnyThread() const;

		UE_API FBoxSphereBounds CalculateBounds_AnyThread() const;

		UE_API const ::Chaos::FClothVisualizationNoGC* GetClothVisualization() const;

		int32 GetNumCloths() const { return NumCloths; }
		int32 GetNumKinematicParticles() const { return NumKinematicParticles; }
		int32 GetNumDynamicParticles() const { return NumDynamicParticles; }
		int32 GetNumIterations() const { return NumIterations; }
		int32 GetNumSubsteps() const { return NumSubsteps; }
		int32 GetNumLinearSolveIterations() const { return LastLinearSolveIterations; }
		float GetLinearSolveError() const { return LastLinearSolveError; }
		float GetSimulationTime() const { return SimulationTime; }
		bool IsTeleported() const { return bIsTeleported; }
		UE_DEPRECATED(5.6, "CacheData is reset after the context is updated, making this method unreliable.")
		bool HasCacheData() const { return CacheData.IsValid(); }

	protected:
		UE_API void PostConstructorInternal(bool bAsyncInitialization);
		UE_API void Tick();
		UE_API void WriteSimulationData(TMap<int32, FClothSimulData>* OutClothSimulDataOverride = nullptr);
		UE_API void InitializeConfigs();

		UE_API void PreProcess_Internal(float DeltaTime);

	private:
		UE_API void FillSimulationContext(float DeltaTime, bool bIsInitialization = false);

		// Begin proxy initialization. Can't be done in parallel.
		UE_API void BeginInitialization_GameThread();
		// Thread-safe part of Initialization.
		UE_API void ExecuteInitialization();
		// Wait for any in-flight initialization.
		UE_API void WaitForParallelInitialization_GameThread();
		// Wait for any in-flight initialization and complete the initialization process. Can't be done in parallel.
		UE_API void CompleteInitialization_GameThread();

		UE_API bool ShouldEnableSolver(bool bSolverCurrentlyEnabled) const;
		UE_API void UpdateClothLODs();

		// Begin FDataflowPhysicsSolverProxy overrides
		virtual void AdvanceSolverDatas(const float DeltaTime) override
		{
			Tick();
		}
		// End FDataflowPhysicsSolverProxy overrides

		// Internal physics thread object
		friend class FClothSimulationProxyParallelTask;

		// Allow FSKMClothingSimulation to call private methods like Tick()
		friend class FSKMClothingSimulation;

		// Reference for the cloth parallel task, to detect whether or not a simulation is running
		FGraphEventRef ParallelTask;

		// Reference for the cloth async initialization parallel task, to detect whether or not initialization is not complete.
		FGraphEventRef ParallelInitializationTask;

		// Simulation data written back to the component after the simulation has taken place
		TMap<int32, FClothSimulData> CurrentSimulationData;

		// Owner component
		const IClothComponentAdapter& ClothComponentAdapter;

		// Simulation context used to store the required component data for the duration of the simulation
		TUniquePtr<FClothSimulationContext> ClothSimulationContext;

		// The collision data for the external collision sources
		TUniquePtr<FCollisionSourcesProxy> CollisionSourcesProxy;

		// The cloth simulation model used to create this simulation, ownership might get transferred to this proxy if it changes during the simulation
		TArray<TSharedPtr<const FChaosClothSimulationModel>> ClothSimulationModels;

		// Simulation objects
		TUniquePtr<::Chaos::FClothingSimulationSolver> Solver;
		TArray<TUniquePtr<::Chaos::FClothingSimulationMesh>> Meshes;
		TArray<TUniquePtr<::Chaos::FClothingSimulationCloth>> Cloths;
		TArray<TUniquePtr<::Chaos::FClothingSimulationConfig>> Configs;
		TArray<TUniquePtr<::Chaos::FClothingSimulationCollider>> Colliders;

		TUniquePtr<::Chaos::FClothVisualizationNoGC> Visualization;

		// Chaos Cache needs to have access to the solver.
		friend class FClothComponentCacheAdapter;

		// Additional data used by the cache adapter
		enum struct ESolverMode : uint8
		{
			Default = 0, // Default behavior. Enable solver if no cache data available.
			EnableSolverForSimulateRecord = 1, // Normal simulation. Also used when Recording.
			DisableSolverForPlayback = 2, // Solver is disabled. Used when live playing back cache.
		};
		TUniquePtr<FClothingSimulationCacheData> CacheData;
		ESolverMode SolverMode = ESolverMode::Default;

		// Properties that must be readable from all threads
		std::atomic<int32> NumCloths;
		std::atomic<int32> NumKinematicParticles;
		std::atomic<int32> NumDynamicParticles;
		std::atomic<int32> NumIterations;
		std::atomic<int32> NumSubsteps;
		std::atomic<float> SimulationTime;
		std::atomic<bool> bIsTeleported;
		std::atomic<int32> LastLinearSolveIterations; // For single cloth only.
		std::atomic<float> LastLinearSolveError; // For single cloth only.

		// Cached value of the MaxPhysicsDeltaTime setting for the life of this proxy
		const float MaxDeltaTime;

		mutable bool bHasInvalidReferenceBoneTransforms : 1 = false;
		mutable bool bHasNonMatchingReferenceBones : 1 = false;
		bool bNeedResetClothFromNanCheck : 1 = false;
		bool bIsSimulating : 1 = false;  // Whether the proxy has been running the simulation, for internal use only
		bool bIsInitialized : 1 = false;  // Whether the proxy has finished initialization, for internal use only
		bool bIsPreProcessed : 1 = false;  // Whether the proxy was initialized when Tick_GameThread/PreProcess_GameThread was called, for internal use only
	};

#if !defined(CHAOS_TRANSFORM_CLOTH_SIMUL_DATA_ISPC_ENABLED_DEFAULT)
#define CHAOS_TRANSFORM_CLOTH_SIMUL_DATA_ISPC_ENABLED_DEFAULT 1
#endif

	// Support run-time toggling on supported platforms in non-shipping configurations
#if !INTEL_ISPC || UE_BUILD_SHIPPING
	static constexpr bool bTransformClothSimulData_ISPC_Enabled = INTEL_ISPC && CHAOS_TRANSFORM_CLOTH_SIMUL_DATA_ISPC_ENABLED_DEFAULT;
#else
	extern bool bTransformClothSimulData_ISPC_Enabled;
#endif
}

#undef UE_API
