// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Features/IModularFeature.h"
#include "SkeletalMeshConverterClassProvider.generated.h"

#define UE_API CHAOSCLOTHASSETENGINE_API

/** Modular base class for skeletal mesh conversion. */
UCLASS(MinimalAPI, Abstract)
class UClothAssetSkeletalMeshConverter : public UObject
{
	GENERATED_BODY()

public:
	/** Build a skeletal mesh from the specified Cloth Asset. */
	virtual bool ExportToSkeletalMesh(const class UChaosClothAssetBase& ClothAssetBase, class USkeletalMesh& SkeletalMesh) const
		PURE_VIRTUAL(UClothAssetSkeletalMeshConverter::ExportToSkeletalMesh, return false;);
};

/**
 * A modular interface to provide ways to build a SkeletalMesh from a DynamicMesh, which can be used to build a SkeletalMesh from a ClothCollection.
 * This cannot be done in a non Editor module due to the required dependency being in a Developper folder,
 * and instead is exposed as a modular feature in order to be called from the UChaosClothAssetBase engine class.
 */
class IClothAssetSkeletalMeshConverterClassProvider : public IModularFeature
{
public:
	inline static const FName FeatureName = TEXT("IClothAssetSkeletalMeshConverterClassProvider");

	virtual TSubclassOf<UClothAssetSkeletalMeshConverter> GetClothAssetSkeletalMeshConverter() const = 0;
};

#undef UE_API
