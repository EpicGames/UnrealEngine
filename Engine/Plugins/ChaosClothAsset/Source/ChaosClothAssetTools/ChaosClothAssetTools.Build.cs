// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ChaosClothAssetTools : ModuleRules
{
	public ChaosClothAssetTools(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[] {
			"MeshConversion",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"Engine",
				"CoreUObject",
				"RenderCore",
				"RHI",
				"Chaos",
				"ChaosCloth",
				"ChaosClothEditor",  // For Chaos::FSimulationEditorExtender
				"ChaosClothAsset",
				"ChaosClothAssetEngine",
				"DataflowEditor",
				"DataflowEngine",
				"SlateCore",
				"ToolWidgets",
				"UnrealEd",
				"Projects",
				"ClothPainter",                 // For clothing asset exporter
				"ClothingSystemRuntimeInterface",
				"ClothingSystemRuntimeCommon",
				"ClothingSystemEditorInterface",  // For UChaosClothAssetSKMClothingAssetFactory
				"GeometryCore",					// For DynamicMesh conversion
				"MeshConversion",
				"MeshDescription",
				"SkeletalMeshDescription",		// For FSkeletalMeshAttributes::DefaultSkinWeightProfileName
				"SkeletalMeshUtilitiesCommon"
			}
		);

		PrivateIncludePaths.Add(System.IO.Path.Combine(GetModuleDirectory("ChaosClothAssetTools"), "Internal"));
	}
}
