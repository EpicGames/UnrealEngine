// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "ThumbnailRendering/SkeletalMeshThumbnailRenderer.h"

#include "ClothAssetThumbnailRenderer.generated.h"

class UChaosClothComponent;
class UChaosClothAssetBase;

/**
 * Minimal actor with a cloth component used to render cloth assets previews
 */
UCLASS(MinimalAPI)
class AChaosClothPreviewActor_Internal : public AActor
{
	GENERATED_BODY()
public:
	AChaosClothPreviewActor_Internal();

	UChaosClothComponent* GetClothComponent() const
	{
		return ClothComponent;
	}

protected:
	UPROPERTY()
	TObjectPtr<UChaosClothComponent> ClothComponent;
};

namespace UE::Chaos::ClothAsset
{
	/**
	 * Preview scene for a Cloth asset thumbnail
	 */
	class FThumbnailScene_Internal : public FThumbnailPreviewScene
	{
	public:
		FThumbnailScene_Internal();

		void SetClothAsset(UChaosClothAssetBase* InClothAsset);

		virtual void GetViewMatrixParameters(const float InFOVDegrees, FVector& OutOrigin, float& OutOrbitPitch, float& OutOrbitYaw, float& OutOrbitZoom) const override;

	private:
		AChaosClothPreviewActor_Internal* PreviewActor = nullptr;
	};
}


/**
 * Renders a preview scene to be used as a thumbnail for a supported cloth asset
 */
UCLASS(MinimalAPI)
class UChaosClothAssetThumbnailRenderer_Internal : public UDefaultSizedThumbnailRenderer
{
	GENERATED_BODY()

	virtual void Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget* Viewport, FCanvas* Canvas, bool bAdditionalViewFamily) override;
	virtual bool CanVisualizeAsset(UObject* Object) override;
	virtual EThumbnailRenderFrequency GetThumbnailRenderFrequency(UObject* Object) const override;

	virtual void BeginDestroy() override;

protected:
	TObjectInstanceThumbnailScene<UE::Chaos::ClothAsset::FThumbnailScene_Internal, 128> ClothThumbnailSceneCache;
};
