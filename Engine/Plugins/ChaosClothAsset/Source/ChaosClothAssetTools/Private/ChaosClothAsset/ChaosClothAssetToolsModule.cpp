// Copyright Epic Games, Inc. All Rights Reserved.

#include "Features/IModularFeatures.h"
#include "Modules/ModuleManager.h"
#include "ChaosClothAsset/ClothAsset.h"
#include "ChaosClothAsset/ClothAssetEditorStyle.h"
#include "ChaosClothAsset/ClothAssetSKMClothingAssetFactory.h"
#include "ChaosClothAsset/ClothAssetThumbnailRenderer.h"
#include "ChaosClothAsset/ClothComponentEditorStyle.h"
#include "ChaosClothAsset/ClothingAssetToClothAssetExporter.h"
#include "ChaosClothAsset/SkeletalMeshConverter.h"
#include "ChaosClothAsset/ClothAssetSKMSimulationEditorExtender.h"
#include "ClothingSystemEditorInterfaceModule.h"
#include "ThumbnailRendering/ThumbnailManager.h"

namespace UE::Chaos::ClothAsset
{
	class FChaosClothAssetToolsModule
		: public IModuleInterface
		, public IClothingAssetExporterClassProvider
		, public IClothAssetSkeletalMeshConverterClassProvider
		, public IClothingAssetFactoryProvider

	{
	public:
		// IModuleInterface implementation
		virtual void StartupModule() override
		{
			// Register asset icons
			FClothAssetEditorStyle::Get();
			FClothComponentEditorStyle::Get();

			// Register modular features
			IModularFeatures::Get().RegisterModularFeature(IClothingAssetExporterClassProvider::FeatureName, static_cast<IClothingAssetExporterClassProvider*>(this));
			IModularFeatures::Get().RegisterModularFeature(IClothAssetSkeletalMeshConverterClassProvider::FeatureName, static_cast<IClothAssetSkeletalMeshConverterClassProvider*>(this));
			IModularFeatures::Get().RegisterModularFeature(IClothingAssetFactoryProvider::FeatureName, static_cast<IClothingAssetFactoryProvider*>(this));
			IModularFeatures::Get().RegisterModularFeature(FClothingSystemEditorInterfaceModule::ExtenderFeatureName, &SKMSimulationEditorExtender);

			// Register thumbnail renderer
			UThumbnailManager::Get().RegisterCustomRenderer(UChaosClothAsset::StaticClass(), UChaosClothAssetThumbnailRenderer_Internal::StaticClass());
		}

		virtual void ShutdownModule() override
		{
			if (UObjectInitialized())
			{
				// Unregister modular features
				IModularFeatures::Get().UnregisterModularFeature(IClothingAssetExporterClassProvider::FeatureName, static_cast<IClothingAssetExporterClassProvider*>(this));
				IModularFeatures::Get().UnregisterModularFeature(IClothAssetSkeletalMeshConverterClassProvider::FeatureName, static_cast<IClothAssetSkeletalMeshConverterClassProvider*>(this));
				IModularFeatures::Get().UnregisterModularFeature(IClothingAssetFactoryProvider::FeatureName, static_cast<IClothingAssetFactoryProvider*>(this));
				IModularFeatures::Get().UnregisterModularFeature(FClothingSystemEditorInterfaceModule::ExtenderFeatureName, &SKMSimulationEditorExtender);

				// Unregister thumbnail renderer
				UThumbnailManager::Get().UnregisterCustomRenderer(UChaosClothAsset::StaticClass());
			}
		}

		//~ Begin IClothingSimulationFactoryClassProvider implementation
		virtual TSubclassOf<UClothingAssetExporter> GetClothingAssetExporterClass() const override
		{
			return UClothingAssetToChaosClothAssetExporter::StaticClass();
		}
		//~ End IClothingSimulationFactoryClassProvider implementation

		//~ Begin IClothAssetSkeletalMeshConverterClassProvider implementation
		virtual TSubclassOf<UClothAssetSkeletalMeshConverter> GetClothAssetSkeletalMeshConverter() const override
		{
			return UClothAssetEditorSkeletalMeshConverter::StaticClass();
		}
		//~ End IClothAssetSkeletalMeshConverterClassProvider implementation

		//~ Begin IClothingAssetFactoryProvider
		virtual UClothingAssetFactoryBase* GetFactory() override
		{
			return UChaosClothAssetSKMClothingAssetFactory::StaticClass()->GetDefaultObject<UClothingAssetFactoryBase>();
		}
		//~ End IClothingAssetFactoryProvider

	private:
		UE::Chaos::ClothAsset::FSKMSimulationEditorExtender SKMSimulationEditorExtender;
	};
}

IMPLEMENT_MODULE(UE::Chaos::ClothAsset::FChaosClothAssetToolsModule, ChaosClothAssetTools);
