// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ClothAssetEditorStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Interfaces/IPluginManager.h"

namespace UE::Chaos::ClothAsset
{
	TOptional<FClothAssetEditorStyle> FClothAssetEditorStyle::Singleton;

	FClothAssetEditorStyle::FClothAssetEditorStyle()
		: FSlateStyleSet("ClothAssetEditorStyle")
	{
		if (const TSharedPtr<IPlugin> ChaosClothAssetPlugin = IPluginManager::Get().GetModuleOwnerPlugin(UE_MODULE_NAME))
		{
			SetContentRoot(ChaosClothAssetPlugin->GetBaseDir() / TEXT("Resources"));

			Set("ClassIcon.ChaosClothAsset", new FSlateVectorImageBrush(RootToContentDir(TEXT("ClothAsset_16.svg")), FVector2D(16)));
			Set("ClassThumbnail.ChaosClothAsset", new FSlateVectorImageBrush(RootToContentDir(TEXT("ClothAsset_64.svg")), FVector2D(64)));

			Set("ClassIcon.ChaosClothPreset", new FSlateVectorImageBrush(RootToContentDir(TEXT("ClothPreset_16.svg")), FVector2D(16)));
			Set("ClassThumbnail.ChaosClothPreset", new FSlateVectorImageBrush(RootToContentDir(TEXT("ClothPreset_64.svg")), FVector2D(64)));
		}
		FSlateStyleRegistry::RegisterSlateStyle(*this);
	}

	FClothAssetEditorStyle::~FClothAssetEditorStyle()
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*this);
	}
}  // End namespace UE::Chaos::ClothAsset
