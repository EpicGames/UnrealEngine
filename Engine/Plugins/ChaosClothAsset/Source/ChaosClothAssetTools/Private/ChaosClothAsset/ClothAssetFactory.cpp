// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ClothAssetFactory.h"
#include "ChaosClothAsset/ClothAsset.h"
#include "Dataflow/AssetDefinition_DataflowAsset.h"
#include "Dialog/SMessageDialog.h"
#include "HAL/IConsoleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClothAssetFactory)

#define LOCTEXT_NAMESPACE "ChaosClothAssetFactory"

DEFINE_LOG_CATEGORY_STATIC(LogChaosClothAssetFactory, Log, All);

UChaosClothAssetFactory::UChaosClothAssetFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bEditorImport = true;
	bEditAfterNew = true;
	SupportedClass = UChaosClothAsset::StaticClass();
}

UObject* UChaosClothAssetFactory::CreateClothAssetFromTemplate(UClass* Class, UObject* Parent, FName Name, EObjectFlags Flags, const FString* TemplatePath, bool bEmbedDataflow)
{
	static const TCHAR* AssetPrefix = TEXT("CA_");
	return UE::DataflowAssetDefinitionHelpers::FactoryCreateNew(Class, Parent, Name, Flags, TemplatePath, bEmbedDataflow, AssetPrefix);
}

UObject* UChaosClothAssetFactory::FactoryCreateNew(UClass* Class, UObject* Parent, FName Name, EObjectFlags Flags, UObject* /*Context*/, FFeedbackContext* /*Warn*/)
{
	const bool bIsNotInteractive = GIsAutomationTesting || FApp::IsUnattended() || IsRunningCommandlet() || GIsRunningUnattendedScript;
	if (bIsNotInteractive)
	{
		// in non interactive mode we need to set a default Dataflow asset
		static const FString EmptyClothTemplatePath(TEXT("/ChaosClothAsset/DF_EmptyClothAssetTemplate.DF_EmptyClothAssetTemplate"));
		static const TCHAR* AssetPrefix = TEXT("CA_");
		constexpr bool bEmbedDataflow = true;
		return UE::DataflowAssetDefinitionHelpers::FactoryCreateNew(Class, Parent, Name, Flags, &EmptyClothTemplatePath, bEmbedDataflow, AssetPrefix);
	}

	UObject* const NewAsset = NewObject<UObject>(Parent, Class, Name, Flags | RF_Transactional | RF_Public | RF_Standalone);
	if (!NewAsset)
	{
		const FText ErrorMessage = FText::Format(LOCTEXT("ErrorCreatingNewAsset", "Failed to create a new asset {0}."), FText::FromName(Name));
		UE_LOGF(LogChaosClothAssetFactory, Error, "%ls", *ErrorMessage.ToString());
		return nullptr;
	}
	NewAsset->MarkPackageDirty();

	// No default blank option since cloth templates already provide one
	UE::DataflowAssetDefinitionHelpers::SetDataflowFromTemplatePicker(NewAsset, /*bShowDefaultBlankOption*/ false);
	return NewAsset;
}

FString UChaosClothAssetFactory::GetDefaultNewAssetName() const
{
	return FString(TEXT("CA_NewChaosClothAsset"));
}

#undef LOCTEXT_NAMESPACE
