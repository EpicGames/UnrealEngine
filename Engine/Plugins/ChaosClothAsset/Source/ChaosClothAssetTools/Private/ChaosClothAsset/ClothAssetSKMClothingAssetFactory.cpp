// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ClothAssetSKMClothingAssetFactory.h"
#include "ChaosClothAsset/ClothAssetBase.h"
#include "Engine/SkeletalMesh.h"
#include "ChaosClothAsset/ClothAssetSKMClothingAsset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClothAssetSKMClothingAssetFactory)

UChaosClothAssetSKMClothingAssetFactory::UChaosClothAssetSKMClothingAssetFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

const UClass* UChaosClothAssetSKMClothingAssetFactory::GetSupportedSourceAssetType() const
{
	return UChaosClothAssetBase::StaticClass();
}

TArray<UClothingAssetBase*> UChaosClothAssetSKMClothingAssetFactory::CreateFromSourceAsset(USkeletalMesh* TargetMesh, const UObject* SourceAsset) const
{
	TArray<UClothingAssetBase*> ClothingAssets;
	if (const UChaosClothAssetBase* const ClothAsset = Cast<UChaosClothAssetBase>(SourceAsset))
	{
		const FName AssetName = StaticFindObject(UClothingAssetBase::StaticClass(), TargetMesh, *ClothAsset->GetName()) ?
			MakeUniqueObjectName(TargetMesh, UClothingAssetBase::StaticClass(), ClothAsset->GetFName()) :
			ClothAsset->GetFName();
		UChaosClothAssetSKMClothingAsset* const ClothAssetSKMClothingAsset = NewObject<UChaosClothAssetSKMClothingAsset>(TargetMesh, AssetName);
		ClothAssetSKMClothingAsset->SetAsset(Cast<UChaosClothAssetBase>(SourceAsset));
		ClothingAssets.Emplace(ClothAssetSKMClothingAsset);
	}
	return ClothingAssets;
}
