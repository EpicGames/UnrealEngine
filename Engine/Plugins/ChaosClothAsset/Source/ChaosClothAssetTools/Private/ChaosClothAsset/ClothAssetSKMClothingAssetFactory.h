// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ClothingAssetFactoryInterface.h"
#include "ClothAssetSKMClothingAssetFactory.generated.h"

UCLASS(MinimalAPI)
class UChaosClothAssetSKMClothingAssetFactory : public UClothingAssetFactoryBase
{
	GENERATED_BODY()
public:
	UChaosClothAssetSKMClothingAssetFactory(const FObjectInitializer& ObjectInitializer);

	//~ Begin UClothingAssetFactoryBase Interface
	virtual const UClass* GetSupportedSourceAssetType() const override;

	virtual TArray<UClothingAssetBase*> CreateFromSourceAsset(USkeletalMesh* TargetMesh, const UObject* SourceAsset) const override;

	virtual UClothingAssetBase* CreateFromSkeletalMesh(USkeletalMesh* TargetMesh, FSkeletalMeshClothBuildParams& Params) override
	{
		return nullptr;  // TODO: Implement create from skeletal mesh
	}

	virtual UClothingAssetBase* CreateFromExistingCloth(USkeletalMesh* /*TargetMesh*/, USkeletalMesh* /*SourceMesh*/, UClothingAssetBase* /*SourceAsset*/) override
	{
		return nullptr;  // TODO: Implement create from another clothing data
	}

	virtual UClothingAssetBase* ImportLodToClothing(USkeletalMesh* TargetMesh, FSkeletalMeshClothBuildParams& Params) override
	{
		return nullptr;
	}
	//~ End UClothingAssetFactoryBase Interface
};
