// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ClothAssetSKMSimulationEditorExtender.h"
#include "ChaosClothAsset/ClothAssetSKMClothingSimulation.h"

#define LOCTEXT_NAMESPACE "ChaosSimulationEditorExtender"

namespace UE::Chaos::ClothAsset
{
	FSKMSimulationEditorExtender::FSKMSimulationEditorExtender()
		: ::Chaos::FSimulationEditorExtender()
	{
	}

	FSKMSimulationEditorExtender::~FSKMSimulationEditorExtender() = default;

	UClass* FSKMSimulationEditorExtender::GetSupportedSimulationFactoryClass()
	{
		return UChaosClothAssetSKMClothingSimulationFactory::StaticClass();
	}

	FName FSKMSimulationEditorExtender::GetSectionName() const
	{
		return FName("ClothAsset_Visualizations");
	}

	FText FSKMSimulationEditorExtender::GetSectionHeadingText() const
	{
		return FText(LOCTEXT("ClothAssetVisualizationSection", "Cloth Asset Visualization"));
	}

	const ::Chaos::FClothVisualizationNoGC* FSKMSimulationEditorExtender::GetClothVisualization(const IClothingSimulationInterface* Simulation) const
	{
		return Simulation ? static_cast<const FSKMClothingSimulation*>(Simulation)->GetClothVisualization() : nullptr;
	}
}  // namespace UE::Chaos::ClothAsset

#undef LOCTEXT_NAMESPACE
