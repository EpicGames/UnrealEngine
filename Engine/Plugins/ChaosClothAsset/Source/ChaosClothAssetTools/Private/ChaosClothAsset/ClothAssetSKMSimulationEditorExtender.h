// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosClothEditor/ChaosSimulationEditorExtender.h"

namespace Chaos
{
	class FClothVisualizationNoGC;
}

namespace UE::Chaos::ClothAsset
{
	/** Chaos extension to the asset editor. */
	class FSKMSimulationEditorExtender final : public ::Chaos::FSimulationEditorExtender
	{
	public:
		FSKMSimulationEditorExtender();
		virtual ~FSKMSimulationEditorExtender() override;

	private:
		//~ Begin ISimulationEditorExtender Interface
		virtual UClass* GetSupportedSimulationFactoryClass() override;
		//~ End ISimulationEditorExtender Interface

		//~ Begin ::Chaos::FSimulationEditorExtender Interface
		virtual FName GetSectionName() const override;
		virtual FText GetSectionHeadingText() const override;
		virtual const ::Chaos::FClothVisualizationNoGC* GetClothVisualization(const IClothingSimulationInterface* Simulation) const override;
		//~ End ::Chaos::FSimulationEditorExtender Interface
	};
}
