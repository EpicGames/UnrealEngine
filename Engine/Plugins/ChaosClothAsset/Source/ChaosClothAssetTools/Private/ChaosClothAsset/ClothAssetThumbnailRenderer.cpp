// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ClothAssetThumbnailRenderer.h"

#include "ChaosClothAsset/ClothAssetBase.h"
#include "ChaosClothAsset/ClothComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "HAL/IConsoleManager.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "ThumbnailRendering/SceneThumbnailInfo.h"
#include "SceneView.h"
#include "ShowFlags.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClothAssetThumbnailRenderer)

AChaosClothPreviewActor_Internal::AChaosClothPreviewActor_Internal()
{
	ClothComponent = CreateDefaultSubobject<UChaosClothComponent>(TEXT("ClothComponent0"));
	RootComponent = ClothComponent;
}

namespace UE::Chaos::ClothAsset
{
	namespace Private
	{
		static TAutoConsoleVariable<bool> CVarThumbnailEnable(TEXT("p.ChaosClothAsset.Thumbnail.Enable"), true, TEXT("Enable Cloth Asset and Outfit Asset thumbnail generation.\n Default: True."));
	}

	FThumbnailScene_Internal::FThumbnailScene_Internal() : FThumbnailPreviewScene()
	{
		bForceAllUsedMipsResident = false;

		if (GetWorld())
		{
			FActorSpawnParameters SpawnInfo;
			SpawnInfo.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
			SpawnInfo.bNoFail = true;
			SpawnInfo.ObjectFlags = RF_Transient;

			PreviewActor = GetWorld()->SpawnActor<AChaosClothPreviewActor_Internal>(SpawnInfo);
			PreviewActor->SetActorEnableCollision(false);
		}
	}

	void FThumbnailScene_Internal::SetClothAsset(UChaosClothAssetBase* InClothAsset)
	{
		if (UChaosClothComponent* const ClothComponent = PreviewActor ? PreviewActor->GetClothComponent() : nullptr;
			ensure(ClothComponent))
		{
			ClothComponent->SetAsset(InClothAsset);
		}
	}

	void FThumbnailScene_Internal::GetViewMatrixParameters(
		const float InFOVDegrees,
		FVector& OutOrigin,
		float& OutOrbitPitch,
		float& OutOrbitYaw,
		float& OutOrbitZoom) const
	{	
		if (UChaosClothComponent* const ClothComponent = PreviewActor ? PreviewActor->GetClothComponent() : nullptr;
			ensure(ClothComponent))
		{
			FBoxSphereBounds Bounds = ClothComponent->Bounds;
			Bounds = Bounds.ExpandBy(2.0f);
		
			const float HalfFOVRadians = FMath::DegreesToRadians<float>(InFOVDegrees) * 0.5f;
			const float HalfMeshSize = static_cast<float>(Bounds.SphereRadius); 
			const float TargetDistance = HalfMeshSize / FMath::Tan(HalfFOVRadians);

			USceneThumbnailInfo* ThumbnailInfo = Cast<USceneThumbnailInfo>(ClothComponent->GetThumbnailInfo());
			if (ThumbnailInfo)
			{
				if (TargetDistance + ThumbnailInfo->OrbitZoom < 0 )
				{
					ThumbnailInfo->OrbitZoom = -TargetDistance;
				}
			}
			else
			{
				ThumbnailInfo = USceneThumbnailInfo::StaticClass()->GetDefaultObject<USceneThumbnailInfo>();
			}

			OutOrigin = -Bounds.Origin;
			OutOrbitPitch = ThumbnailInfo->OrbitPitch;
			OutOrbitYaw = ThumbnailInfo->OrbitYaw;
			OutOrbitZoom = TargetDistance + ThumbnailInfo->OrbitZoom;
		}
		else
		{
			OutOrigin = FVector::ZeroVector;
			OutOrbitPitch = 0.f;
			OutOrbitYaw = 0.f;
			OutOrbitZoom =0.f;
		}
	}
}

void UChaosClothAssetThumbnailRenderer_Internal::Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget* Viewport, FCanvas* Canvas, bool bAdditionalViewFamily)
{
	UChaosClothAssetBase* const ClothAsset = Cast<UChaosClothAssetBase>(Object);
	if (!ClothAsset)
	{
		return;
	}

	const TSharedRef<UE::Chaos::ClothAsset::FThumbnailScene_Internal> ThumbnailScene = ClothThumbnailSceneCache.EnsureThumbnailScene(Object);

	// Create a preview copy for assets that need special handling (sim-mesh-only cloth assets, multi-size outfits, etc.)
	UChaosClothAssetBase* const PreviewCopy = ClothAsset->CreatePreviewAssetCopy(GetTransientPackage(), RF_Transient | RF_TextExportTransient | RF_DuplicateTransient);

	ThumbnailScene->SetClothAsset(PreviewCopy ? PreviewCopy : ClothAsset);

	FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(Viewport, ThumbnailScene->GetScene(), FEngineShowFlags(ESFIM_Game))
		.SetTime(UThumbnailRenderer::GetTime())
		.SetAdditionalViewFamily(bAdditionalViewFamily));

	ViewFamily.EngineShowFlags.DisableAdvancedFeatures();
	ViewFamily.EngineShowFlags.MotionBlur = 0;
	ViewFamily.EngineShowFlags.LOD = 0;

	RenderViewFamily(Canvas, &ViewFamily, ThumbnailScene->CreateView(&ViewFamily, X, Y, Width, Height));
	ThumbnailScene->SetClothAsset(nullptr);
}

bool UChaosClothAssetThumbnailRenderer_Internal::CanVisualizeAsset(UObject* Object)
{
	return
		UE::Chaos::ClothAsset::Private::CVarThumbnailEnable.GetValueOnAnyThread() &&
		Cast<UChaosClothAssetBase>(Object) != nullptr;
}

EThumbnailRenderFrequency UChaosClothAssetThumbnailRenderer_Internal::GetThumbnailRenderFrequency(UObject* Object) const
{
	// OnAssetSave caches the thumbnail in the package and re-renders when the user saves
	return EThumbnailRenderFrequency::OnAssetSave;
}

void UChaosClothAssetThumbnailRenderer_Internal::BeginDestroy()
{
	ClothThumbnailSceneCache.Clear();
	Super::BeginDestroy();
}
