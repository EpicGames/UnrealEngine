// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClothComponentEditorStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Interfaces/IPluginManager.h"

namespace UE::Chaos::ClothAsset
{
	TOptional<FClothComponentEditorStyle> FClothComponentEditorStyle::Singleton;

	FClothComponentEditorStyle::FClothComponentEditorStyle()
		: FSlateStyleSet("ClothComponentEditorStyle")
	{
		if (const TSharedPtr<IPlugin> ChaosClothAssetPlugin = IPluginManager::Get().GetModuleOwnerPlugin(UE_MODULE_NAME))
		{
			SetContentRoot(ChaosClothAssetPlugin->GetBaseDir() / TEXT("Resources"));

			Set("ClassIcon.ChaosClothComponent", new FSlateVectorImageBrush(RootToContentDir(TEXT("ClothAsset_16.svg")), FVector2D(16)));
			Set("ClassThumbnail.ChaosClothComponent", new FSlateVectorImageBrush(RootToContentDir(TEXT("ClothAsset_64.svg")), FVector2D(64)));

		}
		FSlateStyleRegistry::RegisterSlateStyle(*this);
	}

	FClothComponentEditorStyle::~FClothComponentEditorStyle()
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*this);
	}
}  // End namespace UE::Chaos::ClothAsset
