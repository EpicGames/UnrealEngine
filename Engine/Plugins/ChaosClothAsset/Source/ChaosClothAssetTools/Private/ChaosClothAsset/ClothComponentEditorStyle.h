// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

namespace UE::Chaos::ClothAsset
{
	/**
	 * Editor style setting up the cloth asset icons in editor.
	 * TODO: Merge all styles to a single singleton.
	 */
	class FClothComponentEditorStyle final : public FSlateStyleSet
	{
	public:
		FClothComponentEditorStyle();
		virtual ~FClothComponentEditorStyle() override;

	public:
		static FClothComponentEditorStyle& Get()
		{
			if (!Singleton.IsSet())
			{
				Singleton.Emplace();
			}
			return Singleton.GetValue();
		}

		static void Destroy()
		{
			Singleton.Reset();
		}

	private:
		static TOptional<FClothComponentEditorStyle> Singleton;
	};
}  // End namespace UE::Chaos::ClothAsset
