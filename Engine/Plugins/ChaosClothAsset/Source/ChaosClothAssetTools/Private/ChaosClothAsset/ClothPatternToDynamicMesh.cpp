// Copyright Epic Games, Inc. All Rights Reserved. 

#if WITH_EDITOR

#include "ChaosClothAsset/ClothPatternToDynamicMesh.h"
#include "ChaosClothAsset/ClothPatternToDynamicMeshMappingSupport.h"
#include "ChaosClothAsset/ClothAsset.h"
#include "ChaosClothAsset/ClothComponent.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "ChaosClothAsset/ClothCollectionGroup.h"
#include "DynamicMesh/MeshNormals.h"
#include "Engine/SkeletalMesh.h"
#include "SkeletalMeshAttributes.h"
#include "ToDynamicMesh.h"

namespace UE::Chaos::ClothAsset
{
	namespace Private
	{
		//
		// Wrapper for accessing a Cloth Pattern. Implements the interface expected by TToDynamicMesh<>.
		//
		class FClothPatternWrapper
		{
		public:

			typedef int32 TriIDType;
			typedef int32 VertIDType;
			typedef int32 WedgeIDType;

			typedef int32 UVIDType;
			typedef int32 NormalIDType;
			typedef int32 ColorIDType;

			FClothPatternWrapper(
				const FCollectionClothConstFacade& ClothFacade,
				int32 PatternIndex,
				EClothPatternVertexType VertexDataType,
				bool bInFlip2DSimFaces = false,
				bool bInConvertWeightMaps = true)
				: VertexDataType(VertexDataType)
				, Cloth(ClothFacade)
				, bFlip2DSimFaces(bInFlip2DSimFaces)
				, bConvertWeightMaps(bInConvertWeightMaps)
			{
				NumTexCoords = 0;
				if (PatternIndex == INDEX_NONE)
				{
					// All patterns in one dynamic mesh
					switch (VertexDataType)
					{
					case EClothPatternVertexType::Render:
					{
						const int32 NumVertices = Cloth.GetNumRenderVertices();
						VertIDs.SetNum(NumVertices);
						TConstArrayView<TArray<FVector2f>> UVs = Cloth.GetRenderUVs();
						for (int32 VtxIndex = 0; VtxIndex < NumVertices; ++VtxIndex)
						{
							VertIDs[VtxIndex] = VtxIndex;
							NumTexCoords = FMath::Max(NumTexCoords, UVs[VtxIndex].Num());
						}
						NormalIDs = VertIDs;

						const TConstArrayView<FIntVector3> Indices = Cloth.GetRenderIndices();
						const int32 NumFaces = Cloth.GetNumRenderFaces();
						TriIDs.Reserve(NumFaces);
						for (int32 TriIndex = 0; TriIndex < NumFaces; ++TriIndex)
						{
							if (Indices[TriIndex][0] != INDEX_NONE &&
								Indices[TriIndex][1] != INDEX_NONE &&
								Indices[TriIndex][2] != INDEX_NONE)
							{
								TriIDs.Add(TriIndex);
							}
						}
					} break;
					case EClothPatternVertexType::Sim2D:
					{
						const int32 NumVertices = Cloth.GetNumSimVertices2D();
						const TConstArrayView<int32> SimVertex3DLookup = Cloth.GetSimVertex3DLookup();
						VertIDs.Reserve(NumVertices);
						NormalIDs.Reserve(NumVertices);
						for (int32 VtxIndex = 0; VtxIndex < NumVertices; ++VtxIndex)
						{
							if (SimVertex3DLookup[VtxIndex] != INDEX_NONE)
							{
								VertIDs.Add(VtxIndex);
								NormalIDs.Add(SimVertex3DLookup[VtxIndex]);
							}
						}

						const TConstArrayView<FIntVector3> Indices = Cloth.GetSimIndices2D();
						const int32 NumFaces = Cloth.GetNumSimFaces();
						TriIDs.Reserve(NumFaces);
						for (int32 TriIndex = 0; TriIndex < NumFaces; ++TriIndex)
						{
							if (Indices[TriIndex][0] != INDEX_NONE &&
								Indices[TriIndex][1] != INDEX_NONE &&
								Indices[TriIndex][2] != INDEX_NONE &&
								SimVertex3DLookup[Indices[TriIndex][0]] != INDEX_NONE &&
								SimVertex3DLookup[Indices[TriIndex][1]] != INDEX_NONE &&
								SimVertex3DLookup[Indices[TriIndex][2]] != INDEX_NONE)
							{
								TriIDs.Add(TriIndex);
							}
						}
					} break;
					case EClothPatternVertexType::Sim3D:
					{
						const int32 NumVertices = Cloth.GetNumSimVertices3D();
						VertIDs.SetNum(NumVertices);
						for (int32 VtxIndex = 0; VtxIndex < NumVertices; ++VtxIndex)
						{
							VertIDs[VtxIndex] = VtxIndex;
						}
						NormalIDs = VertIDs;

						const TConstArrayView<FIntVector3> Indices = Cloth.GetSimIndices3D();
						const int32 NumFaces = Cloth.GetNumSimFaces();
						TriIDs.Reserve(NumFaces);
						for (int32 TriIndex = 0; TriIndex < NumFaces; ++TriIndex)
						{
							if (Indices[TriIndex][0] != INDEX_NONE &&
								Indices[TriIndex][1] != INDEX_NONE &&
								Indices[TriIndex][2] != INDEX_NONE)
							{
								TriIDs.Add(TriIndex);
							}
						}
						NumTexCoords = 1; // SimPositions2D will be stored as UVs
					} break;
					default:
						checkNoEntry();
					}
				}
				else
				{
					switch (VertexDataType)
					{
					case EClothPatternVertexType::Render:
					{
						FCollectionClothRenderPatternConstFacade Pattern = Cloth.GetRenderPattern(PatternIndex);
						const int32 NumVertices = Pattern.GetNumRenderVertices();
						const int32 VertexOffset = Pattern.GetRenderVerticesOffset();
						VertIDs.SetNum(NumVertices);
						TConstArrayView<TArray<FVector2f>> UVs = Pattern.GetRenderUVs();
						for (int32 VtxIndex = 0; VtxIndex < NumVertices; ++VtxIndex)
						{
							VertIDs[VtxIndex] = VtxIndex + VertexOffset;
							NumTexCoords = FMath::Max(NumTexCoords, UVs[VtxIndex].Num());
						}
						NormalIDs = VertIDs;

						const TConstArrayView<FIntVector3> Indices = Pattern.GetRenderIndices();
						const int32 NumFaces = Pattern.GetNumRenderFaces();
						const int32 FaceOffset = Pattern.GetRenderFacesOffset();
						TriIDs.Reserve(NumFaces);
						for (int32 TriIndex = 0; TriIndex < NumFaces; ++TriIndex)
						{
							if (Indices[TriIndex][0] != INDEX_NONE &&
								Indices[TriIndex][1] != INDEX_NONE &&
								Indices[TriIndex][2] != INDEX_NONE)
							{
								TriIDs.Add(TriIndex + FaceOffset);
							}
						}
					} break;
					case EClothPatternVertexType::Sim2D:
					{
						FCollectionClothSimPatternConstFacade Pattern = Cloth.GetSimPattern(PatternIndex);
						const int32 NumVertices = Pattern.GetNumSimVertices2D();
						const TConstArrayView<int32> SimVertex3DLookup = Pattern.GetSimVertex3DLookup();
						const int32 VertexOffset = Pattern.GetSimVertices2DOffset();
						VertIDs.Reserve(NumVertices);
						NormalIDs.Reserve(NumVertices);
						for (int32 VtxIndex = 0; VtxIndex < NumVertices; ++VtxIndex)
						{
							if (SimVertex3DLookup[VtxIndex] != INDEX_NONE)
							{
								VertIDs.Add(VtxIndex + VertexOffset);
								NormalIDs.Add(SimVertex3DLookup[VtxIndex]);
							}
						}

						const TConstArrayView<FIntVector3> Indices = Pattern.GetSimIndices2D();
						const int32 NumFaces = Pattern.GetNumSimFaces();
						const int32 FaceOffset = Pattern.GetSimFacesOffset();
						TriIDs.Reserve(NumFaces);
						for (int32 TriIndex = 0; TriIndex < NumFaces; ++TriIndex)
						{
							if (Indices[TriIndex][0] != INDEX_NONE &&
								Indices[TriIndex][1] != INDEX_NONE &&
								Indices[TriIndex][2] != INDEX_NONE &&
								SimVertex3DLookup[Indices[TriIndex][0] - VertexOffset] != INDEX_NONE &&
								SimVertex3DLookup[Indices[TriIndex][1] - VertexOffset] != INDEX_NONE &&
								SimVertex3DLookup[Indices[TriIndex][2] - VertexOffset] != INDEX_NONE)
							{
								TriIDs.Add(TriIndex + FaceOffset);
							}
						}
					} break;
					case EClothPatternVertexType::Sim3D:
					{
						FCollectionClothSimPatternConstFacade Pattern = Cloth.GetSimPattern(PatternIndex);
						VertIDs = Pattern.GetSimVertex3DLookup();
						NormalIDs = Pattern.GetSimVertex3DLookup();

						const TConstArrayView<FIntVector3> Indices = Pattern.GetSimIndices3D();
						const int32 NumFaces = Pattern.GetNumSimFaces();
						const int32 FaceOffset = Pattern.GetSimFacesOffset();
						TriIDs.Reserve(NumFaces);
						for (int32 TriIndex = 0; TriIndex < NumFaces; ++TriIndex)
						{
							if (Indices[TriIndex][0] != INDEX_NONE &&
								Indices[TriIndex][1] != INDEX_NONE &&
								Indices[TriIndex][2] != INDEX_NONE)
							{
								TriIDs.Add(TriIndex + FaceOffset);
							}
						}
						NumTexCoords = 1; // SimPositions2D will be stored as UVs
					} break;
					default:
						checkNoEntry();
					}
				}

				//
				// Weight map layers precomputation
				//
				if (bConvertWeightMaps)
				{
					if (VertexDataType != EClothPatternVertexType::Render)
					{
						WeightMapNames = Cloth.GetWeightMapNames();
					}
					else
					{
						WeightMapNames = Cloth.GetUserDefinedAttributeNames<float>(ClothCollectionGroup::RenderVertices);
					}
				}

				// Set the reference skeleton if available
				const FSoftObjectPath& SkeletalMeshPathName = ClothFacade.GetSkeletalMeshSoftObjectPathName();
				const USkeletalMesh* const SkeletalMesh = Cast<USkeletalMesh>(SkeletalMeshPathName.TryLoad());
				RefSkeleton = SkeletalMesh ? &SkeletalMesh->GetRefSkeleton() : nullptr;
			}

			int32 NumTris() const
			{
				return TriIDs.Num();
			}

			int32 NumVerts() const
			{
				return VertIDs.Num();
			}

			int32 NumUVLayers() const
			{
				return NumTexCoords;
			}

			int32 NumWeightMapLayers() const
			{
				return WeightMapNames.Num();
			}

			FName GetWeightMapName(int32 LayerIndex) const
			{
				checkSlow(WeightMapNames.IsValidIndex(LayerIndex));
				return WeightMapNames[LayerIndex];
			}

			float GetVertexWeight(int32 LayerIndex, VertIDType VertexIndex) const
			{
				checkSlow(LayerIndex < WeightMapNames.Num());

				// All weights live on 3D indices.
				const int32 VertexWeightIndex = (VertexDataType == EClothPatternVertexType::Sim2D) ? Cloth.GetSimVertex3DLookup()[VertexIndex] : VertexIndex;

				TConstArrayView<float> WeightMap;

				if (VertexDataType != EClothPatternVertexType::Render)
				{
					WeightMap = Cloth.GetWeightMap(WeightMapNames[LayerIndex]);
				}
				else
				{
					WeightMap = Cloth.GetUserDefinedAttribute<float>(WeightMapNames[LayerIndex], ClothCollectionGroup::RenderVertices);
				}

				checkSlow(VertexWeightIndex < WeightMap.Num());
				if (VertexWeightIndex < WeightMap.Num())
				{
					return WeightMap[VertexWeightIndex];
				}
				else
				{
					return 0;
				}
			}

			// --"Vertex Buffer" info
			const TArray<VertIDType>& GetVertIDs() const
			{
				return VertIDs;
			}

			FVector3d GetPosition(VertIDType VtxID) const
			{
				switch (VertexDataType)
				{
				case EClothPatternVertexType::Render:
				{
					const TConstArrayView<FVector3f> RenderPositions = Cloth.GetRenderPosition();
					return FVector3d(RenderPositions[VtxID]);
				}
				case EClothPatternVertexType::Sim2D:
				{
					const TConstArrayView<FVector2f> SimPositions = Cloth.GetSimPosition2D();
					const FVector2f& Pos = SimPositions[VtxID];
					return FVector3d(Pos[0], Pos[1], 0.0);
				}
				case EClothPatternVertexType::Sim3D:
				{
					const TConstArrayView<FVector3f> SimPositions = Cloth.GetSimPosition3D();
					const FVector3f& Pos = SimPositions[VtxID];
					return FVector3d(Pos[0], Pos[1], Pos[2]);
				}
				default:
					checkNoEntry();
					return FVector3d();
				};
			}

			// --"Index Buffer" info
			const TArray<TriIDType>& GetTriIDs() const
			{
				return TriIDs;
			}

			bool GetTri(TriIDType TriID, VertIDType& VID0, VertIDType& VID1, VertIDType& VID2) const
			{
				FIntVector Face;

				switch (VertexDataType)
				{
				case EClothPatternVertexType::Render:
					Face = Cloth.GetRenderIndices()[TriID];
					break;
				case EClothPatternVertexType::Sim2D:
					Face = Cloth.GetSimIndices2D()[TriID];
					if (bFlip2DSimFaces)
					{
						Swap(Face[0], Face[1]);
					}
					break;
				case EClothPatternVertexType::Sim3D:
					Face = Cloth.GetSimIndices3D()[TriID];
					break;
				default:
					checkNoEntry();
				}

				VID0 = Face[0];
				VID1 = Face[1];
				VID2 = Face[2];

				return true;
			}

			bool HasNormals() const
			{
				return true;
			}

			bool HasTangents() const
			{
				return VertexDataType == EClothPatternVertexType::Render;
			}

			bool HasBiTangents() const
			{
				return VertexDataType == EClothPatternVertexType::Render;
			}

			bool HasColors() const
			{
				return VertexDataType == EClothPatternVertexType::Render;
			}

			// -- Access to per-wedge attributes -- //
			void GetWedgeIDs(const TriIDType& TriID, WedgeIDType& WID0, WedgeIDType& WID1, WedgeIDType& WID2) const
			{
				check(VertexDataType == EClothPatternVertexType::Sim3D);
				const FIntVector& Face = Cloth.GetSimIndices2D()[TriID];

				WID0 = Face[0];
				WID1 = Face[1];
				WID2 = Face[2];
			}

			FVector2f GetWedgeUV(int32 UVLayerIndex, WedgeIDType WID) const
			{
				check(VertexDataType == EClothPatternVertexType::Sim3D);
				return Cloth.GetSimPosition2D()[WID];
			}

			FVector3f GetWedgeNormal(WedgeIDType WID) const
			{
				checkf(false, TEXT("FClothPatternWrapper: ClothPatterns are not expected to use Wedges"));
				return FVector3f();
			}

			FVector3f GetWedgeTangent(WedgeIDType WID) const
			{
				checkf(false, TEXT("FClothPatternWrapper: ClothPatterns are not expected to use Wedges"));
				return FVector3f();
			}

			FVector3f GetWedgeBiTangent(WedgeIDType WID) const
			{
				checkf(false, TEXT("FClothPatternWrapper: ClothPatterns are not expected to use Wedges"));
				return FVector3f();
			}

			FVector4f GetWedgeColor(WedgeIDType WID) const
			{
				checkf(false, TEXT("FClothPatternWrapper: ClothPatterns are not expected to use Wedges"));
				return FVector4f();
			}
			// -- End of per-wedge attribute access -- //

			int32 GetMaterialIndex(TriIDType TriID) const
			{
				checkf(false, TEXT("FClothPatternWrapper: Material indexing should be accomplished by passing a function into Convert"));
				return 0;
			}

			int32 NumSkinWeightAttributes() const
			{
				return 1;
			}

			UE::AnimationCore::FBoneWeights GetVertexSkinWeight(int32 SkinWeightAttributeIndex, VertIDType InVertexID) const
			{
				using namespace UE::AnimationCore;

				checkfSlow(SkinWeightAttributeIndex == 0, TEXT("Cloth assets should only have one skin weight profile"));

				const bool bGetRenderMeshData = (VertexDataType == EClothPatternVertexType::Render);
				const TConstArrayView<TArray<int32>> BoneIndices = bGetRenderMeshData ? Cloth.GetRenderBoneIndices() : Cloth.GetSimBoneIndices();
				const TConstArrayView<TArray<float>> BoneWeights = bGetRenderMeshData ? Cloth.GetRenderBoneWeights() : Cloth.GetSimBoneWeights();

				// All weights live on 3D indices. Need to convert 2D index to 3D index.
				const VertIDType VertexID = (VertexDataType == EClothPatternVertexType::Sim2D) ? Cloth.GetSimVertex3DLookup()[InVertexID] : InVertexID;

				if (ensure(VertexID >= 0 && VertexID < BoneIndices.Num()))
				{
					const TArray<int32> Indices = BoneIndices[VertexID];
					const TArray<float> Weights = BoneWeights[VertexID];
					const int32 NumInfluences = Indices.Num();
					check(Weights.Num() == NumInfluences);

					TArray<FBoneWeight> BoneWeightArray;
					BoneWeightArray.SetNumUninitialized(NumInfluences);

					for (int32 Idx = 0; Idx < NumInfluences; ++Idx)
					{
						BoneWeightArray[Idx] = FBoneWeight(static_cast<FBoneIndexType>(Indices[Idx]), Weights[Idx]);
					}

					return FBoneWeights::Create(BoneWeightArray, FBoneWeightsSettings());
				}
				else
				{
					return FBoneWeights();
				}
			}

			FName GetSkinWeightAttributeName(int32 SkinWeightAttributeIndex) const
			{
				checkfSlow(SkinWeightAttributeIndex == 0, TEXT("Cloth assets should only have one skin weight profile"));

				return FSkeletalMeshAttributes::DefaultSkinWeightProfileName;
			}

			int32 GetNumBones() const
			{
				return RefSkeleton ? RefSkeleton->GetRawBoneNum() : 0;
			}

			FName GetBoneName(int32 BoneIdx) const
			{
				if (ensure(BoneIdx >= 0 && BoneIdx < GetNumBones()) && RefSkeleton)
				{
					return RefSkeleton->GetRawRefBoneInfo()[BoneIdx].Name;
				}

				return NAME_None;
			}

			int32 GetBoneParentIndex(int32 BoneIdx) const
			{
				if (ensure(BoneIdx >= 0 && BoneIdx < GetNumBones()) && RefSkeleton)
				{
					return RefSkeleton->GetRawRefBoneInfo()[BoneIdx].ParentIndex;
				}

				return INDEX_NONE;
			}

			FTransform GetBonePose(int32 BoneIdx) const
			{
				if (ensure(BoneIdx >= 0 && BoneIdx < GetNumBones()) && RefSkeleton)
				{
					return RefSkeleton->GetRawRefBonePose()[BoneIdx];
				}

				return FTransform::Identity;
			}

			FVector4f GetBoneColor(int32 BoneIdx) const
			{
				return FVector4f::One();
			}

			const TArray<int32>& GetNormalIDs() const
			{
				return NormalIDs;
			}

			FVector3f GetNormal(NormalIDType ID) const
			{
				return (VertexDataType == EClothPatternVertexType::Render) ? Cloth.GetRenderNormal()[ID] : Cloth.GetSimNormal()[ID];
			}

			bool GetNormalTri(const TriIDType& TriID, NormalIDType& ID0, NormalIDType& ID1, NormalIDType& ID2) const
			{
				if (VertexDataType == EClothPatternVertexType::Sim2D)
				{
					// All normal data lives on 3D indices.
					const FIntVector Face = Cloth.GetSimIndices3D()[TriID];

					ID0 = Face[0];
					ID1 = Face[1];
					ID2 = Face[2];
					if (bFlip2DSimFaces)
					{
						Swap(ID0, ID1);
					}

					return true;
				}
				else
				{
					return GetTri(TriID, ID0, ID1, ID2);
				}
			}

			const TArray<int32>& GetUVIDs(int32 LayerID) const
			{
				return (VertexDataType == EClothPatternVertexType::Render) ? NormalIDs : EmptyArray;
			}

			FVector2f GetUV(int32 LayerID, UVIDType UVID) const
			{
				checkf(VertexDataType == EClothPatternVertexType::Render, TEXT("Requested UVs from a Sim mesh"));
				const TConstArrayView<FVector2f> VertexUVs = Cloth.GetRenderUVs()[UVID];
				return VertexUVs.IsValidIndex(LayerID) ? VertexUVs[LayerID] : FVector2f(0.f);
			}

			bool GetUVTri(int32 LayerID, const TriIDType& TriID, UVIDType& ID0, UVIDType& ID1, UVIDType& ID2) const
			{
				return GetTri(TriID, ID0, ID1, ID2);
			}

			const TArray<int32>& GetTangentIDs() const
			{
				return (VertexDataType == EClothPatternVertexType::Render) ? NormalIDs : EmptyArray;
			}

			FVector3f GetTangent(NormalIDType ID) const
			{
				checkf(VertexDataType == EClothPatternVertexType::Render, TEXT("Requested Tangent from a Sim mesh"));
				return Cloth.GetRenderTangentU()[ID];
			}

			bool GetTangentTri(const TriIDType& TriID, NormalIDType& ID0, NormalIDType& ID1, NormalIDType& ID2) const
			{
				return GetNormalTri(TriID, ID0, ID1, ID2);
			}

			const TArray<int32>& GetBiTangentIDs() const
			{
				return (VertexDataType == EClothPatternVertexType::Render) ? NormalIDs : EmptyArray;
			}

			FVector3f GetBiTangent(NormalIDType ID) const
			{
				checkf(VertexDataType == EClothPatternVertexType::Render, TEXT("Requested Bitangent from a Sim mesh"));
				return Cloth.GetRenderTangentV()[ID];
			}

			bool GetBiTangentTri(const TriIDType& TriID, NormalIDType& ID0, NormalIDType& ID1, NormalIDType& ID2) const
			{
				return GetNormalTri(TriID, ID0, ID1, ID2);
			}

			const TArray<int32>& GetColorIDs() const
			{
				return (VertexDataType == EClothPatternVertexType::Render) ? NormalIDs : EmptyArray;
			}

			FVector4f GetColor(ColorIDType VID) const
			{
				checkf(VertexDataType == EClothPatternVertexType::Render, TEXT("Requested color from a Sim mesh"));
				return Cloth.GetRenderColor()[VID];
			}

			bool GetColorTri(const TriIDType& TriID, ColorIDType& ID0, ColorIDType& ID1, ColorIDType& ID2) const
			{
				return GetNormalTri(TriID, ID0, ID1, ID2);
			}

		private:

			const EClothPatternVertexType VertexDataType;
			const FCollectionClothConstFacade& Cloth;

			TArray<TriIDType> TriIDs;		// indices into Indices
			TArray<VertIDType> VertIDs;		// indices into Positions2D or Positions3D
			TArray<NormalIDType> NormalIDs; // indices into Normals (and all per vertex data other than positions)
			int32 NumTexCoords;
			bool bFlip2DSimFaces = false;
			bool bConvertWeightMaps = true;

			TArray<FName> WeightMapNames;

			TArray<int32> EmptyArray;

			const FReferenceSkeleton* RefSkeleton = nullptr;
		};
	}  // namespace Private

	void FClothPatternToDynamicMesh::Convert(
		const TSharedRef<const FManagedArrayCollection> ClothCollection,
		int32 PatternIndex,
		EClothPatternVertexType VertexDataType,
		UE::Geometry::FDynamicMesh3& MeshOut,
		bool bDisableAttributes,
		int32 MaterialOffset,
		bool bFlip2DSimFaces,
		bool bConvertWeightMaps)
	{
		const FCollectionClothConstFacade ClothFacade(ClothCollection);

		// Actual conversion
		UE::Geometry::TToDynamicMesh<Private::FClothPatternWrapper> PatternToDynamicMesh;
		Private::FClothPatternWrapper PatternWrapper(ClothFacade, PatternIndex, VertexDataType, bFlip2DSimFaces, bConvertWeightMaps);

		auto TriangleToGroupFunction = [](Private::FClothPatternWrapper::TriIDType) { return 0; };

		if (bDisableAttributes)
		{
			PatternToDynamicMesh.ConvertWOAttributes(MeshOut, PatternWrapper, TriangleToGroupFunction);
		}
		else
		{
			constexpr bool bCopyTangents = false;
			const bool bIsRenderType = VertexDataType == EClothPatternVertexType::Render;
			auto TriangleToMaterialFunction = [PatternIndex, bIsRenderType, &ClothFacade](Private::FClothPatternWrapper::TriIDType TriID)->int32
			{
				if (bIsRenderType)
				{
					if (PatternIndex != INDEX_NONE)
					{
						return PatternIndex;
					}
					const int32 FoundPattern = ClothFacade.FindRenderPatternByFaceIndex(TriID);
					check(FoundPattern != INDEX_NONE);
					return FoundPattern;
				}

				// Sim meshes will have a default material with MaterialID zero applied
				return 0;
			};

			PatternToDynamicMesh.Convert(MeshOut, PatternWrapper, TriangleToGroupFunction, TriangleToMaterialFunction, bCopyTangents);
		}

		// Add non-manifold mapping data if DynamicMesh indices don't match orig indices
		const int32 NumVertices = VertexDataType == EClothPatternVertexType::Render ? ClothFacade.GetNumRenderVertices() :
			VertexDataType == EClothPatternVertexType::Sim2D ? ClothFacade.GetNumSimVertices2D() : ClothFacade.GetNumSimVertices3D();
		bool bVertexIndicesMatch = false;
		if (PatternToDynamicMesh.ToSrcVertIDMap.Num() == NumVertices)
		{
			bVertexIndicesMatch = true;
			for (int32 VertId = 0; VertId < NumVertices; ++VertId)
			{
				if (PatternToDynamicMesh.ToSrcVertIDMap[VertId] != VertId)
				{
					bVertexIndicesMatch = false;
					break;
				}
			}
		}
		if (!bVertexIndicesMatch)
		{
			MeshOut.EnableAttributes();
			FClothPatternToDynamicMeshMappingSupport::AttachVertexMappingData(PatternToDynamicMesh.ToSrcVertIDMap, MeshOut);
		}
		const int32 NumTriangles = VertexDataType == EClothPatternVertexType::Render ? ClothFacade.GetNumRenderFaces() : ClothFacade.GetNumSimFaces();
		bool bTriangleIndicesMatch = false;
		if (PatternToDynamicMesh.ToSrcTriIDMap.Num() == NumTriangles)
		{
			bTriangleIndicesMatch = true;
			for (int32 TriId = 0; TriId < NumTriangles; ++TriId)
			{
				if (PatternToDynamicMesh.ToSrcTriIDMap[TriId] != TriId)
				{
					bTriangleIndicesMatch = false;
					break;
				}
			}
		}
		if (!bTriangleIndicesMatch)
		{
			MeshOut.EnableAttributes();
			FClothPatternToDynamicMeshMappingSupport::AttachTriangleMappingData(PatternToDynamicMesh.ToSrcTriIDMap, MeshOut);
		}

		// Fix up any triangles without valid material IDs and apply material offset
		if (VertexDataType == EClothPatternVertexType::Render && MaterialOffset != INDEX_NONE)
		{
			const TConstArrayView<FSoftObjectPath> MaterialPaths = ClothFacade.GetRenderMaterialSoftObjectPathName();
			int32 DefaultMaterialID = 0;
			for (const int32 TriID : MeshOut.TriangleIndicesItr())
			{
				int32 MaterialID;
				MeshOut.Attributes()->GetMaterialID()->GetValue(TriID, &MaterialID);
				if (!MaterialPaths.IsValidIndex(MaterialID))
				{
					MeshOut.Attributes()->GetMaterialID()->SetValue(TriID, DefaultMaterialID);
				}
				else
				{
					MeshOut.Attributes()->GetMaterialID()->SetValue(TriID, MaterialID + MaterialOffset);
				}
			}
		}
	}

	void FClothPatternToDynamicMesh::Convert(
		const UChaosClothAsset* ClothAssetMeshIn,
		int32 LODIndex,
		int32 PatternIndex,
		EClothPatternVertexType VertexDataType,
		UE::Geometry::FDynamicMesh3& MeshOut,
		int32 MaterialOffset,
		bool bFlip2DSimFaces,
		bool bConvertWeightMaps)
	{
		const TArray<TSharedRef<const FManagedArrayCollection>>& ClothCollections = ClothAssetMeshIn->GetClothCollections();
		check(ClothCollections.IsValidIndex(LODIndex));

		constexpr bool bDisableAttributes = false;
		Convert(ClothCollections[LODIndex], PatternIndex, VertexDataType, MeshOut, bDisableAttributes, MaterialOffset, bFlip2DSimFaces, bConvertWeightMaps);
	}
}	// namespace UE::Chaos::ClothAsset

#else  // #if WITH_EDITOR

namespace UE::Chaos::ClothAsset
{
	void FClothPatternToDynamicMesh::Convert(
		const TSharedRef<const FManagedArrayCollection> ClothCollection,
		int32 PatternIndex,
		EClothPatternVertexType VertexDataType,
		UE::Geometry::FDynamicMesh3& MeshOut,
		bool bDisableAttributes,
		int32 MaterialOffset,
		bool bFlip2DSimFaces,
		bool bConvertWeightMaps)
	{
		// Conversion only supported with editor.
		check(0);
	}

	void FClothPatternToDynamicMesh::Convert(
		const UChaosClothAsset* ClothAssetMeshIn,
		int32 LODIndex,
		int32 PatternIndex,
		EClothPatternVertexType VertexDataType,
		UE::Geometry::FDynamicMesh3& MeshOut,
		int32 MaterialOffset,
		bool bFlip2DSimFaces,
		bool bConvertWeightMaps)
	{
		// Conversion only supported with editor.
		check(0);
	}
}	// namespace UE::Chaos::ClothAsset

#endif  // #if WITH_EDITOR #else


