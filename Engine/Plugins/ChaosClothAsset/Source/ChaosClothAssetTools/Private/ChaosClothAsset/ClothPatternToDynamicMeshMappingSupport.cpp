// Copyright Epic Games, Inc. All Rights Reserved. 

#include "ChaosClothAsset/ClothPatternToDynamicMeshMappingSupport.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"

namespace UE::Chaos::ClothAsset
{

class FDynamicMeshSourceTriangleIdAttributeChange final :
	public UE::Geometry::FDynamicMeshAttributeChangeBase
{
	friend class FDynamicMeshSourceTriangleIdAttribute;
	TArray<TPair<int32, int32>> OldSourceTriangleIds, NewSourceTriangleIds;

	bool bOldInvalidState = false;
	bool bNewInvalidState = false;

public:
	FDynamicMeshSourceTriangleIdAttributeChange() = default;

	virtual ~FDynamicMeshSourceTriangleIdAttributeChange() override = default;

	void SaveInitialTriangle(const UE::Geometry::FDynamicMeshAttributeBase* Attribute, int TriangleID) override;

	void StoreAllFinalTriangles(const UE::Geometry::FDynamicMeshAttributeBase* Attribute, const TArray<int>& TriangleIDs) override;

	bool Apply(UE::Geometry::FDynamicMeshAttributeBase* Attribute, bool bRevert) const override;
};

// TODO: Can this class inherit from UE::Geometry::TDynamicMeshScalarTriangleAttribute<int32>
// and then only need to implement a smaller subset of the methods? (On* and IsValid?)
class FDynamicMeshSourceTriangleIdAttribute final :
	public UE::Geometry::FDynamicMeshAttributeBase
{
	friend class FDynamicMeshSourceTriangleIdAttributeChange;

	UE::Geometry::FDynamicMesh3* Parent = nullptr;
	UE::Geometry::TDynamicVector<int32> SourceTriangleIds;
	bool bValid = false;

public:
	FDynamicMeshSourceTriangleIdAttribute() = default;

	FDynamicMeshSourceTriangleIdAttribute(UE::Geometry::FDynamicMesh3* InParent) : Parent(InParent)
	{
	}

	virtual ~FDynamicMeshSourceTriangleIdAttribute() override = default;

	const UE::Geometry::FDynamicMesh3* GetParent() const
	{
		return Parent;
	}
	UE::Geometry::FDynamicMesh3* GetParent()
	{
		return Parent;
	}

	bool IsValid() const
	{
		return bValid;
	}

	UE::Geometry::FDynamicMeshAttributeBase* MakeCopy(UE::Geometry::FDynamicMesh3* ParentIn) const override
	{
		FDynamicMeshSourceTriangleIdAttribute* Attribute = new FDynamicMeshSourceTriangleIdAttribute(ParentIn);
		Attribute->Copy(*this);
		return Attribute;
	}

	UE::Geometry::FDynamicMeshAttributeBase* MakeNew(UE::Geometry::FDynamicMesh3* ParentIn) const override
	{
		FDynamicMeshSourceTriangleIdAttribute* Attribute = new FDynamicMeshSourceTriangleIdAttribute(ParentIn);
		Attribute->Initialize();
		return Attribute;
	}

	void CompactInPlace(const UE::Geometry::FCompactMaps& CompactMaps) override
	{
		for (int32 FromTID = 0, NumTID = CompactMaps.NumTriangleMappings(); FromTID < NumTID; FromTID++)
		{
			const int32 ToTID = CompactMaps.GetTriangleMapping(FromTID);
			if (ToTID == UE::Geometry::FCompactMaps::InvalidID)
			{
				continue;
			}
			if (ensure(ToTID <= FromTID))
			{
				SourceTriangleIds[ToTID] = SourceTriangleIds[FromTID];
			}
		}
		SourceTriangleIds.Resize(Parent->MaxTriangleID());
	}

	void Reparent(UE::Geometry::FDynamicMesh3* NewParent) override
	{
		Parent = NewParent;
	}

	bool CopyThroughMapping(const UE::Geometry::TDynamicAttributeBase<UE::Geometry::FDynamicMesh3>* Source, const UE::Geometry::FMeshIndexMappings& Mapping) override
	{
		for (const TPair<int32, int32>& MapTID : Mapping.GetTriangleMap().GetForwardMap())
		{
			int32 SourceTId;
			if (!ensure(Source->CopyOut(MapTID.Key, &SourceTId, sizeof(SourceTId))))
			{
				return false;
			}
			SetValue(MapTID.Value, SourceTId);
		}
		return true;
	}

	virtual bool Append(const TDynamicAttributeBase& Source, const UE::Geometry::FDynamicMesh3::FAppendInfo& Info) override
	{
		int32 NewMaxID = Info.NumTriangle + Info.TriangleOffset;
		if (NewMaxID > SourceTriangleIds.Num())
		{
			SourceTriangleIds.SetNum(NewMaxID);
		}
		for (int32 Idx = 0; Idx < Info.NumTriangle; ++Idx)
		{
			int32 TargetID = Idx + Info.TriangleOffset;
			if (!Parent->IsTriangle(TargetID))
			{
				continue;
			}
			int32 SourceTID;
			if (!ensure(Source.CopyOut(Idx, &SourceTID, sizeof(SourceTID))))
			{
				return false;
			}
			SetValue(Idx + Info.TriangleOffset, SourceTID);
		}
		return true;
	}

	virtual void AppendDefaulted(const UE::Geometry::FDynamicMesh3::FAppendInfo& Info) override
	{
		int32 NewMaxID = Info.NumTriangle + Info.TriangleOffset;
		if (NewMaxID > SourceTriangleIds.Num())
		{
			SourceTriangleIds.Resize(NewMaxID, INDEX_NONE);
		}
	}

	bool CopyOut(int RawID, void* Buffer, int BufferSize) const override
	{
		if (BufferSize != sizeof(int32) || !Parent->IsTriangle(RawID))
		{
			return false;
		}

		*static_cast<int32*>(Buffer) = SourceTriangleIds[RawID];
		return true;
	}

	bool CopyIn(int RawID, void* Buffer, int BufferSize) override
	{
		if (BufferSize != sizeof(int32) || !Parent->IsTriangle(RawID))
		{
			return false;
		}

		SourceTriangleIds[RawID] = *static_cast<int32*>(Buffer);
		return true;
	}

	TUniquePtr<UE::Geometry::TDynamicAttributeChangeBase<UE::Geometry::FDynamicMesh3>> NewBlankChange() const override
	{
		return MakeUnique<FDynamicMeshSourceTriangleIdAttributeChange>();
	}

	void Initialize()
	{
		SourceTriangleIds.Resize(Parent->MaxTriangleID());
		SourceTriangleIds.Fill(INDEX_NONE);
	}

	void InitializeFromArray(TConstArrayView<int32> TriangleToNonManifoldVertexIDMap)
	{
		check(TriangleToNonManifoldVertexIDMap.Num() == Parent->MaxTriangleID());
		SourceTriangleIds.Resize(Parent->MaxTriangleID());

		for (int32 TriangleID : Parent->TriangleIndicesItr())
		{
			SetValue(TriangleID, TriangleToNonManifoldVertexIDMap[TriangleID]);
		}
		bValid = true;
	}

	void Copy(const FDynamicMeshSourceTriangleIdAttribute& Copy)
	{
		UE::Geometry::FDynamicMeshAttributeBase::CopyParentClassData(Copy);
		SourceTriangleIds = Copy.SourceTriangleIds;
		bValid = Copy.bValid;
	}

	int32 GetValue(int32 InTriangleId) const
	{
		return SourceTriangleIds[InTriangleId];
	}

	void SetValue(int32 InTriangleId, int32 InSourceTriangled)
	{
		SourceTriangleIds[InTriangleId] = InSourceTriangled;
	}

	// Any topo operation on the mesh will invalidate the non-manifold information.
	virtual void OnSplitEdge(const DynamicMeshInfo::FEdgeSplitInfo&) override
	{
		bValid = false;
	}

	virtual void OnFlipEdge(const DynamicMeshInfo::FEdgeFlipInfo& FlipInfo) override
	{
		bValid = false;
	}

	virtual void OnCollapseEdge(const DynamicMeshInfo::FEdgeCollapseInfo&) override
	{
		bValid = false;
	}

	virtual void OnPokeTriangle(const DynamicMeshInfo::FPokeTriangleInfo&) override
	{
		bValid = false;
	}

	virtual void OnMergeEdges(const DynamicMeshInfo::FMergeEdgesInfo&) override
	{
		bValid = false;
	}

	virtual void OnMergeVertices(const DynamicMeshInfo::FMergeVerticesInfo&) override
	{
		bValid = false;
	}

	virtual void OnSplitVertex(const DynamicMeshInfo::FVertexSplitInfo&, const TArrayView<const int>&) override
	{
		bValid = false;
	}

	virtual void OnNewVertex(int VertexID, bool bInserted) override
	{
		bValid = false;
	}

	virtual void OnRemoveVertex(int VertexID) override
	{
		bValid = false;
	}
};


void FDynamicMeshSourceTriangleIdAttributeChange::SaveInitialTriangle(const UE::Geometry::FDynamicMeshAttributeBase* Attribute, int TriangleID)
{
	const FDynamicMeshSourceTriangleIdAttribute* SourceAttribute = static_cast<const FDynamicMeshSourceTriangleIdAttribute*>(Attribute);
	if (OldSourceTriangleIds.IsEmpty())
	{
		bOldInvalidState = SourceAttribute->IsValid();
	}

	OldSourceTriangleIds.Emplace(TriangleID, SourceAttribute->GetValue(TriangleID));
}

void FDynamicMeshSourceTriangleIdAttributeChange::StoreAllFinalTriangles(const UE::Geometry::FDynamicMeshAttributeBase* Attribute, const TArray<int>& TriangleIDs)
{
	NewSourceTriangleIds.Reserve(NewSourceTriangleIds.Num() + TriangleIDs.Num());
	const FDynamicMeshSourceTriangleIdAttribute* SourceAttribute = static_cast<const FDynamicMeshSourceTriangleIdAttribute*>(Attribute);
	for (int32 TriangleID : TriangleIDs)
	{
		NewSourceTriangleIds.Emplace(TriangleID, SourceAttribute->GetValue(TriangleID));
	}

	// Store the last known valid state.  
	bNewInvalidState = SourceAttribute->IsValid();
}

bool FDynamicMeshSourceTriangleIdAttributeChange::Apply(UE::Geometry::FDynamicMeshAttributeBase* Attribute, bool bRevert) const
{
	const TArray<TPair<int32, int32>>& Changes = bRevert ? OldSourceTriangleIds : NewSourceTriangleIds;
	FDynamicMeshSourceTriangleIdAttribute* SourceAttribute = static_cast<FDynamicMeshSourceTriangleIdAttribute*>(Attribute);
	for (const TPair<int32, int32>& Item : Changes)
	{
		if (ensure(SourceAttribute->GetParent()->IsTriangle(Item.Key)))
		{
			SourceAttribute->SetValue(Item.Key, Item.Value);
		}
	}

	// Restore the valid state as well.
	SourceAttribute->bValid = bRevert ? bOldInvalidState : bNewInvalidState;

	return true;
}

FClothPatternToDynamicMeshMappingSupport::FClothPatternToDynamicMeshMappingSupport(const UE::Geometry::FDynamicMesh3& Mesh)
	: UE::Geometry::FNonManifoldMappingSupport(Mesh)
{
	Reset(Mesh);
}

const FName FClothPatternToDynamicMeshMappingSupport::ClothMeshTIDsAttrName = FName(TEXT("ClothMeshTIDsAttr"));

void FClothPatternToDynamicMeshMappingSupport::Reset(const UE::Geometry::FDynamicMesh3& Mesh)
{
	UE::Geometry::FNonManifoldMappingSupport::Reset(Mesh);

	if (const UE::Geometry::FDynamicMeshAttributeSet* Attributes = DynamicMesh->Attributes())
	{
		if (Attributes->HasAttachedAttribute(ClothMeshTIDsAttrName))
		{
			SrcTIDsAttribute = static_cast<const FDynamicMeshSourceTriangleIdAttribute*>(Attributes->GetAttachedAttribute(ClothMeshTIDsAttrName));
		}
	}
}

bool FClothPatternToDynamicMeshMappingSupport::IsMappedTriangleInSource() const
{
	return SrcTIDsAttribute != nullptr && SrcTIDsAttribute->IsValid();
}

int32 FClothPatternToDynamicMeshMappingSupport::GetOriginalTriangleID(const int32 Tid) const
{
	checkSlow(DynamicMesh->IsTriangle(Tid));

	int32 Result = Tid;
	if (SrcTIDsAttribute)
	{
		Result = SrcTIDsAttribute->GetValue(Tid);
	}
	return Result;
}

bool FClothPatternToDynamicMeshMappingSupport::AttachTriangleMappingData(const TArray<int32>& TriangleToOriginalVertexIDMap, UE::Geometry::FDynamicMesh3& InOutMesh)
{
	const int32 MaxTID = InOutMesh.MaxTriangleID();
	UE::Geometry::FDynamicMeshAttributeSet* const Attributes = InOutMesh.Attributes();

	if (TriangleToOriginalVertexIDMap.Num() < MaxTID || !Attributes)
	{
		return false;
	}

	// create and attach a triangle ID buffer, removing any pre-exising one.
	FDynamicMeshSourceTriangleIdAttribute* TIDAttr = [&Attributes, &InOutMesh] {
		// replace existing buffer.  we *should* be able to re-use it
		if (Attributes->HasAttachedAttribute(ClothMeshTIDsAttrName))
		{
			Attributes->RemoveAttribute(ClothMeshTIDsAttrName);
		}
		FDynamicMeshSourceTriangleIdAttribute* SrcMeshTIDAttr = new FDynamicMeshSourceTriangleIdAttribute(&InOutMesh);
		SrcMeshTIDAttr->SetName(ClothMeshTIDsAttrName);
		InOutMesh.Attributes()->AttachAttribute(ClothMeshTIDsAttrName, SrcMeshTIDAttr);
		return SrcMeshTIDAttr;
		}();

	TIDAttr->InitializeFromArray(TriangleToOriginalVertexIDMap);
	return true;
}

void FClothPatternToDynamicMeshMappingSupport::RemoveTriangleMappingData(UE::Geometry::FDynamicMesh3& InOutMesh)
{
	UE::Geometry::FDynamicMeshAttributeSet* Attributes = InOutMesh.Attributes();

	if (Attributes && Attributes->HasAttachedAttribute(ClothMeshTIDsAttrName))
	{
		Attributes->RemoveAttribute(ClothMeshTIDsAttrName);
	}
}


}	// namespace UE::Chaos::ClothAsset

