// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClothingAssetToClothAssetExporter.h"
#include "ChaosClothAsset/ClothAsset.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "ChaosClothAsset/ClothGeometryTools.h"
#include "ChaosClothAsset/LegacyClothingConverterProvider.h"
#include "Animation/Skeleton.h"
#include "Features/IModularFeatures.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/NonManifoldMappingSupport.h"
#include "ToDynamicMesh.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Engine/SkeletalMesh.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Materials/Material.h"
#include "Misc/MessageDialog.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "ClothingAsset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClothingAssetToClothAssetExporter)

#define LOCTEXT_NAMESPACE "ClothingAssetToClothAssetExporter"

namespace UE::Chaos::ClothAsset::Private
{
struct FSimpleSrcMeshInterface
{
	typedef int32     VertIDType;
	typedef int32     TriIDType;

	FSimpleSrcMeshInterface(const TArray<FVector3f>& InPositions, const TArray<uint32>& InIndices)
		:Positions(InPositions), Indices(InIndices)
	{
		VertIDs.SetNumUninitialized(Positions.Num());
		for (int32 VtxIndex = 0; VtxIndex < Positions.Num(); ++VtxIndex)
		{
			VertIDs[VtxIndex] = VtxIndex;
		}

		check(Indices.Num() % 3 == 0);
		const int32 NumFaces = Indices.Num() / 3;
		TriIDs.SetNumUninitialized(NumFaces);
		for (int32 TriIndex = 0; TriIndex < NumFaces; ++TriIndex)
		{
			TriIDs[TriIndex] = 3 * TriIndex;
		}
	}

	// accounting.
	int32 NumTris() const { return TriIDs.Num(); }
	int32 NumVerts() const { return VertIDs.Num(); }

	// --"Vertex Buffer" info
	const TArray<VertIDType>& GetVertIDs() const { return VertIDs; }
	const FVector GetPosition(const VertIDType VtxID) const { return FVector(Positions[VtxID]); }

	// --"Index Buffer" info
	const TArray<TriIDType>& GetTriIDs() const { return TriIDs; }
	// return false if this TriID is not contained in mesh.
	bool GetTri(const TriIDType TriID, VertIDType& VID0, VertIDType& VID1, VertIDType& VID2) const
	{
		VID0 = Indices[TriID + 0];
		VID1 = Indices[TriID + 1];
		VID2 = Indices[TriID + 2];

		return true;
	}

private:
	const TArray<FVector3f>& Positions;
	const TArray<uint32>& Indices;

	TArray<TriIDType> TriIDs; // TriID = first index in flat Indices array
	TArray<VertIDType> VertIDs;
};
} // namespace UE::Chaos::ClothAsset::Private


UClass* UClothingAssetToChaosClothAssetExporter::GetExportedType() const
{
	return UChaosClothAsset::StaticClass();
}

void UClothingAssetToChaosClothAssetExporter::Export(const UClothingAssetBase* ClothingAsset, UObject* ExportedAsset)
{
	using namespace UE::Chaos::ClothAsset;

	const UClothingAssetCommon* const ClothingAssetCommon = ExactCast<UClothingAssetCommon>(ClothingAsset);
	if (!ClothingAssetCommon)
	{
		const FText TitleMessage = LOCTEXT("ClothingAssetExporterTitle", "Error Exporting Clothing Asset");
		const FText ErrorMessage = LOCTEXT("ClothingAssetExporterError", "Can only export from known ClothingAssetCommon types.");
		FMessageDialog::Open(EAppMsgType::Ok, EAppReturnType::Ok, ErrorMessage, TitleMessage);
		return;
	}

	UChaosClothAsset* const ClothAsset = CastChecked<UChaosClothAsset>(ExportedAsset);
	check(ClothAsset);

	// Prefer the new Dataflow-graph-based converter when ChaosClothAssetEditor has registered its
	// ILegacyClothingConverterProvider implementation. That path bakes the legacy config + weight
	// maps into an editable Dataflow graph on the target asset, matching what the AICallable
	// ConvertClothingAssetCommonToChaosClothAsset toolset method produces. Falls through to the
	// legacy in-tree flat-mesh export below when no provider is registered (e.g. when the editor
	// module is unloaded).
	const TArray<ILegacyClothingConverterProvider*> Providers =
		IModularFeatures::Get().GetModularFeatureImplementations<ILegacyClothingConverterProvider>(
			ILegacyClothingConverterProvider::FeatureName);
	if (Providers.Num() > 0 && Providers[0])
	{
		if (Providers[0]->ConvertInto(ClothingAssetCommon, ClothAsset))
		{
			return;
		}
		// Provider ran but rejected the inputs -- surface a generic error and bail rather than silently
		// falling back to the legacy export (which would overwrite the asset with the flat-mesh path).
		const FText TitleMessage = LOCTEXT("ClothingAssetExporterTitle", "Error Exporting Clothing Asset");
		const FText ErrorMessage = LOCTEXT("ClothingAssetConverterFailed", "Legacy clothing asset conversion failed. See the Output Log for details.");
		FMessageDialog::Open(EAppMsgType::Ok, EAppReturnType::Ok, ErrorMessage, TitleMessage);
		return;
	}

	static const FString DefaultMaterialPathName = FString(TEXT("/Engine/EditorMaterials/Cloth/CameraLitDoubleSided.CameraLitDoubleSided"));
	const FString PhysicsAssetPathName = ClothingAssetCommon->PhysicsAsset ? ClothingAssetCommon->PhysicsAsset->GetPathName() : FString();
	const FString SkeletalMeshPathName = CastChecked<USkeletalMesh>(ClothingAssetCommon->GetOuter())->GetPathName();

	const int32 NumLods = ClothingAssetCommon->LodData.Num();
	TArray<TSharedRef<const FManagedArrayCollection>> ClothCollections;
	ClothCollections.Reset(NumLods);

	// Create the LODs
	for (int32 LodIndex = 0; LodIndex < NumLods; ++LodIndex)
	{
		const FClothLODDataCommon& ClothLODData = ClothingAssetCommon->LodData[LodIndex];
		const FClothPhysicalMeshData& PhysicalMeshData = ClothLODData.PhysicalMeshData;

		TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>();
		FCollectionClothFacade ClothFacade(ClothCollection);
		ClothFacade.DefineSchema();

		using namespace UE::Chaos::ClothAsset::Private;

		// Build a DynamicMesh from Positions and Indices
		UE::Geometry::TToDynamicMeshBase<FSimpleSrcMeshInterface> ToDynamicMesh;
		FSimpleSrcMeshInterface SimpleSrc(PhysicalMeshData.Vertices, PhysicalMeshData.Indices);

		UE::Geometry::FDynamicMesh3 DynamicMesh;
		ToDynamicMesh.Convert(DynamicMesh, SimpleSrc, [](FSimpleSrcMeshInterface::TriIDType) {return 0; });
		UE::Geometry::FNonManifoldMappingSupport::AttachNonManifoldVertexMappingData(ToDynamicMesh.ToSrcVertIDMap, DynamicMesh);

		constexpr int32 UVChannelIndexNone = INDEX_NONE;
		constexpr bool bAppend = false;
		FClothGeometryTools::BuildSimMeshFromDynamicMesh(ClothCollection, DynamicMesh, UVChannelIndexNone, FVector2f(1.f), bAppend);

		// Set the physics asset if any
		ClothFacade.SetPhysicsAssetSoftObjectPathName(PhysicsAssetPathName);
		// Set the skeleton
		ClothFacade.SetSkeletalMeshSoftObjectPathName(SkeletalMeshPathName);

		// Bind the sim mesh to root bone
		constexpr bool bBindSimMesh = true;
		constexpr bool bBindRenderMesh = false;  // Will get bound below in CopySimMeshToRenderMesh
		FClothGeometryTools::BindMeshToRootBone(ClothCollection, bBindSimMesh, bBindRenderMesh);

		// Set the render mesh to duplicate the sim mesh
		constexpr bool bSingleRenderPattern = true;
		FClothGeometryTools::CopySimMeshToRenderMesh(ClothCollection, DefaultMaterialPathName, bSingleRenderPattern);

		ClothCollections.Emplace(MoveTemp(ClothCollection));
	}

	if (!NumLods)
	{
		// Make sure that at least one empty LOD is always created
		TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>();
		FCollectionClothFacade ClothFacade(ClothCollection);
		ClothFacade.DefineSchema();
		ClothFacade.SetPhysicsAssetSoftObjectPathName(PhysicsAssetPathName);
		ClothFacade.SetSkeletalMeshSoftObjectPathName(SkeletalMeshPathName);
		ClothCollections.Emplace(MoveTemp(ClothCollection));
	}

	// Rebuild the asset
	ClothAsset->Build(ClothCollections);
}

#undef LOCTEXT_NAMESPACE

