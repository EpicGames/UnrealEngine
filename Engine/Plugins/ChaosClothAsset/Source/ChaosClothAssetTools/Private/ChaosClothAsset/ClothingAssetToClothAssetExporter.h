// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ClothingAssetExporter.h"

#include "ClothingAssetToClothAssetExporter.generated.h"

/**
 * Implement a clothing asset exporter as a way to migrate clothing assets to the new Chaos Cloth asset format.
 */
UCLASS()
class UClothingAssetToChaosClothAssetExporter : public UClothingAssetExporter
{
    GENERATED_BODY()
public:
	/** Return the class of the exported asset type. */
	virtual UClass* GetExportedType() const override;

	/**
	 * Export the specified asset.
	 * \param ClothingAsset the source clothing asset to be exported.
	 * \param ExportedAsset the destination asset object in the exported type provided by the caller, ready to be filled by the Export function.
	 */
	virtual void Export(const UClothingAssetBase* ClothingAsset, UObject* ExportedAsset) override;
};
