// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/SkeletalMeshConverter.h"
#include "ChaosClothAsset/ClothAssetBase.h"
#include "ChaosClothAsset/ClothPatternToDynamicMesh.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "Engine/SkeletalMesh.h"
#include "Materials/Material.h"
#include "Rendering/SkeletalMeshModel.h"
#include "DynamicMeshToMeshDescription.h"
#include "MaterialDomain.h"
#include "SkeletalMeshAttributes.h"
#include "StaticToSkeletalMeshConverter.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SkeletalMeshConverter)

bool UClothAssetEditorSkeletalMeshConverter::ExportToSkeletalMesh(const UChaosClothAssetBase& ClothAssetBase, USkeletalMesh& SkeletalMesh) const
{
	using namespace UE::Chaos::ClothAsset;

	// Create one dynamic mesh from the multiple collections
	TArray<UMaterialInterface*> RenderMaterials;
	TArray<FDynamicMesh3> DynamicMeshes;

	const int32 NumLods = ClothAssetBase.GetLODNum();
	DynamicMeshes.SetNum(NumLods);

	int32 MaterialOffset = 0;

	const int32 NumModels = ClothAssetBase.GetNumClothSimulationModels();

	for (int32 ModelIndex = 0; ModelIndex < NumModels; ++ModelIndex)
	{
		const TArray<TSharedRef<const FManagedArrayCollection>>& ClothCollections = ClothAssetBase.GetCollections(ModelIndex);

		for (int32 LodIndex = 0; LodIndex < NumLods; ++LodIndex)
		{
			if (!ClothCollections.IsValidIndex(LodIndex))
			{
				continue;
			}
			const TSharedRef<const FManagedArrayCollection>& ClothCollection = ClothCollections[LodIndex];
			FCollectionClothConstFacade ClothFacade(ClothCollection);

			if (ClothFacade.IsValid())
			{
				FDynamicMesh3 DynamicMeshPiece;

				constexpr bool bDisableAttributes = false;
				constexpr bool bFlip2DSimFaces = false;
				constexpr bool bConvertWeightMaps = false;
				const int32 PatternIndex = INDEX_NONE;
				DynamicMeshPiece.EnableAttributes();
				DynamicMeshPiece.Attributes()->EnableMaterialID();
				FClothPatternToDynamicMesh ClothPatternToDynamicMesh;
				ClothPatternToDynamicMesh.Convert(
					ClothCollection,
					PatternIndex,
					EClothPatternVertexType::Render,
					DynamicMeshPiece,
					bDisableAttributes,
					MaterialOffset,
					bFlip2DSimFaces,
					bConvertWeightMaps);

				if (!DynamicMeshes[LodIndex].VertexCount() || !DynamicMeshes[LodIndex].TriangleCount())
				{
					DynamicMeshes[LodIndex] = MoveTemp(DynamicMeshPiece);
				}
				else
				{
					DynamicMeshes[LodIndex].AppendWithOffsets(DynamicMeshPiece);
				}

				const TArrayView<const FSoftObjectPath> MaterialPaths = ClothFacade.GetRenderMaterialSoftObjectPathName();
				RenderMaterials.Reserve(RenderMaterials.Num() + MaterialPaths.Num());
				for (int32 MaterialIndex = 0; MaterialIndex < MaterialPaths.Num(); ++MaterialIndex)
				{
					RenderMaterials.Emplace(Cast<UMaterialInterface>(MaterialPaths[MaterialIndex].TryLoad()));
				}
				MaterialOffset += MaterialPaths.Num();
			}
		}
	}

	// Create the skeletal mesh from the dynamic mesh
	TArray<const FMeshDescription*> MeshDescriptions;

	SkeletalMesh.PreEditChange(nullptr);
	SkeletalMesh.GetImportedModel()->LODModels.Empty();
	SkeletalMesh.SetNumSourceModels(0);

	TArray<FMeshDescription> LocalDescriptions;
	for (const FDynamicMesh3& DynamicMesh : DynamicMeshes)
	{
		// Create a mesh description
		FMeshDescription& MeshDescription = LocalDescriptions.AddDefaulted_GetRef();

		// Add skeletal mesh attributes to the mesh description
		FSkeletalMeshAttributes Attributes(MeshDescription);
		Attributes.Register();

		// Convert dynamic mesh to the mesh description
		FConversionToMeshDescriptionOptions ConverterOptions;
		FDynamicMeshToMeshDescription Converter(ConverterOptions);
		constexpr bool bCopyTangents = true;
		Converter.Convert(&DynamicMesh, MeshDescription, bCopyTangents);

		// Add the created description to the list
		MeshDescriptions.Add(&MeshDescription);
	}

	TArray<FSkeletalMaterial> SkeletalMaterials;
	SkeletalMaterials.Reserve(RenderMaterials.Num());
	for (UMaterialInterface* RenderMaterial : RenderMaterials)
	{
		if (RenderMaterial)
		{
			SkeletalMaterials.Emplace(RenderMaterial, RenderMaterial->GetFName());
		}
		else
		{
			SkeletalMaterials.Emplace(UMaterial::GetDefaultMaterial(MD_Surface));
		}
	}
	if (!SkeletalMaterials.Num())
	{
		SkeletalMaterials.Emplace(UMaterial::GetDefaultMaterial(MD_Surface));
	}

	FStaticToSkeletalMeshConverter::FInitializationParams Parameters;
		Parameters.Materials = SkeletalMaterials;
		Parameters.bRecomputeNormals = false;
		Parameters.bRecomputeTangents = false;
	
	return FStaticToSkeletalMeshConverter::InitializeSkeletalMeshFromMeshDescriptions(
		&SkeletalMesh,
		MeshDescriptions,
		ClothAssetBase.GetRefSkeleton(),
		Parameters);
}
