// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosClothAsset/SkeletalMeshConverterClassProvider.h"
#include "SkeletalMeshConverter.generated.h"

UCLASS()
class UClothAssetEditorSkeletalMeshConverter : public UClothAssetSkeletalMeshConverter
{
	GENERATED_BODY()

public:
	/** Build a skeletal mesh from the specified Cloth Asset. */
	virtual bool ExportToSkeletalMesh(const class UChaosClothAssetBase& ClothAssetBase, class USkeletalMesh& SkeletalMesh) const override;
};
