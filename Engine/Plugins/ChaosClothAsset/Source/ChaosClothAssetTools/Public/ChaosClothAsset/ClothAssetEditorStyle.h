// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

#define UE_API CHAOSCLOTHASSETTOOLS_API

namespace UE::Chaos::ClothAsset
{
	/**
	 * Editor style setting up the cloth asset icons in editor.
	 */
	class FClothAssetEditorStyle final : public FSlateStyleSet
	{
	public:
		UE_API FClothAssetEditorStyle();
		UE_API virtual ~FClothAssetEditorStyle() override;

	public:
		static FClothAssetEditorStyle& Get()
		{
			if (!Singleton.IsSet())
			{
				Singleton.Emplace();
			}
			return Singleton.GetValue();
		}

		static void Destroy()
		{
			Singleton.Reset();
		}

	private:
		static UE_API TOptional<FClothAssetEditorStyle> Singleton;
	};
}  // End namespace UE::Chaos::ClothAsset

#undef UE_API
