// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"

#include "ClothAssetFactory.generated.h"

#define UE_API CHAOSCLOTHASSETTOOLS_API

/**
 * Having a cloth factory allows the cloth asset to be created from the Editor's menus.
 */
UCLASS(MinimalAPI, Experimental)
class UChaosClothAssetFactory : public UFactory
{
	GENERATED_BODY()
public:
	UE_API UChaosClothAssetFactory(const FObjectInitializer& ObjectInitializer);

	/** UFactory Interface */
	virtual bool CanCreateNew() const override { return true; }
	virtual bool FactoryCanImport(const FString& Filename) override { return false; }
	virtual bool ShouldShowInNewMenu() const override { return true; }
	UE_API virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	UE_API virtual FString GetDefaultNewAssetName() const override;
	/** End UFactory Interface */

	/**
	 * Non-modal helper that creates a new cloth asset from a Dataflow template asset path,
	 * bypassing the template-picker dialog. Use from scripted / agent-driven flows.
	 * @param Class           Asset class to create (typically UChaosClothAsset::StaticClass()).
	 * @param Parent          Outer package for the new asset.
	 * @param Name            Asset name.
	 * @param Flags           Object flags.
	 * @param TemplatePath    Object path of the Dataflow template to clone (may be nullptr for "no Dataflow").
	 * @param bEmbedDataflow  Whether to embed the Dataflow graph in the asset or keep it external.
	 */
	UE_API static UObject* CreateClothAssetFromTemplate(
		UClass* Class, UObject* Parent, FName Name, EObjectFlags Flags,
		const FString* TemplatePath, bool bEmbedDataflow);
};

#undef UE_API
