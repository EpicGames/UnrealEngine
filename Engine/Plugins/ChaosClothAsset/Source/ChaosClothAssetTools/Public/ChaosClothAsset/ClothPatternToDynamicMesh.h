// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ClothPatternVertexType.h"
#include "Templates/SharedPointer.h"

#define UE_API CHAOSCLOTHASSETTOOLS_API


namespace UE::Geometry { class FDynamicMesh3; }
class UChaosClothAsset;
struct FManagedArrayCollection;

namespace UE::Chaos::ClothAsset
{

/**
* Convert a single pattern from a ClothAsset to a FDynamicMesh3. 
* When PatternIndex = INDEX_NONE, convert the entire cloth asset to an FDynamicMesh3.
* When VertexDataType = EClothPatternVertexType::Sim3D, this generates a welded mesh (with native welded indexing) when PatternIndex = INDEX_NONE,
*	and an unwelded (with native 2D unwelded indexing) when PatternIndex is a valid SimPattern Index.
* Use MaterialOffset to specify an offset into the material group ID, and run a Fix up on any triangles without valid material IDs.
* When MaterialOffset is INDEX_NONE, no offset is applied and no fix up is run.
* Use bConvertWeightMaps to turn the cloth collection weight maps into a Dynamic Mesh attribute.
*/
class FClothPatternToDynamicMesh
{
public:

	UE_API void Convert(
		const TSharedRef<const FManagedArrayCollection> ClothCollection,
		int32 PatternIndex,
		EClothPatternVertexType VertexDataType,
		UE::Geometry::FDynamicMesh3& MeshOut,
		bool bDisableAttributes = false,
		int32 MaterialOffset = INDEX_NONE,
		bool bFlip2DSimFaces = false,
		bool bConvertWeightMaps = true);

	UE_API void Convert(
		const UChaosClothAsset* ClothAssetMeshIn,
		int32 LODIndex,
		int32 PatternIndex,
		EClothPatternVertexType VertexDataType,
		UE::Geometry::FDynamicMesh3& MeshOut,
		int32 MaterialOffset = INDEX_NONE,
		bool bFlip2DSimFaces = false,
		bool bConvertWeightMaps = true);
};

}	// namespace UE::Chaos::ClothAsset

#undef UE_API
