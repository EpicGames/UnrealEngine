// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "DynamicMesh/NonManifoldMappingSupport.h"


namespace UE::Geometry { class FDynamicMesh3; }

namespace UE::Chaos::ClothAsset
{

class FDynamicMeshSourceTriangleIdAttribute;

// This uses the NonManifoldMapping attribute for vertices, and a new attribute for triangles.
class FClothPatternToDynamicMeshMappingSupport : public UE::Geometry::FNonManifoldMappingSupport
{
public:
	CHAOSCLOTHASSETTOOLS_API FClothPatternToDynamicMeshMappingSupport(const UE::Geometry::FDynamicMesh3& Mesh);

	/**
	* Update the support for a new DynamicMesh.
	*/
	CHAOSCLOTHASSETTOOLS_API void Reset(const UE::Geometry::FDynamicMesh3& Mesh);

	/*
	* Return true if attribute data indicates that the source data that was converted to this DynamicMesh contained mapped vertices.
	*/
	bool IsMappedVertexInSource() const
	{
		return IsNonManifoldVertexInSource();
	}

	/*
	* Return true if attribute data indicates that the source data that was converted to this DynamicMesh contained mapped triangles.
	*/
	CHAOSCLOTHASSETTOOLS_API bool IsMappedTriangleInSource() const;

	/*
	* Return true if the provided DynamicMesh vertex id resulted from a remapped vertex in the source data.
	* @param vid - the id of a vertex in the DynamicMesh.
	*
	* Note: the code assumes but does not check that vid is a valid vertex id
	*/
	bool IsMappedVertexID(const int32 vid) const
	{
		return !(vid == GetOriginalNonManifoldVertexID(vid));
	}

	/*
	* Return true if the provided DynamicMesh triangle id resulted from a remapped triangle in the source data.
	* @param Tid - the id of a triangle in the DynamicMesh.
	*
	* Note: the code assumes but does not check that Tid is a valid triangle id
	*/
	bool IsMappedTriangleID(const int32 Tid) const
	{
		return !(Tid == GetOriginalTriangleID(Tid));
	}

	/*
	* Return the vertex ID in the original cloth data used to generate this DynamicMesh associated with the provided vertex id.
	* In the case that the source data was actually manifold the returned vertex id will be identical to the DynamicMesh vertex id.
	* @param vid - the id of a vertex in the DynamicMesh.
	*
	* Note: the code assumes but does not check that vid is a valid vertex vid.
	*/
	int32 GetOriginalVertexID(const int32 vid) const
	{
		return GetOriginalNonManifoldVertexID(vid);
	}

	/*
	* Return the triangle ID in the original cloth data used to generate this DynamicMesh associated with the provided triangle id.
	* In the case that the source data was actually manifold the returned triangle id will be identical to the DynamicMesh triangle id.
	* @param Tid - the id of a triangle in the DynamicMesh.
	*
	* Note: the code assumes but does not check that Tid is a valid triangle id
	*/
	CHAOSCLOTHASSETTOOLS_API int32 GetOriginalTriangleID(const int32 Tid) const;


	// --- helper functions.

	/*
	* Attaches or replaces vertex mapping data to the provided mesh.
	* @param VertexToOriginalVertexIDMap - an array that maps each DynamicMesh vertex id to the associated original vertex id.
	* @return false on failure (no attribute will be attached to the DynamicMesh in this case)
	*
	* Note: Failure occurs if the DynamicMesh does not have attributes enabled or if the provided array is not long enough to provide a mapping value for each DynamicMesh vertex id.
	*/
	static bool AttachVertexMappingData(const TArray<int32>& VertexToOriginalVertexIDMap, UE::Geometry::FDynamicMesh3& InOutMesh)
	{
		return AttachNonManifoldVertexMappingData(VertexToOriginalVertexIDMap, InOutMesh);
	}

	/*
	* Attaches or replaces  triangle mapping data to the provided mesh.
	* @param TriangleToOriginalVertexIDMap - an array that maps each DynamicMesh triangle id to the associated original triangle id.
	* @return false on failure (no attribute will be attached to the DynamicMesh in this case)
	*
	* Note: Failure occurs if the DynamicMesh does not have attributes enabled or if the provided array is not long enough to provide a mapping value for each DynamicMesh triangle id.
	*/
	static CHAOSCLOTHASSETTOOLS_API bool AttachTriangleMappingData(const TArray<int32>& TriangleToOriginalVertexIDMap, UE::Geometry::FDynamicMesh3& InOutMesh);

	/*
	*  Removes triangle mapping data.
	*
	*  Note: this will invalidate any NonManifoldMappingSupport/ClothPatternToDynamicMeshMappingSupport object associated with this DynamicMesh,
	*  and subsequent use of such object will produce unexpected results.
	*/
	static void RemoveVertexMappingData(UE::Geometry::FDynamicMesh3& InOutMesh)
	{
		RemoveNonManifoldVertexMappingData(InOutMesh);
	}

	static CHAOSCLOTHASSETTOOLS_API void RemoveTriangleMappingData(UE::Geometry::FDynamicMesh3& InOutMesh);

	/*
	*  Removes all mapping data.
	*
	*  Note, this will invalidate any  NonManifoldMappingSupport/ClothPatternToDynamicMeshMappingSupport object associated with this DynamicMesh,
	*  and subsequent use of such object will produce unexpected results.
	*/
	static void RemoveAllMappingData(UE::Geometry::FDynamicMesh3& InOutMesh)
	{
		RemoveAllNonManifoldMappingData(InOutMesh);
		RemoveTriangleMappingData(InOutMesh);
	}

	/*
	* Name used to identify triangle attribute data generated during conversion to a DynamicMesh in the case that the source was non-manifold.
	*/
	static const CHAOSCLOTHASSETTOOLS_API FName ClothMeshTIDsAttrName;

protected:

	const FDynamicMeshSourceTriangleIdAttribute* SrcTIDsAttribute = nullptr;
};

}	// namespace UE::Chaos::ClothAsset

