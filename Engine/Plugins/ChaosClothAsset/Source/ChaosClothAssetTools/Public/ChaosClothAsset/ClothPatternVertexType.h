// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

namespace UE::Chaos::ClothAsset
{
	enum class EClothPatternVertexType : uint8
	{
		Sim2D = 0,
		Sim3D = 1,
		Render = 2
	};
}

