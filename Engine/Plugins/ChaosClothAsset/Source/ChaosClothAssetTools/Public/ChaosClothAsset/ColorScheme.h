// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Color.h"

namespace UE::Chaos::ClothAsset
{
	struct FColorScheme
	{
		static inline const FLinearColor Asset = FColor(242, 77, 26);  // Orange
		static inline const FLinearColor NodeHeader = FColor(180, 120, 110);
		static inline const FLinearColor NodeBody = FColor(18, 12, 11, 127);
	};
}

