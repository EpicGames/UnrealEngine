// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Features/IModularFeature.h"
#include "UObject/NameTypes.h"

class UChaosClothAsset;
class UClothingAssetCommon;

namespace UE::Chaos::ClothAsset
{
	/**
	 * Modular-feature interface that lets ChaosClothAssetTools delegate legacy UClothingAssetCommon
	 * -> UChaosClothAsset conversion to FLegacyClothingConverter::ConvertInto, which lives in the
	 * higher-level ChaosClothAssetEditor module. The dependency direction prevents a direct call
	 * (ChaosClothAssetEditor -> ChaosClothAssetTools), so the higher module registers an
	 * implementation on startup and the lower module looks it up via IModularFeatures when the
	 * right-click "Export to Chaos Cloth Asset" exporter runs.
	 *
	 * Returns true on success (TargetAsset has been populated by the new converter). Returns false
	 * when the converter rejected the inputs; with no registered implementation, the caller should
	 * fall back to the legacy in-tree exporter logic.
	 */
	class ILegacyClothingConverterProvider : public IModularFeature
	{
	public:
		inline static const FName FeatureName = TEXT("ILegacyClothingConverterProvider");

		virtual bool ConvertInto(const UClothingAssetCommon* Source, UChaosClothAsset* Target) const = 0;
	};
}
