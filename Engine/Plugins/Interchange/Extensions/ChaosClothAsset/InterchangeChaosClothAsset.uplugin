{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Interchange Chaos Cloth Asset",
	"Description": "Contains the classes and objects required to allow Interchange to produce Chaos Cloth Assets",
	"Category": "Importers",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"Installed": false,
	"IsExperimentalVersion": true,
	"SupportedTargetPlatforms": [
		"Win64",
		"Linux",
		"Mac"
	],
	"Modules": [
		{
			"Name": "InterchangeChaosClothAssetImport",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "Interchange",
			"Enabled": true
		},
		{
			"Name": "ChaosClothAsset",
			"Enabled": true
		},
		{
			"Name": "ChaosClothAssetDataflowNodes",
			"Enabled": true
		}
	]
}
