// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class InterchangeChaosClothAssetImport : ModuleRules
	{
		public InterchangeChaosClothAssetImport(ReadOnlyTargetRules Target) : base(Target)
		{
			ShortName = "IntCCA";

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Chaos",
					"ChaosClothAsset",
					"ChaosClothAssetDataflowNodes",
					"ChaosClothAssetEngine",
					"DataflowEngine",
					"InterchangeAnalytics",
					"InterchangeCore",
					"InterchangeEngine",
					"InterchangeFactoryNodes",
					"InterchangeImport",
					"InterchangeNodes",
					"MeshDescription",
					"StaticMeshDescription",
				}
			);

			// Explicitly add this internal folder as an include as moving the files out to "Public" would
			// require some deprecation procedures
			PrivateIncludePaths.Add(System.IO.Path.Combine(GetModuleDirectory("ChaosClothAssetDataflowNodes"), "Internal"));
		}
	}
}
