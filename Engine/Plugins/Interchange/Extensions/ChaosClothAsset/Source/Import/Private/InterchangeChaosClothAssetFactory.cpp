// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeChaosClothAssetFactory.h"

#include "InterchangeChaosClothAssetDefinitions.h"
#include "InterchangeChaosClothAssetFactoryNode.h"
#include "InterchangeChaosClothAssetImportLog.h"
#include "InterchangeChaosClothAssetPayloadData.h"

#include "InterchangeImportCommon.h"
#include "InterchangeMeshFactoryNode.h"
#include "InterchangeMeshNode.h"
#include "InterchangeSceneNode.h"
#include "InterchangeSourceData.h"
#include "InterchangeTranslatorBase.h"
#include "InterchangeGenericPayloadData.h"
#include "InterchangeGenericPayloadInterface.h"
#include "Mesh/InterchangeMeshPayload.h"
#include "Mesh/InterchangeMeshPayloadInterface.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "Nodes/InterchangeUserDefinedAttribute.h"

#include "ChaosClothAsset/ClothAsset.h"
#include "ChaosClothAsset/ClothGeometryTools.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "ChaosClothAsset/CollectionClothSelectionFacade.h"
#include "ChaosClothAsset/RenderMeshImport.h"
#include "Dataflow/DataflowObject.h"

#include "StaticMeshAttributes.h"
#include "StaticMeshOperations.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeChaosClothAssetFactory)

#define LOCTEXT_NAMESPACE "InterchangeChaosClothAssetFactory"

static float GMeshNormalTangentRepairThreshold = 0.05f;
static FAutoConsoleVariableRef CVarMeshNormalTangentRepairThreshold(
	TEXT("ChaosCloth.MeshNormalTangentRepairThreshold"),
	GMeshNormalTangentRepairThreshold,
	TEXT("We will try repairing up to this fraction of a Mesh's normals when invalid. If a Mesh has more invalid normals than this, we will "
		 "recompute all of them. Defaults to 0.05 (5% of all normals).")
);

namespace UE::InterchangeChaosClothAssetFactory::Private
{
	void InstantiateDataflowIfNeeded(UChaosClothAsset* ChaosClothAsset, const FSoftObjectPath& GraphTemplatePath)
	{
		// Reference: AssetDefinition_DataflowAsset.cpp, FactoryCreateNew

		if (!ChaosClothAsset || !GraphTemplatePath.IsValid())
		{
			return;
		}

		UDataflow* DataflowTemplate = LoadObject<UDataflow>(nullptr, GraphTemplatePath.ToString());
		if (!DataflowTemplate)
		{
			return;
		}

		UDataflow* const EmbeddedDataflow = DuplicateObject(DataflowTemplate, ChaosClothAsset, TEXT("EmbeddedDataflow"));
		if (!EmbeddedDataflow)
		{
			return;
		}

		// If we don't clear these we'll have trouble deleting this asset later as it will contain another standalone asset, 
		// so its internal references will count as strong references
		EmbeddedDataflow->ClearFlags(RF_Public | RF_Standalone); 
		EmbeddedDataflow->SetFlags(RF_Transactional);

		if (IDataflowInstanceInterface* Interface = Cast<IDataflowInstanceInterface>(ChaosClothAsset))
		{
			Interface->GetDataflowInstance().SetDataflowAsset(EmbeddedDataflow);
		}
	}
}

UClass* UInterchangeChaosClothAssetFactory::GetFactoryClass() const
{
	return UChaosClothAsset::StaticClass();
}

void UInterchangeChaosClothAssetFactory::CreatePayloadTasks(const FImportAssetObjectParams& Arguments, bool bAsync, TArray<TSharedPtr<UE::Interchange::FInterchangeTaskBase>>& PayloadTasks)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeChaosClothAssetFactory::CreatePayloadTasks);

	using namespace UE::Interchange;
	using namespace UE::Interchange::ChaosCloth;

	UInterchangeChaosClothAssetFactoryNode* ChaosClothFactoryNode = Cast<UInterchangeChaosClothAssetFactoryNode>(Arguments.AssetNode);
	if (ChaosClothFactoryNode == nullptr)
	{
		return;
	}

	const IInterchangeGenericPayloadInterface* ChaosClothPayloadInterface = Cast<IInterchangeGenericPayloadInterface>(Arguments.Translator);
	if (!ChaosClothPayloadInterface)
	{
		UE_LOG(
			LogInterchangeChaosClothAssetImport,
			Error,
			TEXT("Cannot import ChaosCloth asset. The translator does not implement IInterchangeGenericPayloadInterface.")
		);
		return;
	}

	const IInterchangeMeshPayloadInterface* MeshTranslatorPayloadInterface = Cast<IInterchangeMeshPayloadInterface>(Arguments.Translator);
	if (!MeshTranslatorPayloadInterface)
	{
		UE_LOG(
			LogInterchangeChaosClothAssetImport,
			Error,
			TEXT("Cannot import ChaosCloth asset. The translator does not implement IInterchangeMeshPayloadInterface.")
		);
		return;
	}

	TArray<FString> ClothRootUids;
	ChaosClothFactoryNode->GetTargetNodeUids(ClothRootUids);
	if (ClothRootUids.Num() != 1)
	{
		return;
	}
	const FString& ClothRootUid = ClothRootUids[0];

	const UInterchangeSceneNode* ClothRootNode = Cast<UInterchangeSceneNode>(Arguments.NodeContainer->GetNode(ClothRootUid));
	if (!ClothRootNode)
	{
		return;
	}

	const EInterchangeTaskThread TaskThread = bAsync ? EInterchangeTaskThread::AsyncThread : EInterchangeTaskThread::GameThread;

	// Collect simulation mesh payload data
	if (bool bImportSimulationMeshes = false; ChaosClothFactoryNode->GetImportSimulationMeshes(bImportSimulationMeshes) && bImportSimulationMeshes)
	{
		TArray<FString> SimulationMeshNodeUids;
		TOptional<FString> UnusedPayloadKey = {};
		UInterchangeUserDefinedAttributesAPI::GetUserDefinedAttribute(ClothRootNode, SimMeshesAttributeName, SimulationMeshNodeUids, UnusedPayloadKey);

		// Note that this is critical, as we can't have PayloadDataArray reallocate while the async tasks are running
		SimMeshPayloadDataArray.Reserve(SimulationMeshNodeUids.Num());

		for (const FString& SimulationMeshNodeUid : SimulationMeshNodeUids)
		{
			const UInterchangeMeshNode* MeshNode = Cast<UInterchangeMeshNode>(Arguments.NodeContainer->GetNode(SimulationMeshNodeUid));
			if (!MeshNode)
			{
				continue;
			}

			TOptional<FInterchangeMeshPayLoadKey> MeshPayloadKey = MeshNode->GetPayLoadKey();
			if (!MeshPayloadKey)
			{
				continue;
			}

			TSharedPtr<FManagedArrayCollection>& PayloadDataRef = SimMeshPayloadDataArray.Emplace_GetRef();
			PayloadDataRef = MakeShared<FManagedArrayCollection>();

			PayloadTasks.Add(MakeShared<FInterchangeTaskLambda, ESPMode::ThreadSafe>(
				TaskThread,
				[&PayloadDataRef, ChaosClothPayloadInterface, MeshPayloadKey]()
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeChaosClothAssetFactory::GetChaosClothSimulationPayloadDataTask);

					UInterchangeChaosClothAssetPayloadData* ClothPayloadData = Cast<UInterchangeChaosClothAssetPayloadData>(
						ChaosClothPayloadInterface->GetGenericPayloadData(*MeshPayloadKey->UniqueId)
					);

					if (ClothPayloadData)
					{
						PayloadDataRef = ClothPayloadData->Collection;
					}
				}
			));
		}
	}

	// Collect render mesh payload data
	if (bool bImportRenderMeshes = false; ChaosClothFactoryNode->GetImportRenderMeshes(bImportRenderMeshes) && bImportRenderMeshes)
	{
		TArray<FString> RenderMeshNodeUids;
		TOptional<FString> UnusedPayloadKey = {};
		UInterchangeUserDefinedAttributesAPI::GetUserDefinedAttribute(ClothRootNode, RenderMeshesAttributeName, RenderMeshNodeUids, UnusedPayloadKey);

		RenderMeshPayloadDataArray.Reserve(RenderMeshNodeUids.Num());

		for (const FString& RenderMeshNodeUid : RenderMeshNodeUids)
		{
			const UInterchangeMeshNode* MeshNode = Cast<UInterchangeMeshNode>(Arguments.NodeContainer->GetNode(RenderMeshNodeUid));
			if (!MeshNode)
			{
				continue;
			}

			// We have to step through the targets here instead of trying to build the factory node UID directly because in some
			// contexts (e.g. combine meshes) the factory node is named after the scene node, not the asset node...
			UInterchangeMeshFactoryNode* MeshFactoryNode = nullptr;
			{
				TArray<FString> TargetNodeUids;
				MeshNode->GetTargetNodeUids(TargetNodeUids);

				// We expect to find only one mesh factory node for each mesh node
				ensure(TargetNodeUids.Num() <= 1);
				if (!TargetNodeUids.IsEmpty())
				{
					MeshFactoryNode = Cast<UInterchangeMeshFactoryNode>(Arguments.NodeContainer->GetFactoryNode(TargetNodeUids[0]));
				}
			}
			if (!MeshFactoryNode)
			{
				continue;
			}

			FAttributeStorage PayloadAttributes;
			PayloadAttributes.RegisterAttribute(UE::Interchange::FAttributeKey{MeshPayload::Attributes::MeshGlobalTransform}, FTransform::Identity);
			UInterchangeMeshFactoryNode::CopyPayloadKeyStorageAttributes(MeshFactoryNode, PayloadAttributes);

			TOptional<FInterchangeMeshPayLoadKey> MeshPayloadKey = MeshNode->GetPayLoadKey();
			if (!MeshPayloadKey)
			{
				continue;
			}

			FRenderMeshPayloadData& RenderMeshPayloadData = RenderMeshPayloadDataArray.Emplace_GetRef();
			RenderMeshPayloadData.MeshPayload.Transform = FTransform::Identity;
			RenderMeshPayloadData.MeshPayload.MeshName = MeshPayloadKey->UniqueId; // For USD translations these are usually just the mesh prim paths
			RenderMeshPayloadData.MeshFactoryNodeUid = MeshFactoryNode->GetUniqueID();

			// Fetch the regular mesh payload data
			PayloadTasks.Add(MakeShared<FInterchangeTaskLambda, ESPMode::ThreadSafe>(
				TaskThread,
				[&RenderMeshPayloadData, MeshTranslatorPayloadInterface, MeshPayloadKey, PayloadAttributes]()
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeChaosClothAssetFactory::GetChaosClothRenderPayloadDataTask);

					RenderMeshPayloadData.MeshPayload.PayloadData = MeshTranslatorPayloadInterface->GetMeshPayloadData(
						MeshPayloadKey.GetValue(),
						PayloadAttributes
					);
				}
			));

			// Fetch render pattern payload data
			PayloadTasks.Add(MakeShared<FInterchangeTaskLambda, ESPMode::ThreadSafe>(
				TaskThread,
				[&RenderMeshPayloadData, ChaosClothPayloadInterface, MeshPayloadKey]()
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeChaosClothAssetFactory::GetChaosClothRenderPatternsDataTask);

					UInterchangeChaosClothAssetPayloadData* ClothPayloadData = Cast<UInterchangeChaosClothAssetPayloadData>(
						ChaosClothPayloadInterface->GetGenericPayloadData(*MeshPayloadKey->UniqueId)
					);

					if (ClothPayloadData)
					{
						RenderMeshPayloadData.RenderPatterns = MoveTemp(ClothPayloadData->RenderPatterns);
					}
				}
			));
		}
	}
}

UInterchangeFactoryBase::FImportAssetResult UInterchangeChaosClothAssetFactory::ImportAsset_Async(const FImportAssetObjectParams& Arguments)
{
	using namespace UE::InterchangeChaosClothAssetFactory::Private;
	using namespace UE::Interchange::ChaosCloth;
	using namespace UE::Chaos::ClothAsset;

	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeChaosClothAssetFactory::ImportAsset_Async);

	FImportAssetResult Result;

	// All our payload tasks should have completed by now, so let's combine them all into a single CombinedCollection we can put into the ClothAsset variable
	CombinedCollection = MakeShared<FManagedArrayCollection>();
	TSharedRef<FManagedArrayCollection> CombinedCollectionRef = CombinedCollection.ToSharedRef();

	UInterchangeChaosClothAssetFactoryNode* ChaosClothFactoryNode = Cast<UInterchangeChaosClothAssetFactoryNode>(Arguments.AssetNode);
	if(!ChaosClothFactoryNode)
	{
		return Result;
	}

	// Add solver properties to the collection
	{
		FCollectionClothFacade ClothFacade{CombinedCollectionRef};
		if (!ClothFacade.IsValid())
		{
			ClothFacade.DefineSchema();
		}

		float AirDamping = 1.0f;
		if (ChaosClothFactoryNode->GetAirDamping(AirDamping))
		{
			ClothFacade.SetSolverAirDamping(AirDamping);
		}

		FVector3f Gravity = {0, -9800, 0};
		if (ChaosClothFactoryNode->GetGravity(Gravity))
		{
			ClothFacade.SetSolverGravity(Gravity);
		}

		int32 SubStepCount = 1;
		if (ChaosClothFactoryNode->GetSubStepCount(SubStepCount))
		{
			ClothFacade.SetSolverSubSteps(SubStepCount);
		}

		float TimeStep = 0.033333f;
		if (ChaosClothFactoryNode->GetTimeStep(TimeStep))
		{
			ClothFacade.SetSolverTimeStep(TimeStep);
		}
	}

	// Add simulation data into the collection
	bool bImportSimulationMeshes = false;
	if (ChaosClothFactoryNode->GetImportSimulationMeshes(bImportSimulationMeshes) && bImportSimulationMeshes)
	{
		// For now we mostly have a single sim mesh it seems, so we'll largely just do the move part
		for (int32 Index = 0; Index < SimMeshPayloadDataArray.Num(); ++Index)
		{
			SimMeshPayloadDataArray[Index]->CopyTo(CombinedCollection.Get());
		}
	}

	// Add render data into the collection
	bool bImportRenderMeshes = false;
	if (ChaosClothFactoryNode->GetImportRenderMeshes(bImportRenderMeshes) && bImportRenderMeshes)
	{
		// Providing nothing for AppendSettings.PolygonGroupDelegate means it will combine sections by name, which works for now
		FStaticMeshOperations::FAppendSettings AppendSettings;
		for (int32 ChannelIdx = 0; ChannelIdx < FStaticMeshOperations::FAppendSettings::MAX_NUM_UV_CHANNELS; ++ChannelIdx)
		{
			AppendSettings.bMergeUVChannels[ChannelIdx] = true;
		}

		FMeshDescription CombinedMeshDescription;
		FStaticMeshAttributes Attributes{CombinedMeshDescription};
		Attributes.Register();

		TMap<FName, TSet<int32>> CombinedRenderPatterns;

		TArray<TPair<FString, FString>> CombinedMaterialDependencies;

		// Here we'll do a fake collapsing in-place, by just appending all the render mesh descriptions together.
		for (FRenderMeshPayloadData& RenderMeshPayload : RenderMeshPayloadDataArray)
		{
			if (!RenderMeshPayload.MeshPayload.PayloadData.IsSet())
			{
				continue;
			}

			// Our render pattern indices corresponds to just the Mesh data we got from this prim.
			// Since we're combining everything, we need to offset them
			const int32 NumExistingTriangles = CombinedMeshDescription.Triangles().Num();
			for (TPair<FName, TSet<int32>>& Pattern : RenderMeshPayload.RenderPatterns)
			{
				for (int32& TriangleIndex : Pattern.Value)
				{
					TriangleIndex += NumExistingTriangles;
				}
			}

			if (CombinedMeshDescription.IsEmpty())
			{
				CombinedMeshDescription = MoveTemp(RenderMeshPayload.MeshPayload.PayloadData->MeshDescription);
			}
			else
			{
				FStaticMeshOperations::AppendMeshDescription(
					RenderMeshPayload.MeshPayload.PayloadData->MeshDescription,
					CombinedMeshDescription,
					AppendSettings
				);
			}

			const int32 NumCombinedBefore = CombinedRenderPatterns.Num();
			CombinedRenderPatterns.Append(RenderMeshPayload.RenderPatterns);
			const int32 NumCombinedAfter = CombinedRenderPatterns.Num();

			// For now we don't expect these render pattern names to collide, so ensure that
			ensure(NumCombinedAfter - NumCombinedBefore == RenderMeshPayload.RenderPatterns.Num());

			// Merge the material slots, since our mesh descriptions will also merge them
			UInterchangeMeshFactoryNode* MeshFactoryNode = Cast<UInterchangeMeshFactoryNode>(
				Arguments.NodeContainer->GetFactoryNode(RenderMeshPayload.MeshFactoryNodeUid)
			);
			if (MeshFactoryNode)
			{
				TMap<FString, FString> MaterialDependencies;
				MeshFactoryNode->GetSlotMaterialDependencies(MaterialDependencies);

				TArray<TPair<FString, FString>> PairsToAdd;
				for (const TPair<FString, FString>& MaterialDependency : MaterialDependencies)
				{
					bool bSlotAlreadyExists = false;
					for (const TPair<FString, FString>& ExistingPair : CombinedMaterialDependencies)
					{
						if (MaterialDependency.Key == ExistingPair.Key)
						{
							bSlotAlreadyExists = true;
							break;
						}
					}

					if (!bSlotAlreadyExists)
					{
						// Add the new slots to a temp array so we don't uselessly loop over them as well while
						// searching for collisions for the next items: We know we don't have collisions internal to the
						// MaterialDependencies map
						PairsToAdd.Add(MaterialDependency);
					}
				}

				CombinedMaterialDependencies.Append(PairsToAdd);
			}
		}

		EComputeNTBsFlags Options = EComputeNTBsFlags::UseMikkTSpace;
		FStaticMeshOperations::ComputeTriangleTangentsAndNormals(CombinedMeshDescription);
		FStaticMeshOperations::RecomputeNormalsAndTangentsIfNeeded(CombinedMeshDescription, Options);

		// Reference: USDGeomMeshTranslator.cpp, CreateStaticMesh()
		static FMeshBuildSettings DefaultBuildSettings;
		DefaultBuildSettings.bGenerateLightmapUVs = false;
		DefaultBuildSettings.bRecomputeNormals = false;
		DefaultBuildSettings.bRecomputeTangents = false;
		DefaultBuildSettings.bRemoveDegenerates = true;	// Note: This may get rid of the entire mesh if it is all invalid

		// This struct's constructor does the "import", by converting the mesh description into an intermediate format
		// that we'll later offload into the CombinedCollection below with the AddRenderSections / AddRenderPatternSelectionSets calls
		//
		// Reference: FChaosClothAssetUSDImportNode_v3::ImportRenderStaticMesh
		UE::Chaos::ClothAsset::FRenderMeshImport RenderMeshImport{CombinedMeshDescription, DefaultBuildSettings};

		// Get existing material paths from the reimport object for fallback during reimport
		TArray<FSoftObjectPath> ExistingMaterialPaths;
		if (const UChaosClothAsset* ExistingClothAsset = Cast<UChaosClothAsset>(Arguments.ReimportObject))
		{
			const TArray<TSharedRef<const FManagedArrayCollection>>& ExistingCollections = ExistingClothAsset->GetClothCollections();
			if (ExistingCollections.Num() > 0)
			{
				FCollectionClothConstFacade ExistingFacade(ExistingCollections[0]);
				if (ExistingFacade.IsValid())
				{
					TConstArrayView<FSoftObjectPath> ExistingPaths = ExistingFacade.GetRenderMaterialSoftObjectPathName();
					ExistingMaterialPaths = TArray<FSoftObjectPath>(ExistingPaths);
				}
			}
		}

		// Collect the material paths from our resolved material dependencies
		TArray<FSoftObjectPath> MaterialPaths;
		MaterialPaths.Reserve(CombinedMaterialDependencies.Num());
		for (int32 MaterialIndex = 0; MaterialIndex < CombinedMaterialDependencies.Num(); ++MaterialIndex)
		{
			const FString& SlotName = CombinedMaterialDependencies[MaterialIndex].Key;
			const FString& MaterialFactoryNodeUid = CombinedMaterialDependencies[MaterialIndex].Value;

			UInterchangeFactoryBaseNode* MaterialFactoryNode = Arguments.NodeContainer->GetFactoryNode(MaterialFactoryNodeUid);

			FSoftObjectPath ReferencedObject;
			if (MaterialFactoryNode && MaterialFactoryNode->GetCustomReferenceObject(ReferencedObject))
			{
				MaterialPaths.Add(ReferencedObject);
			}
			else if (ExistingMaterialPaths.IsValidIndex(MaterialIndex))
			{
				// Reimport fallback: reuse the material from the existing asset
				MaterialPaths.Add(ExistingMaterialPaths[MaterialIndex]);
			}
		}

		RenderMeshImport.AddRenderSections(CombinedCollectionRef, MaterialPaths, OriginalIndicesName, OriginalIndicesName);
		RenderMeshImport.AddRenderPatternSelectionSets(CombinedCollectionRef, CombinedRenderPatterns, OriginalIndicesName);
	}

	// Bind to root bone on exit
	{
		const bool bBindSimulationMesh = bImportSimulationMeshes;
		const bool bBindRenderMesh = bImportRenderMeshes;
		FClothGeometryTools::BindMeshToRootBone(CombinedCollectionRef, bBindSimulationMesh, bBindRenderMesh);
	}

	return Result;
}

UInterchangeFactoryBase::FImportAssetResult UInterchangeChaosClothAssetFactory::BeginImportAsset_GameThread(const FImportAssetObjectParams& Arguments)
{
	using namespace UE::InterchangeChaosClothAssetFactory::Private;

	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeChaosClothAssetFactory::BeginImportAsset_GameThread);

	FImportAssetResult ImportAssetResult;

#if WITH_EDITOR
	auto LogChaosClothFactoryError = [this, &Arguments, &ImportAssetResult](const FText& Info)
		{
			UInterchangeResultError_Generic* Message = AddMessage<UInterchangeResultError_Generic>();
			Message->SourceAssetName = Arguments.SourceData->GetFilename();
			Message->DestinationAssetName = Arguments.AssetName;
			Message->AssetType = GetFactoryClass();
			Message->Text = FText::Format(
				LOCTEXT("ChaosClothFactory_Failure", "UInterchangeChaosClothAssetFactory: Could not create ChaosCloth asset '{0}'. Reason: {1}"),
				FText::FromString(Arguments.AssetName),
				Info
			);
			ImportAssetResult.bIsFactorySkipAsset = true;
		};

	const IInterchangeGenericPayloadInterface* ChaosClothPayloadInterface = Cast<IInterchangeGenericPayloadInterface>(Arguments.Translator);
	if (!ChaosClothPayloadInterface)
	{
		LogChaosClothFactoryError(LOCTEXT("ChaosClothFactory_NoPayloadInterface", "The translator does not implement IInterchangeGenericPayloadInterface"));
		return ImportAssetResult;
	}

	UInterchangeChaosClothAssetFactoryNode* ChaosClothFactoryNode = Cast<UInterchangeChaosClothAssetFactoryNode>(Arguments.AssetNode);
	if (!ChaosClothFactoryNode)
	{
		LogChaosClothFactoryError(LOCTEXT("ChaosClothFactory_AssetNodeNull", "Asset node parameter is not a UInterchangeChaosClothAssetFactoryNode."));
		return ImportAssetResult;
	}

	const UClass* ObjectClass = ChaosClothFactoryNode->GetObjectClass();
	if (!ObjectClass || !ObjectClass->IsChildOf(GetFactoryClass()))
	{
		LogChaosClothFactoryError(LOCTEXT("ChaosClothFactory_NodeClassMissmatch", "Asset node parameter class doesn't derive the UChaosClothAsset class."));
		return ImportAssetResult;
	}

	if (SimMeshPayloadDataArray.Num() == 0 && RenderMeshPayloadDataArray.Num() == 0)
	{
		LogChaosClothFactoryError(LOCTEXT("ChaosClothFactory_NoPayload", "No ChaosCloth payload could be translated from the source."));
		return ImportAssetResult;
	}

	bool bHasValidPayload = false;
	for (const TSharedPtr<FManagedArrayCollection>& PayloadData : SimMeshPayloadDataArray)
	{
		if (!PayloadData->IsEmpty())
		{
			bHasValidPayload = true;
			break;
		}
	}
	if (!bHasValidPayload)
	{
		for (const FRenderMeshPayloadData& PayloadData : RenderMeshPayloadDataArray)
		{
			if (PayloadData.MeshPayload.PayloadData.IsSet() && !PayloadData.MeshPayload.PayloadData->MeshDescription.IsEmpty())
			{
				bHasValidPayload = true;
				break;
			}
		}
	}
	if (!bHasValidPayload)
	{
		LogChaosClothFactoryError(LOCTEXT("ChaosClothFactory_InvalidPayloadData", "No valid ChaosCloth payload data found."));
		return ImportAssetResult;
	}

	UObject* ExistingAsset = Arguments.ReimportObject;
	if (!ExistingAsset)
	{
		FSoftObjectPath ReferenceObject;
		if (ChaosClothFactoryNode->GetCustomReferenceObject(ReferenceObject))
		{
			ExistingAsset = ReferenceObject.TryLoad();
		}
	}

	UChaosClothAsset* ChaosClothAsset = Cast<UChaosClothAsset>(ExistingAsset);
	if (!ChaosClothAsset)
	{
		ChaosClothAsset = NewObject<UChaosClothAsset>(Arguments.Parent, *Arguments.AssetName, RF_Public | RF_Standalone);

		// Set the cloth asset with a new dataflow graph, but only if we created it ourselves
		FSoftObjectPath TemplateDataflowGraphPath;
		if (ChaosClothFactoryNode->GetDataflowGraphPath(TemplateDataflowGraphPath))
		{
			InstantiateDataflowIfNeeded(ChaosClothAsset, TemplateDataflowGraphPath);
		}
	}
	if (!ChaosClothAsset)
	{
		LogChaosClothFactoryError(LOCTEXT("ChaosClothFactory_ChaosClothCreationFailure", "Could not allocate new ChaosClothAsset."));
		return ImportAssetResult;
	}

	if (ChaosClothAsset)
	{
		ImportAssetResult.ImportedObject = ChaosClothAsset;
	}
#endif // WITH_EDITOR

	return ImportAssetResult;
}

void UInterchangeChaosClothAssetFactory::SetupObject_GameThread(const FSetupObjectParams& Arguments)
{
	using namespace UE::Interchange::ChaosCloth;

	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeChaosClothAssetFactory::SetupObject_GameThread);

	Super::SetupObject_GameThread(Arguments);

	UChaosClothAsset* ChaosClothAsset = CastChecked<UChaosClothAsset>(Arguments.ImportedObject);
	if(!ChaosClothAsset)
	{
		return;
	}

#if WITH_EDITORONLY_DATA
	if (ensure(ChaosClothAsset && Arguments.SourceData))
	{
		// We must call the Update of the asset source file in the main thread because UAssetImportData::Update
		// executes some delegate we do not control
		UE::Interchange::FFactoryCommon::FUpdateImportAssetDataParameters UpdateImportAssetDataParameters(
			ChaosClothAsset,
			ChaosClothAsset->GetAssetImportData(),
			Arguments.SourceData,
			Arguments.NodeUniqueID,
			Arguments.NodeContainer,
			Arguments.OriginalPipelines,
			Arguments.Translator
		);
		ChaosClothAsset->SetAssetImportData(UE::Interchange::FFactoryCommon::UpdateImportAssetData(UpdateImportAssetDataParameters));
	}
#endif

	if (CombinedCollection)
	{
		// Set the imported collection into the ImportedCollection variable (even when reimporting)
		FDataflowVariableOverrides& Overrides = ChaosClothAsset->GetDataflowInstance().GetVariableOverrides();
		Overrides.OverrideVariableStruct<FManagedArrayCollection>(ImportedCollectionVariableName, *CombinedCollection);
	}

	// Evaluate the graph right away, or else the asset won't be visible on the viewport
	// TODO: Can the be done in an async thread? It can take a little while for large cloth assets, and this locks up the editor UI
	const bool bUpdateDependentAssets = true;
	bool bSuccess = ChaosClothAsset->GetDataflowInstance().UpdateOwnerAsset(bUpdateDependentAssets);
	ensure(bSuccess);
}

bool UInterchangeChaosClothAssetFactory::GetSourceFilenames(const UObject* Object, TArray<FString>& OutSourceFilenames) const
{
#if WITH_EDITORONLY_DATA
	if (const UChaosClothAsset* ChaosClothAsset = Cast<UChaosClothAsset>(Object))
	{
		return UE::Interchange::FFactoryCommon::GetSourceFilenames(ChaosClothAsset->GetAssetImportData(), OutSourceFilenames);
	}
#endif

	return false;
}

bool UInterchangeChaosClothAssetFactory::SetSourceFilename(const UObject* Object, const FString& SourceFilename, int32 SourceIndex) const
{
#if WITH_EDITORONLY_DATA
	if (const UChaosClothAsset* ChaosClothAsset = Cast<UChaosClothAsset>(Object))
	{
		return UE::Interchange::FFactoryCommon::SetSourceFilename(ChaosClothAsset->GetAssetImportData(), SourceFilename, SourceIndex);
	}
#endif

	return false;
}

void UInterchangeChaosClothAssetFactory::BackupSourceData(const UObject* Object) const
{
#if WITH_EDITORONLY_DATA
	if (const UChaosClothAsset* ChaosClothAsset = Cast<UChaosClothAsset>(Object))
	{
		UE::Interchange::FFactoryCommon::BackupSourceData(ChaosClothAsset->GetAssetImportData());
	}
#endif
}

void UInterchangeChaosClothAssetFactory::ReinstateSourceData(const UObject* Object) const
{
#if WITH_EDITORONLY_DATA
	if (const UChaosClothAsset* ChaosClothAsset = Cast<UChaosClothAsset>(Object))
	{
		UE::Interchange::FFactoryCommon::ReinstateSourceData(ChaosClothAsset->GetAssetImportData());
	}
#endif
}

void UInterchangeChaosClothAssetFactory::ClearBackupSourceData(const UObject* Object) const
{
#if WITH_EDITORONLY_DATA
	if (const UChaosClothAsset* ChaosClothAsset = Cast<UChaosClothAsset>(Object))
	{
		UE::Interchange::FFactoryCommon::ClearBackupSourceData(ChaosClothAsset->GetAssetImportData());
	}
#endif
}

#undef LOCTEXT_NAMESPACE
