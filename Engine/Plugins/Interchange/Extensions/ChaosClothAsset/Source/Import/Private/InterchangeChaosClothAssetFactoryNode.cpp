// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeChaosClothAssetFactoryNode.h"

#include "ChaosClothAsset/ClothAsset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeChaosClothAssetFactoryNode)

FString UInterchangeChaosClothAssetFactoryNode::GetTypeName() const
{
	const FString TypeName = TEXT("ChaosClothFactoryNode");
	return TypeName;
}

UClass* UInterchangeChaosClothAssetFactoryNode::GetObjectClass() const
{
	return UChaosClothAsset::StaticClass();
}

bool UInterchangeChaosClothAssetFactoryNode::GetImportSimulationMeshes(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(ImportSimulationMeshes, bool)
}

bool UInterchangeChaosClothAssetFactoryNode::SetImportSimulationMeshes(bool AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(ImportSimulationMeshes, bool)
}

bool UInterchangeChaosClothAssetFactoryNode::GetImportRenderMeshes(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(ImportRenderMeshes, bool)
}

bool UInterchangeChaosClothAssetFactoryNode::SetImportRenderMeshes(bool AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(ImportRenderMeshes, bool)
}

bool UInterchangeChaosClothAssetFactoryNode::GetDataflowGraphPath(FSoftObjectPath& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(DataflowGraphPath, FSoftObjectPath)
}

bool UInterchangeChaosClothAssetFactoryNode::SetDataflowGraphPath(const FSoftObjectPath& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(DataflowGraphPath, FSoftObjectPath)
}

bool UInterchangeChaosClothAssetFactoryNode::GetAirDamping(float& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(AirDamping, float)
}

bool UInterchangeChaosClothAssetFactoryNode::SetAirDamping(float AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(AirDamping, float)
}

bool UInterchangeChaosClothAssetFactoryNode::GetGravity(FVector3f& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(Gravity, FVector3f)
}

bool UInterchangeChaosClothAssetFactoryNode::SetGravity(const FVector3f& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(Gravity, FVector3f)
}

bool UInterchangeChaosClothAssetFactoryNode::GetSubStepCount(int32& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(SubStepCount, int32)
}

bool UInterchangeChaosClothAssetFactoryNode::SetSubStepCount(int32 AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(SubStepCount, int32)
}

bool UInterchangeChaosClothAssetFactoryNode::GetTimeStep(float& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(TimeStep, float)
}

bool UInterchangeChaosClothAssetFactoryNode::SetTimeStep(float AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(TimeStep, float)
}
