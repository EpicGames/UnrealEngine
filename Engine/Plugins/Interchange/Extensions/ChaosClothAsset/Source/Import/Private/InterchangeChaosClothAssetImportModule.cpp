// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeChaosClothAssetImportModule.h"

#include "InterchangeChaosClothAssetFactory.h"
#include "InterchangeChaosClothAssetImportLog.h"

#include "InterchangeAnalyticsAssetTypeTracker.h"
#include "InterchangeAnalyticsHandlerDefault.h"
#include "InterchangeManager.h"

#include "ChaosClothAsset/ClothAsset.h"

#include "Modules/ModuleManager.h"

DEFINE_LOG_CATEGORY(LogInterchangeChaosClothAssetImport);

void FInterchangeChaosClothAssetImportModule::StartupModule()
{
	using namespace UE::Interchange;

	UInterchangeManager& InterchangeManager = UInterchangeManager::GetInterchangeManager();
	InterchangeManager.RegisterFactory(UInterchangeChaosClothAssetFactory::StaticClass());
	InterchangeManager.SetAnalyticsHandlerClass(UInterchangeAnalyticsHandlerDefault::StaticClass());

	FInterchangeAnalyticsAssetTypeTracker::RegisterAssetType(UChaosClothAsset::StaticClass(), TEXT("Physics"));
}

void FInterchangeChaosClothAssetImportModule::ShutdownModule()
{
}

IMPLEMENT_MODULE(FInterchangeChaosClothAssetImportModule, InterchangeChaosClothAssetImport)
