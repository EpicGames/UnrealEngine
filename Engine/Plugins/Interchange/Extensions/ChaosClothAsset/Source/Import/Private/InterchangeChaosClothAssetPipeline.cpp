// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeChaosClothAssetPipeline.h"

#include "InterchangeChaosClothAssetDefinitions.h"
#include "InterchangeChaosClothAssetFactoryNode.h"

#include "InterchangeMaterialInstanceNode.h"
#include "InterchangeMeshFactoryNode.h"
#include "InterchangeMeshNode.h"
#include "InterchangeSceneNode.h"
#include "Nodes/InterchangeFactoryBaseNode.h"
#include "Nodes/InterchangeSourceNode.h"
#include "Nodes/InterchangeUserDefinedAttribute.h"

#include "ChaosClothAsset/ClothAsset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeChaosClothAssetPipeline)

FString UInterchangeChaosClothAssetPipeline::GetPipelineCategory(UClass* AssetClass)
{
	return TEXT("Physics");
}

void UInterchangeChaosClothAssetPipeline::ExecutePipeline(
	UInterchangeBaseNodeContainer* InBaseNodeContainer,
	const TArray<UInterchangeSourceData*>& SourceDatas,
	const FString& ContentBasePath
)
{
	using namespace UE::Interchange;
	using namespace UE::Interchange::ChaosCloth;

	if (!bImportClothAssets)
	{
		return;
	}

	// Find all cloth root nodes
	TArray<UInterchangeSceneNode*> ClothRootNodes;
	InBaseNodeContainer->IterateNodesOfType<UInterchangeSceneNode>(
		[&ClothRootNodes](const FString& NodeUid, UInterchangeSceneNode* Node)
		{
			bool bIsClothRoot = false;
			if (Node && Node->GetBooleanAttribute(*ChaosCloth::ClothRootTag, bIsClothRoot) && bIsClothRoot)
			{
				ClothRootNodes.Add(Node);
			}
		}
	);
	if (ClothRootNodes.Num() == 0)
	{
		return;
	}

	// Get whether to use subpath prefixes and suffixes, and what the should be
	// Reference: FillSubPathFromSourceNode from InterchangePipelineHelper.h
	// We have to replicate some of that code here as InterchangePipelines shouldn't depend on the Chaos Cloth modules
	FString SubPathPrefix;
	FString SubPathSuffix;
	bool bUseSubPathPrefix = false;
	bool bUseSubPathSuffix = false;
	{
		UInterchangeSourceNode* SourceNode = UInterchangeSourceNode::FindOrCreateUniqueInstance(InBaseNodeContainer);

		bUseSubPathPrefix = SourceNode->GetCustomSubPathPrefix(SubPathPrefix);

		bUseSubPathSuffix = SourceNode->GetCustomUseAssetTypeSubPathSuffix(bUseSubPathSuffix) && bUseSubPathSuffix;
		if (bUseSubPathSuffix)
		{
			SubPathSuffix = TEXT("ClothAssets");
		}
	}

	for (const UInterchangeSceneNode* ClothRootNode : ClothRootNodes)
	{
		// Construct a factory node for the cloth roots
		const FString& TranslatedNodeUid = ClothRootNode->GetUniqueID();
		const FString& TranslatedNodeDisplayLabel = ClothRootNode->GetDisplayLabel();

		const FString FactoryNodeUid = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(TranslatedNodeUid);

		UInterchangeChaosClothAssetFactoryNode* ClothFactoryNode = NewObject<UInterchangeChaosClothAssetFactoryNode>(InBaseNodeContainer);
		InBaseNodeContainer->SetupNode(ClothFactoryNode, FactoryNodeUid, TranslatedNodeDisplayLabel, EInterchangeNodeContainerType::FactoryData);

		ClothFactoryNode->AddTargetNodeUid(TranslatedNodeUid);
		ClothFactoryNode->SetImportSimulationMeshes(bImportSimulationMeshes);
		ClothFactoryNode->SetImportRenderMeshes(bImportRenderMeshes);
		ClothFactoryNode->SetDataflowGraphPath(DataflowGraphAsset);
		if (bUseSubPathPrefix || bUseSubPathSuffix)
		{
			ClothFactoryNode->SetCustomSubPath(FPaths::Combine(SubPathPrefix, SubPathSuffix));
		}

		TOptional<FString> UnusedPayloadKey = {};

		// Hoist specific solver attributes from translated node user attributes into factory node custom attributes
		if (float AirDamping = 1.0f; UInterchangeUserDefinedAttributesAPI::GetUserDefinedAttribute(ClothRootNode, SolverAirDamping, AirDamping, UnusedPayloadKey))
		{
			ClothFactoryNode->SetAirDamping(AirDamping);
		}
		if (FVector3f Gravity = {0, -9800, 0}; UInterchangeUserDefinedAttributesAPI::GetUserDefinedAttribute(ClothRootNode, SolverGravity, Gravity, UnusedPayloadKey))
		{
			ClothFactoryNode->SetGravity(Gravity);
		}
		if (int32 SubStepCount = 1; UInterchangeUserDefinedAttributesAPI::GetUserDefinedAttribute(ClothRootNode, SolverSubStepCount, SubStepCount, UnusedPayloadKey))
		{
			ClothFactoryNode->SetSubStepCount(SubStepCount);
		}
		if (float TimeStep = 0.033333f; UInterchangeUserDefinedAttributesAPI::GetUserDefinedAttribute(ClothRootNode, SolverTimeStep, TimeStep, UnusedPayloadKey))
		{
			ClothFactoryNode->SetTimeStep(TimeStep);
		}

		// Traverse the mesh factory nodes as there are a few tweaks we need to do
		{
			TArray<FString> SimMeshNodeUids;
			TArray<FString> RenderMeshNodeUids;
			UInterchangeUserDefinedAttributesAPI::GetUserDefinedAttribute(ClothRootNode, SimMeshesAttributeName, SimMeshNodeUids, UnusedPayloadKey);
			UInterchangeUserDefinedAttributesAPI::GetUserDefinedAttribute(ClothRootNode, RenderMeshesAttributeName, RenderMeshNodeUids, UnusedPayloadKey);

			auto VisitMeshFactoryNodes = [NodeContainer = InBaseNodeContainer, ClothFactoryNode](const TArray<FString>& TranslatedNodeUids)
			{
				for (const FString& TranslatedNodeUid : TranslatedNodeUids)
				{
					if (const UInterchangeMeshNode* MeshNode = Cast<UInterchangeMeshNode>(NodeContainer->GetNode(TranslatedNodeUid)))
					{
						// We have to step through the targets here instead of trying to build the factory node UID directly because in some
						// contexts (e.g. combine meshes) the factory node is named after the scene node, not the asset node...
						TArray<FString> TargetNodeUids;
						MeshNode->GetTargetNodeUids(TargetNodeUids);
						for (const FString& TargetNodeUid : TargetNodeUids)
						{
							if (UInterchangeMeshFactoryNode* FactoryNode = Cast<UInterchangeMeshFactoryNode>(NodeContainer->GetFactoryNode(TargetNodeUid)))
							{
								// If we're running and importing cloth, we'll consume these mesh nodes into the cloth data shouldn't emit
								// any actual static meshes
								FactoryNode->SetEnabled(false);

								// Also make sure we only process our cloth after we're done processing meshes (if any).
								// Not super clear if this necessary if we're disabling the factory nodes themselves,
								// but it makes sense to do this in general?
								ClothFactoryNode->AddFactoryDependencyUid(TargetNodeUid);

								// We also need to ensure we run the cloth factory after the material factory, as the FRenderMeshImport struct
								// that it uses to create the render mesh requires the full content path to the materials to be available.
								// Note that these are likely factory dependencies of the mesh factory node as well, but it seems that if we
								// disable the mesh factory node those dependencies are not factored in and we don't get them transitively
								TMap<FString, FString> SlotNameToFactoryNodeUid;
								FactoryNode->GetSlotMaterialDependencies(SlotNameToFactoryNodeUid);

								for (const TPair<FString, FString>& Pair : SlotNameToFactoryNodeUid)
								{
									const FString& MaterialFactoryNodeUid = Pair.Value;
									ClothFactoryNode->AddFactoryDependencyUid(MaterialFactoryNodeUid);
								}
							}
						}
					}
				}
			};
			VisitMeshFactoryNodes(SimMeshNodeUids);
			VisitMeshFactoryNodes(RenderMeshNodeUids);
		}
	}
}

#if WITH_EDITOR

void UInterchangeChaosClothAssetPipeline::GetSupportAssetClasses(TArray<UClass*>& PipelineSupportAssetClasses) const
{
	PipelineSupportAssetClasses.Add(UChaosClothAsset::StaticClass());
}

#endif //WITH_EDITOR
