// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "UObject/NameTypes.h"

namespace UE::Interchange::ChaosCloth
{
	// We add this to scene nodes to flag them so that we can find them later with the ChaosCloth pipeline and produce cloth factory nodes
	inline const FString ClothRootTag = TEXT("ClothRoot");

	// These are used as user attributes names on the cloth root scene node, where we store the child render and simulation meshes
	inline const FString RenderMeshesAttributeName = TEXT("ClothRenderMeshes");
	inline const FString SimMeshesAttributeName = TEXT("ClothSimMeshes");

	// Name of the variable that we'll override on the Dataflow graph with the import data
	inline const FName ImportedCollectionVariableName = TEXT("ImportedCollection");

	// Name used by the chaos cloth import for tracking the original mesh triangle indices in the combined managed array collection
	inline const FName OriginalIndicesName = TEXT("OriginalIndices");

	// Solver properties: We also store these as user attributes on the cloth root scene node, and hoist them into factory node custom attributes
	inline const FString SolverAirDamping = TEXT("airDamping");
	inline const FString SolverGravity = TEXT("gravity");
	inline const FString SolverSubStepCount = TEXT("subStepCount");
	inline const FString SolverTimeStep = TEXT("timeStep");

}	 // namespace UE::Interchange::ChaosCloth
