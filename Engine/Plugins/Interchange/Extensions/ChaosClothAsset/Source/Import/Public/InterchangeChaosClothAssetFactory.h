// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InterchangeFactoryBase.h"
#include "Mesh/InterchangeMeshPayload.h"

#include "InterchangeChaosClothAssetFactory.generated.h"

#define UE_API INTERCHANGECHAOSCLOTHASSETIMPORT_API

class UInterchangeChaosClothAssetPayloadData;
struct FManagedArrayCollection;
namespace UE::Interchange
{
	struct FMeshPayload;
}

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeChaosClothAssetFactory : public UInterchangeFactoryBase
{
	GENERATED_BODY()

public:
	// Begin UInterchangeFactoryBase interface
	UE_API virtual UClass* GetFactoryClass() const override;
	virtual EInterchangeFactoryAssetType GetFactoryAssetType() override { return EInterchangeFactoryAssetType::Physics; }
	UE_API virtual void CreatePayloadTasks(const FImportAssetObjectParams& Arguments, bool bAsync, TArray<TSharedPtr<UE::Interchange::FInterchangeTaskBase>>& PayloadTasks) override;
	UE_API virtual FImportAssetResult ImportAsset_Async(const FImportAssetObjectParams& Arguments);
	UE_API virtual FImportAssetResult BeginImportAsset_GameThread(const FImportAssetObjectParams& Arguments) override;
	UE_API virtual void SetupObject_GameThread(const FSetupObjectParams& Arguments) override;
	UE_API virtual bool GetSourceFilenames(const UObject* Object, TArray<FString>& OutSourceFilenames) const override;
	UE_API virtual bool SetSourceFilename(const UObject* Object, const FString& SourceFilename, int32 SourceIndex) const override;
	UE_API virtual void BackupSourceData(const UObject* Object) const override;
	UE_API virtual void ReinstateSourceData(const UObject* Object) const override;
	UE_API virtual void ClearBackupSourceData(const UObject* Object) const override;
	// End UInterchangeFactoryBase interface

private:
	// All the render data that we extract from a single render mesh prim
	struct FRenderMeshPayloadData
	{
		FString MeshFactoryNodeUid;
		UE::Interchange::FMeshPayload MeshPayload;
		TMap<FName, TSet<int32>> RenderPatterns;
	};
	TArray<FRenderMeshPayloadData> RenderMeshPayloadDataArray;
	TArray<TSharedPtr<FManagedArrayCollection>> SimMeshPayloadDataArray;

	// Basically the final output of the factory, is added to the Dataflow graph as a variable override
	// Shared ownership to easily meet the facade interfaces
	TSharedPtr<FManagedArrayCollection> CombinedCollection;
};

#undef UE_API
