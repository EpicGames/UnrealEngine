// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Nodes/InterchangeFactoryBaseNode.h"

#include "InterchangeChaosClothAssetFactoryNode.generated.h"

#define UE_API INTERCHANGECHAOSCLOTHASSETIMPORT_API

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeChaosClothAssetFactoryNode : public UInterchangeFactoryBaseNode
{
	GENERATED_BODY()

public:
	UE_API virtual FString GetTypeName() const override;
	UE_API virtual class UClass* GetObjectClass() const override;

public:
	/** Gets whether to import the simulation mesh into the final ClothAsset */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Chaos | Cloth")
	UE_API bool GetImportSimulationMeshes(bool& OutImportSimulationMeshes) const;

	/** Sets whether to import the simulation mesh into the final ClothAsset */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Chaos | Cloth")
	UE_API bool SetImportSimulationMeshes(bool ImportSimulationMeshes);

	/** Gets whether to import the render mesh into the final ClothAsset */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Chaos | Cloth")
	UE_API bool GetImportRenderMeshes(bool& OutImportRenderMeshes) const;

	/** Sets whether to import the render mesh into the final ClothAsset */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Chaos | Cloth")
	UE_API bool SetImportRenderMeshes(bool ImportRenderMeshes);

	/** Gets the content path to the Dataflow graph template to instantiate into the generated cloth asset */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Chaos | Cloth")
	UE_API bool GetDataflowGraphPath(FSoftObjectPath& OutDataflowGraphPath) const;

	/** Sets the content path to the Dataflow graph template to instantiate into the generated cloth asset */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Chaos | Cloth")
	UE_API bool SetDataflowGraphPath(const FSoftObjectPath& DataflowGraphPath);

public: // Solver properties
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Chaos | Cloth | Solver")
	UE_API bool GetAirDamping(float& OutAirDamping) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Chaos | Cloth | Solver")
	UE_API bool SetAirDamping(float AirDamping);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Chaos | Cloth | Solver")
	UE_API bool GetGravity(FVector3f& OutGravity) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Chaos | Cloth | Solver")
	UE_API bool SetGravity(const FVector3f& Gravity);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Chaos | Cloth | Solver")
	UE_API bool GetSubStepCount(int32& OutSubStepCount) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Chaos | Cloth | Solver")
	UE_API bool SetSubStepCount(int32 SubStepCount);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Chaos | Cloth | Solver")
	UE_API bool GetTimeStep(float& OutTimeStep) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Chaos | Cloth | Solver")
	UE_API bool SetTimeStep(float TimeStep);

private:
	IMPLEMENT_NODE_ATTRIBUTE_KEY(ImportSimulationMeshes);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(ImportRenderMeshes);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(DataflowGraphPath);

	IMPLEMENT_NODE_ATTRIBUTE_KEY(AirDamping);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(Gravity);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(SubStepCount);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(TimeStep);
};

#undef UE_API
