// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InterchangeGenericPayloadData.h"

#include "InterchangeChaosClothAssetPayloadData.generated.h"

struct FManagedArrayCollection;

/** Cloth-specific payload data provided for a Mesh prim */
UCLASS()
class UInterchangeChaosClothAssetPayloadData : public UInterchangeGenericPayloadData
{
	GENERATED_BODY()

public:
	/**
	 * Collection extracted from the Mesh. Currently this is used only for simulation data.
	 * Owned by SharedPtr to easily meet Cloth Facade interfaces
	 */
	TSharedPtr<FManagedArrayCollection> Collection;

	/**
	 * Mesh render data is mostly provided via the standard Interchange mesh payloads.
	 * We need this one additional bit of data for cloth render meshes however
	 */
	TMap<FName, TSet<int32>> RenderPatterns;
};
