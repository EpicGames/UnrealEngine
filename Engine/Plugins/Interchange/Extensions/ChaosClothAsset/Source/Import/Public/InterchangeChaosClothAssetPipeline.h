// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InterchangePipelineBase.h"

#include "UObject/ObjectMacros.h"

#include "InterchangeChaosClothAssetPipeline.generated.h"

#define UE_API INTERCHANGECHAOSCLOTHASSETIMPORT_API

class UInterchangeBaseNodeContainer;

UCLASS(MinimalAPI, BlueprintType, editinlinenew)
class UInterchangeChaosClothAssetPipeline : public UInterchangePipelineBase
{
	GENERATED_BODY()

public:
	static UE_API FString GetPipelineCategory(UClass* AssetClass);

	/** The name of the pipeline that will be displayed in the import dialog. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Cloth Assets", meta = (StandAlonePipelineProperty = "True", PipelineInternalEditionData = "True"))
	FString PipelineDisplayName;

	/** If enabled, import all cloth assets found in the source. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Cloth Assets")
	bool bImportClothAssets = true;

	/** Whether to add simulation data to the produced cloth collection */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Cloth Assets", meta = (EditCondition = "bImportClothAssets"))
	bool bImportSimulationMeshes = true;

	/** Whether to add render data to the produced cloth collection */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Cloth Assets", meta = (EditCondition = "bImportClothAssets"))
	bool bImportRenderMeshes = true;

	/**
	 * Template of the dataflow graph that should be used on the spawned cloth assets.
	 *
	 * This pipeline will only set this on the produced cloth factory nodes. The cloth factory that ultimately will instantiate the template
	 * graph and add it to the spawned cloth asset.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Cloth Assets", meta = (StandAlonePipelineProperty = "True", PipelineInternalEditionData = "True", AllowedClasses = "/Script/DataflowEngine.Dataflow"))
	FSoftObjectPath DataflowGraphAsset;

public:
	UE_API virtual void ExecutePipeline(UInterchangeBaseNodeContainer* BaseNodeContainer, const TArray<UInterchangeSourceData*>& SourceDatas, const FString& ContentBasePath) override;
#if WITH_EDITOR
	UE_API virtual void GetSupportAssetClasses(TArray<UClass*>& PipelineSupportAssetClasses) const override;
#endif
};

#undef UE_API
